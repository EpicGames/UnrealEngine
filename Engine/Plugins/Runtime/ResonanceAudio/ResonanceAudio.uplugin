{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "Resonance Audio",
  "Description": "3D audio spatialization and room acoustics simulation plugin by Google.",
  "Category": "Audio",
  "CreatedBy": "Google",
  "CreatedByURL": "https://developers.google.com/resonance-audio",
  "DocsURL": "https://developers.google.com/resonance-audio/develop/unreal/getting-started",
  "MarketplaceURL": "",
  "SupportURL": "https://developers.google.com/resonance-audio/community/connect",
  "EnabledByDefault": true,
  "CanContainContent": true,
  "IsBetaVersion": true,
  "Installed": false,
  "Modules": [
	{
		"Name": "ResonanceAudio",
		"Type": "Runtime",
		"LoadingPhase": "PreDefault",
		"PlatformAllowList": [
			"Android",
			"IOS",
			"Linux",
			"Mac",
			"Win64"
		]
	},
	{
		"Name": "ResonanceAudioEditor",
		"Type": "Editor",
		"LoadingPhase": "PostEngineInit",
		"PlatformAllowList": [
			"Linux",
			"Mac",
			"Win64"
		]
	}
  ],
  "Plugins": [
	{
		"Name": "ProceduralMeshComponent",
		"Enabled": true
	}
  ]
}
