//
// Copyright (C) Google Inc. 2017. All rights reserved.
//
#include "ResonanceAudioAmbisonics.h"
#include "ResonanceAudioAmbisonicsSettings.h"
#include "ResonanceAudioAmbisonicsSettingsProxy.h"
#include "ResonanceAudioPluginListener.h"
#include "SoundFieldRendering.h"
#include "AudioPluginUtilities.h"

namespace ResonanceAudioUtils
{
	bool IsResonanceAudioTheCurrentSpatializationPlugin()
	{
#if WITH_EDITOR
		static FString ResonanceDisplayName = FString(TEXT("Resonance Audio"));
		return AudioPluginUtilities::GetDesiredPluginName(EAudioPlugin::REVERB).Equals(ResonanceDisplayName);
#else
		// For non editor situations, we can cache whether this is the current plugin or not the first time we check.
		static FString ResonanceDisplayName = FString(TEXT("Resonance Audio"));
		static bool bCheckedSpatializationPlugin = false;
		static bool bIsResonanceCurrentSpatiatizationPlugin = false;
		
		if (!bCheckedSpatializationPlugin)
		{
			bIsResonanceCurrentSpatiatizationPlugin = AudioPluginUtilities::GetDesiredPluginName(EAudioPlugin::REVERB).Equals(ResonanceDisplayName);
			bCheckedSpatializationPlugin = true;
		}

		return bIsResonanceCurrentSpatiatizationPlugin;
#endif
	}
}

namespace ResonanceAudio
{
	class FEncoder : public ISoundfieldEncoderStream
	{
	private:
		FResonanceAudioApiSharedPtr ResonanceSystem;

		TArray<vraudio::SourceId> SourceIdArray;
		TArray<Audio::AlignedFloatBuffer> DeinterleavedBuffers;
		int32 NumChannels;

		void ChangeNumChannels(int32 NewNumChannels, vraudio::RenderingMode InRenderingMode)
		{
			check(ResonanceSystem);
			for (vraudio::SourceId Source : SourceIdArray)
			{
				ResonanceSystem->DestroySource(Source);
			}

			DeinterleavedBuffers.Reset();

			if (NewNumChannels > 1)
			{
				for (int32 Index = 0; Index < NewNumChannels; Index++)
				{
					DeinterleavedBuffers.AddDefaulted();
					SourceIdArray.Add(ResonanceSystem->CreateSoundObjectSource(InRenderingMode));
				}
			}

			NumChannels = NewNumChannels;
		}

	public:
		FEncoder(FResonanceAudioApiSharedPtr InResonanceSystem, int32 InNumChannels, vraudio::RenderingMode InRenderingMode)
			: ResonanceSystem(InResonanceSystem)
			, NumChannels(InNumChannels)
		{
			check(ResonanceSystem);
			check(NumChannels > 0);
			
			for (int32 Index = 0; Index < NumChannels; Index++)
			{
				DeinterleavedBuffers.AddDefaulted();
				SourceIdArray.Add(ResonanceSystem->CreateSoundObjectSource(InRenderingMode));
			}
		}

		virtual ~FEncoder()
		{
			check(ResonanceSystem);
			for (vraudio::SourceId Source : SourceIdArray)
			{
				ResonanceSystem->DestroySource(Source);
			}
		}


		virtual void Encode(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) override
		{
			OutputData.Reset();
			EncodeAndMixIn(InputData, OutputData);
		}


		virtual void EncodeAndMixIn(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) override
		{
			check(ResonanceSystem);
			FResonancePacket& OutPacket = DowncastSoundfieldRef<FResonancePacket>(OutputData);
			OutPacket.SetResonanceApi(ResonanceSystem.Get());

			const FResonanceAmbisonicsSettingsProxy& Settings = DowncastSoundfieldRef<const FResonanceAmbisonicsSettingsProxy>(InputData.InputSettings);

			if (NumChannels == 1)
			{
				check(SourceIdArray.Num() == 1);

				// Don't use the DeinterleavedBuffers, submit the audio directly.
				ResonanceSystem->SetInterleavedBuffer(SourceIdArray[0], InputData.AudioBuffer.GetData(), 1, InputData.AudioBuffer.Num());
			}
			else
			{
				check(InputData.PositionalData.ChannelPositions);
				const TArray<Audio::FChannelPositionInfo>& ChannelPositions = *InputData.PositionalData.ChannelPositions;


				if (ChannelPositions.Num() != NumChannels)
				{
					ChangeNumChannels(ChannelPositions.Num(), Settings.RenderingMode);
				}

				const int32 NumFrames = InputData.AudioBuffer.Num() / InputData.NumChannels;
				check(DeinterleavedBuffers.Num() == InputData.NumChannels);

				for (int32 ChannelIndex = 0; ChannelIndex < InputData.NumChannels; ChannelIndex++)
				{
					// Deinterleave the audio for this channel.
					Audio::AlignedFloatBuffer& DeinterleavedBuffer = DeinterleavedBuffers[ChannelIndex];
					DeinterleavedBuffer.Reset();
					DeinterleavedBuffer.AddUninitialized(NumFrames);

					for (int32 FrameIndex = 0; FrameIndex < NumFrames; FrameIndex++)
					{
						DeinterleavedBuffer[FrameIndex] = InputData.AudioBuffer[FrameIndex * NumChannels + ChannelIndex];
					}

					// Set the position and audio buffer for this source.
					vraudio::SourceId& SourceId = SourceIdArray[ChannelIndex];
					FVector ResonancePosition = ConvertToResonanceAudioCoordinates(ChannelPositions[ChannelIndex]);
					ResonanceSystem->SetSourcePosition(SourceId, ResonancePosition.X, ResonancePosition.Y, ResonancePosition.Z);
					ResonanceSystem->SetInterleavedBuffer(SourceId, DeinterleavedBuffer.GetData(), 1, NumFrames);
				}
			}
		}
	};

	class FDecoder : public ISoundfieldDecoderStream
	{
	private:
		const bool bShouldOutputAudio;

	public:

		FDecoder(bool bInShouldOutputAudio)
			: bShouldOutputAudio(bInShouldOutputAudio)
		{}

		FDecoder() = delete;

		virtual void Decode(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) override
		{
			DecodeAndMixIn(InputData, OutputData);
		}


		virtual void DecodeAndMixIn(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) override
		{
			// If the reverb plugin is enabled, FillInterleavedOutputBuffer will be called there.
			if (bShouldOutputAudio)
			{
				const int32 NumChannels = InputData.PositionalData.NumChannels;
				const FResonancePacket& Packet = DowncastSoundfieldRef<const FResonancePacket>(InputData.SoundfieldBuffer);
				OutputData.AudioBuffer.Reset();
				OutputData.AudioBuffer.AddZeroed(NumChannels * InputData.NumFrames);

				if (vraudio::ResonanceAudioApi* ResonanceSystem = const_cast<vraudio::ResonanceAudioApi*>(Packet.GetResonanceApi()))
				{
					ResonanceSystem->FillInterleavedOutputBuffer(NumChannels, InputData.NumFrames, OutputData.AudioBuffer.GetData());
				}
			}
		}

	};

	/**
	 * This class takes an audio stream in Unreal's internal ambisonics format and mixes it into Resonance's ambisonics bed.
	 */
	class FAmbisonicsTranscoder : public ISoundfieldTranscodeStream
	{
	private:
		FResonanceAudioApiSharedPtr ResonanceSystem;
		vraudio::SourceId AmbisonicsSourceId;
		int32 NumChannels;

		void ChangeNumChannels(int32 NewNumChannels)
		{
			check(ResonanceSystem);
			ResonanceSystem->DestroySource(AmbisonicsSourceId);

			NumChannels = NewNumChannels;
			AmbisonicsSourceId = ResonanceSystem->CreateAmbisonicSource(NumChannels);
		}

	public:
		FAmbisonicsTranscoder(FResonanceAudioApiSharedPtr InResonanceSystem, int32 InNumChannels, bool bInShouldDestroyResonanceSystem)
			: ResonanceSystem(InResonanceSystem)
			, NumChannels(InNumChannels)
		{
			check(ResonanceSystem);
			AmbisonicsSourceId = ResonanceSystem->CreateAmbisonicSource(NumChannels);
		}

		virtual ~FAmbisonicsTranscoder()
		{
			check(ResonanceSystem);
			ResonanceSystem->DestroySource(AmbisonicsSourceId);
		}

		virtual void Transcode(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& OutputData, const ISoundfieldEncodingSettingsProxy& OutputSettings) override
		{
			TranscodeAndMixIn(InputData, InputSettings, OutputData, OutputSettings);
		}


		virtual void TranscodeAndMixIn(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& PacketToSumTo, const ISoundfieldEncodingSettingsProxy& OutputSettings) override
		{
			const FAmbisonicsSoundfieldBuffer& AmbisonicsBuffer = DowncastSoundfieldRef<const FAmbisonicsSoundfieldBuffer>(InputData);
			FResonancePacket& OutPacket = DowncastSoundfieldRef<FResonancePacket>(PacketToSumTo);
			
			check(ResonanceSystem);

			if (AmbisonicsBuffer.NumChannels > 0)
			{
				if (AmbisonicsBuffer.NumChannels != NumChannels)
				{
					ChangeNumChannels(AmbisonicsBuffer.NumChannels);
				}

				ResonanceSystem->SetInterleavedBuffer(AmbisonicsSourceId, AmbisonicsBuffer.AudioBuffer.GetData(), NumChannels, AmbisonicsBuffer.AudioBuffer.Num() / NumChannels);
			}

			OutPacket.SetResonanceApi(ResonanceSystem.Get());
		}

	};

	class FResonanceMixer : public ISoundfieldMixerStream
	{
	public:

		virtual void MixTogether(const FSoundfieldMixerInputData& InputData, ISoundfieldAudioPacket& PacketToSumTo) override
		{
			// All this does is forward the resonance system to the PacketToSumTo.
			const FResonancePacket& InPacket = DowncastSoundfieldRef<const FResonancePacket>(InputData.InputPacket);
			FResonancePacket& OutPacket = DowncastSoundfieldRef<FResonancePacket>(PacketToSumTo);

			OutPacket.SetResonanceApi(InPacket.GetResonanceApi());
		}

	};

	FAmbisonicsFactory::FAmbisonicsFactory()
	{
		ISoundfieldFactory::RegisterSoundfieldFormat(this);
	}

	FAmbisonicsFactory::~FAmbisonicsFactory()
	{
		ISoundfieldFactory::UnregisterSoundfieldFormat(this);
	}

	FName FAmbisonicsFactory::GetSoundfieldFormatName()
	{
		static FName ResonanceSoundfieldFormat = TEXT("Resonance Binaural Spatialization");
		return ResonanceSoundfieldFormat;
	}

	TUniquePtr<ISoundfieldEncoderStream> FAmbisonicsFactory::CreateEncoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings)
	{
		const FResonanceAmbisonicsSettingsProxy& Settings = DowncastSoundfieldRef<const FResonanceAmbisonicsSettingsProxy>(InitialSettings);
		FAudioDevice* AudioDevice = InitInfo.AudioDevicePtr;

		check(AudioDevice);

		FResonanceAudioApiSharedPtr ResonanceApi = FResonanceAudioPluginListener::GetResonanceAPIForAudioDevice(AudioDevice);

		// If there's no globablly available Resonance api, spawn a new one.
		if (!ResonanceApi.IsValid())
		{
			ResonanceApi = CreateNewResonanceApiInstance(AudioDevice, InitInfo);
			return TUniquePtr<ISoundfieldEncoderStream>(new FEncoder(ResonanceApi, InitInfo.NumOutputChannels, Settings.RenderingMode));
		}
		else
		{
			return TUniquePtr<ISoundfieldEncoderStream>(new FEncoder(ResonanceApi, InitInfo.NumOutputChannels, Settings.RenderingMode));
		}
	}

	TUniquePtr<ISoundfieldDecoderStream> FAmbisonicsFactory::CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings)
	{
		const bool bShouldOutputResonanceAudioThroughSubmix = !ResonanceAudioUtils::IsResonanceAudioTheCurrentSpatializationPlugin();
		UE_CLOGF(!bShouldOutputResonanceAudioThroughSubmix, LogAudio, Display, "Since Resonance Audio is the currently selected spatialization plugin. The output of this plugin will be sent through the Reverb Plugin submix.");
		return TUniquePtr<ISoundfieldDecoderStream>(new FDecoder(bShouldOutputResonanceAudioThroughSubmix));
	}

	TUniquePtr<ISoundfieldTranscodeStream> FAmbisonicsFactory::CreateTranscoderStream(const FName SourceFormat, const ISoundfieldEncodingSettingsProxy& InitialSourceSettings, const FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& InitialDestinationSettings, const FAudioPluginInitializationParams& InitInfo)
	{
		check(SourceFormat == GetUnrealAmbisonicsFormatName());
		check(DestinationFormat == GetSoundfieldFormatName());

		const FAmbisonicsSoundfieldSettings& SourceSettings = DowncastSoundfieldRef<const FAmbisonicsSoundfieldSettings>(InitialSourceSettings);
		const FResonanceAmbisonicsSettingsProxy& DestinationSettings = DowncastSoundfieldRef<const FResonanceAmbisonicsSettingsProxy>(InitialDestinationSettings);

		int32 NumChannels = (SourceSettings.Order + 1) * (SourceSettings.Order + 1);

		FAudioDevice* AudioDevice = InitInfo.AudioDevicePtr;
		
		FResonanceAudioApiSharedPtr ResonanceApi = FResonanceAudioPluginListener::GetResonanceAPIForAudioDevice(AudioDevice);

		// If there's no globabally available Resonance api, spawn a new one.
		if (!ResonanceApi.IsValid())
		{
			ResonanceApi = CreateNewResonanceApiInstance(AudioDevice, InitInfo);
			return TUniquePtr<ISoundfieldTranscodeStream>(new FAmbisonicsTranscoder(ResonanceApi, NumChannels, true));
		}
		else
		{
			return TUniquePtr<ISoundfieldTranscodeStream>(new FAmbisonicsTranscoder(ResonanceApi, NumChannels, false));
		}
	}

	TUniquePtr<ISoundfieldMixerStream> FAmbisonicsFactory::CreateMixerStream(const ISoundfieldEncodingSettingsProxy& InitialSettings)
	{
		return TUniquePtr<ISoundfieldMixerStream>(new FResonanceMixer());
	}

	TUniquePtr<ISoundfieldAudioPacket> FAmbisonicsFactory::CreateEmptyPacket()
	{
		return TUniquePtr<ISoundfieldAudioPacket>(new FResonancePacket(nullptr));
	}

	bool FAmbisonicsFactory::IsTranscodeRequiredBetweenSettings(const ISoundfieldEncodingSettingsProxy& SourceSettings, const ISoundfieldEncodingSettingsProxy& DestinationSettings)
	{
		// Since all audio is encoded to third order ambisonics upon encoding for Resonance, we don't need to transcode anything.
		return false;
	}

	bool FAmbisonicsFactory::CanTranscodeFromSoundfieldFormat(FName SourceFormat, const ISoundfieldEncodingSettingsProxy& SourceEncodingSettings)
	{
		FName AmbisonicsSoundfieldFormatName = GetUnrealAmbisonicsFormatName();
		if (SourceFormat == AmbisonicsSoundfieldFormatName)
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	bool FAmbisonicsFactory::CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings)
	{
		return false;
	}

	UClass* FAmbisonicsFactory::GetCustomEncodingSettingsClass() const
	{
		return UResonanceAudioSoundfieldSettings::StaticClass();
	}

	const USoundfieldEncodingSettingsBase* FAmbisonicsFactory::GetDefaultEncodingSettings()
	{
		return GetDefault<UResonanceAudioSoundfieldSettings>();
	}

	FResonanceAudioApiSharedPtr FAmbisonicsFactory::CreateNewResonanceApiInstance(FAudioDevice* AudioDevice, const FAudioPluginInitializationParams& InInitInfo)
	{
		const size_t FramesPerBuffer = static_cast<size_t>(AudioDevice->GetBufferLength());
		const int SampleRate = static_cast<int>(AudioDevice->GetSampleRate());

		vraudio::ResonanceAudioApi* ApiPtr = CreateResonanceAudioApi(FResonanceAudioModule::GetResonanceAudioDynamicLibraryHandle(), 2 /* num channels */, FramesPerBuffer, SampleRate);

		FResonanceAudioApiSharedPtr ResonanceApi(ApiPtr);

		FResonanceAudioPluginListener::SetResonanceAPIForAudioDevice(AudioDevice, ResonanceApi);

		return ResonanceApi;
	}

	void FResonancePacket::Serialize(FArchive& Ar)
	{
		// Since the FResonancePacket is just a handle to the API, 
		// we don't have anything to serialize.
	}

	TUniquePtr<ISoundfieldAudioPacket> FResonancePacket::Duplicate() const
	{
		FResonancePacket* Packet = new FResonancePacket(ResonanceSystem);
		return TUniquePtr<ISoundfieldAudioPacket>(Packet);
	}

	void FResonancePacket::Reset()
	{
		ResonanceSystem = nullptr;
	}

}
