//
// Copyright (C) Google Inc. 2017. All rights reserved.
//
#pragma once

#include "IAudioExtensionPlugin.h"
#include "ResonanceAudioCommon.h"

namespace ResonanceAudio
{
	// Packet passed between Resonance submixes. All it holds onto is a raw pointer
	// to the Resonance system used to encode whatever sources were passed in.
	class FResonancePacket : public ISoundfieldAudioPacket
	{
	private:
		FResonancePacket() {}

		vraudio::ResonanceAudioApi* ResonanceSystem;
	public:

		FResonancePacket(vraudio::ResonanceAudioApi* InSystem)
			: ResonanceSystem(InSystem)
		{
		}

		virtual void Serialize(FArchive& Ar) override;
		virtual TUniquePtr<ISoundfieldAudioPacket> Duplicate() const override;
		virtual void Reset() override;

		vraudio::ResonanceAudioApi* GetResonanceApi() const
		{
			return ResonanceSystem;
		}

		void SetResonanceApi(vraudio::ResonanceAudioApi* InResonanceSystem)
		{
			check(InResonanceSystem);
			ResonanceSystem = InResonanceSystem;
		}

		vraudio::ResonanceAudioApi* GetResonanceApi()
		{
			return ResonanceSystem;
		}
	};

	class FAmbisonicsFactory : public ISoundfieldFactory
	{
	public:
		FAmbisonicsFactory();
		virtual ~FAmbisonicsFactory();

		// This function is called by the ResonanceAudioPluginListener as soon as the ResonanceAudioApi is initialized.
		void SetResonanceAudioApi(uint32 AudioDeviceID, FResonanceAudioApiSharedPtr InResonanceAudioApi) 
		{
			FResonanceAudioApiSharedPtr& ApiPtr = ResonanceAudioAPIMap.FindOrAdd(AudioDeviceID);
			ApiPtr = InResonanceAudioApi;
		};

		// Begin IAmbisonicsMixer
		virtual FName GetSoundfieldFormatName() override;
		virtual TUniquePtr<ISoundfieldEncoderStream> CreateEncoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
		virtual TUniquePtr<ISoundfieldDecoderStream> CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
		virtual TUniquePtr<ISoundfieldTranscodeStream> CreateTranscoderStream(const FName SourceFormat, const ISoundfieldEncodingSettingsProxy& InitialSourceSettings, const FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& InitialDestinationSettings, const FAudioPluginInitializationParams& InitInfo) override;
		virtual TUniquePtr<ISoundfieldMixerStream> CreateMixerStream(const ISoundfieldEncodingSettingsProxy& InitialSettings) override;
		virtual TUniquePtr<ISoundfieldAudioPacket> CreateEmptyPacket() override;
		virtual bool IsTranscodeRequiredBetweenSettings(const ISoundfieldEncodingSettingsProxy& SourceSettings, const ISoundfieldEncodingSettingsProxy& DestinationSettings) override;
		virtual bool CanTranscodeFromSoundfieldFormat(FName SourceFormat, const ISoundfieldEncodingSettingsProxy& SourceEncodingSettings) override;
		virtual bool CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings) override;
		virtual UClass* GetCustomEncodingSettingsClass() const override;
		virtual const USoundfieldEncodingSettingsBase* GetDefaultEncodingSettings() override;
		//~ IAmbisonicsMixer

	private:
		FResonanceAudioApiSharedPtr CreateNewResonanceApiInstance(FAudioDevice* AudioDevice, const FAudioPluginInitializationParams& InInitInfo);

		// Map of which audio engine StreamIds map to which Resonance SourceIds.
		TMap<uint32, FResonanceAudioApiSharedPtr> ResonanceAudioAPIMap;
	};
}
