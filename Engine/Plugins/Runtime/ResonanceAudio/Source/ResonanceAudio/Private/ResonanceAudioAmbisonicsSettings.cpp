//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioAmbisonicsSettings.h"

#include "IAudioExtensionPlugin.h"
#include "HAL/LowLevelMemTracker.h"
#include "ResonanceAudioAmbisonicsSettingsProxy.h"
#include "ResonanceAudioCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioAmbisonicsSettings)

TUniquePtr<ISoundfieldEncodingSettingsProxy> UResonanceAudioSoundfieldSettings::GetProxy() const
{
	AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE
	FResonanceAmbisonicsSettingsProxy* Proxy = new FResonanceAmbisonicsSettingsProxy();

	switch (RenderMode)
	{
		case EResonanceRenderMode::StereoPanning:
		{
			Proxy->RenderingMode = vraudio::kStereoPanning;
			break;
		}
		case EResonanceRenderMode::BinauralLowQuality:
		{
			Proxy->RenderingMode = vraudio::kBinauralLowQuality;
			break;
		}
		case EResonanceRenderMode::BinauralMediumQuality:
		{
			Proxy->RenderingMode = vraudio::kBinauralMediumQuality;
			break;
		}
		case EResonanceRenderMode::BinauralHighQuality:
		{
			Proxy->RenderingMode = vraudio::kBinauralHighQuality;
			break;
		}
		case EResonanceRenderMode::RoomEffectsOnly:
		{
			Proxy->RenderingMode = vraudio::kRoomEffectsOnly;
			break;
		}
		default:
		{
			checkNoEntry();
		}
	}

	return TUniquePtr<ISoundfieldEncodingSettingsProxy>(Proxy);
}
