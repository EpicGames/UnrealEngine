//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "ResonanceAudioAmbisonicsSettings.h"

#include "IAudioExtensionPlugin.h"
#include "HAL/LowLevelMemTracker.h"
#include "ResonanceAudioCommon.h"

class FResonanceAmbisonicsSettingsProxy : public ISoundfieldEncodingSettingsProxy
{
public:
	vraudio::RenderingMode RenderingMode;

	virtual uint32 GetUniqueId() const override
	{
		return static_cast<uint32>(RenderingMode);
	}

	virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> Duplicate() const override
	{
		AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE
		FResonanceAmbisonicsSettingsProxy* Proxy = new FResonanceAmbisonicsSettingsProxy();
		Proxy->RenderingMode = RenderingMode;
		return TUniquePtr<ISoundfieldEncodingSettingsProxy>(Proxy);
	}
};
