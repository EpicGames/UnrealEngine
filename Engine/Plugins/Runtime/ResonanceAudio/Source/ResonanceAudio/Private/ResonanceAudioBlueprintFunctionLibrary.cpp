//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioBlueprintFunctionLibrary.h"
#include "AudioPluginUtilities.h"
#include "ResonanceAudioModule.h"
#include "ResonanceAudioReverb.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioBlueprintFunctionLibrary)

// Convenience function to obtain the Reverb Plugin pointer currently used by the given AudioDevice.
static ResonanceAudio::FResonanceAudioReverb* GetReverbPluginFromAudioDevice(FAudioDevice* InDevice)
{
	return (ResonanceAudio::FResonanceAudioReverb*) (InDevice->ReverbPluginInterface).Get();
}

void UResonanceAudioBlueprintFunctionLibrary::SetGlobalReverbPreset(UResonanceAudioReverbPluginPreset* InPreset)
{
	// Retrieve all audio devices registered by the module.
	ResonanceAudio::FResonanceAudioModule* Module = &FModuleManager::GetModuleChecked<ResonanceAudio::FResonanceAudioModule>("ResonanceAudio");
	TArray<FAudioDevice*>& RegisteredAudioDevices = Module->GetRegisteredAudioDevices();

	// Iterate through all audio devices and get the Reverb plugins registered with them.
	for (FAudioDevice* Device : RegisteredAudioDevices)
	{
		ResonanceAudio::FResonanceAudioReverb* Plugin = GetReverbPluginFromAudioDevice(Device);

		if (Plugin != nullptr)
		{
			Plugin->SetGlobalReverbPluginPreset(InPreset);
		}
	}
}

UResonanceAudioReverbPluginPreset* UResonanceAudioBlueprintFunctionLibrary::GetGlobalReverbPreset()
{
	UResonanceAudioReverbPluginPreset* GlobalPreset = nullptr;

	//Retrieve all audio devices registered by the module.
	ResonanceAudio::FResonanceAudioModule* Module = &FModuleManager::GetModuleChecked<ResonanceAudio::FResonanceAudioModule>("ResonanceAudio");
	TArray<FAudioDevice*> const& RegisteredAudioDevices = Module->GetRegisteredAudioDevices();

	// Iterate through all audio devices and get the Reverb plugins registered with them.
	for (FAudioDevice* Device : RegisteredAudioDevices)
	{
		// Since SetGlobalReverbSettings applies the Global Reverb Settings to all audio devices currently active,
		// We can just retrieve the Global Reverb Settings from the first audio device that has them.
		ResonanceAudio::FResonanceAudioReverb* Plugin = GetReverbPluginFromAudioDevice(Device);

		if (Plugin != nullptr)
		{
			GlobalPreset = Plugin->GetGlobalReverbPluginPreset();
		}
	}
	return GlobalPreset;
}

