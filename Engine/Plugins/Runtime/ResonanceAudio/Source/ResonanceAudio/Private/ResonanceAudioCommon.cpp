//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioCommon.h"
#include "ResonanceAudioConstants.h"
#include "Misc/Paths.h"
#include "EngineUtils.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "AudioMixerDevice.h"

DEFINE_LOG_CATEGORY(LogResonanceAudio);

namespace ResonanceAudio
{

	void* LoadResonanceAudioDynamicLibrary()
	{
		auto LoadLibrary = [](const FString& FilePath)
		{
			FString DynamicLibraryToLoad = FPaths::EngineDir() / TEXT("Source/ThirdParty/ResonanceAudioApi/lib") / FilePath;
			void* DynamicLibraryHandle = nullptr;

			UE_LOGF(LogResonanceAudio, Log, "Attempting to load %ls", *DynamicLibraryToLoad);

			if (FPaths::FileExists(DynamicLibraryToLoad))
			{
				DynamicLibraryHandle = FPlatformProcess::GetDllHandle(*DynamicLibraryToLoad);
			}
			else
			{
				UE_LOGF(LogResonanceAudio, Log, "File does not exist. %ls", *DynamicLibraryToLoad);
			}
	
			if (!DynamicLibraryHandle)
			{
				UE_LOGF(LogResonanceAudio, Log, "Unable to load %ls.", *FPaths::ConvertRelativePathToFull(DynamicLibraryToLoad));
			}
			else
			{
				UE_LOGF(LogResonanceAudio, Log, "Loaded %ls.", *DynamicLibraryToLoad);
			}
	
			return DynamicLibraryHandle;
		};

#if PLATFORM_WINDOWS
		return LoadLibrary(TEXT("win_x64/vraudio.dll"));
#elif PLATFORM_MAC
		return LoadLibrary(TEXT("darwin/libvraudio.dylib"));
#elif PLATFORM_ANDROID || PLATFORM_IOS
		 // Not necessary on this platform.
		return nullptr;
#elif PLATFORM_LINUX
		return LoadLibrary(TEXT("linux/libvraudio.so"));
#else
		UE_LOGF(LogResonanceAudio, Error, "Unsupported Platform. Supported platforms are ANDROID, IOS, LINUX, MAC and WINDOWS");
		return nullptr;
#endif  // PLATFORM_WINDOWS
	}

	vraudio::ResonanceAudioApi* CreateResonanceAudioApi(void* DynamicLibraryHandle, size_t NumChannels, size_t NumFrames, int SampleRate) {

		 // For the static case, or for Android.
		return vraudio::CreateResonanceAudioApi(NumChannels, NumFrames, SampleRate);
	}

	vraudio::MaterialName ConvertToResonanceMaterialName(ERaMaterialName UnrealMaterialName)
	{
		// These are rough estimates of what scalar gain coefficients may correspond to a given material,
		// though many of these materials have similar gain coefficients and drastically different frequency characteristics.
		switch (UnrealMaterialName)
		{
		case ERaMaterialName::TRANSPARENT:
			return vraudio::MaterialName::kTransparent;
		case ERaMaterialName::ACOUSTIC_CEILING_TILES:
			return vraudio::MaterialName::kAcousticCeilingTiles;
		case ERaMaterialName::BRICK_BARE:
			return vraudio::MaterialName::kBrickBare;
		case ERaMaterialName::BRICK_PAINTED:
			return vraudio::MaterialName::kBrickPainted;
		case ERaMaterialName::CONCRETE_BLOCK_COARSE:
			return vraudio::MaterialName::kConcreteBlockCoarse;
		case ERaMaterialName::CONCRETE_BLOCK_PAINTED:
			return vraudio::MaterialName::kConcreteBlockPainted;
		case ERaMaterialName::CURTAIN_HEAVY:
			return vraudio::MaterialName::kCurtainHeavy;
		case ERaMaterialName::FIBER_GLASS_INSULATION:
			return vraudio::MaterialName::kFiberGlassInsulation;
		case ERaMaterialName::GLASS_THICK:
			return vraudio::MaterialName::kGlassThick;
		case ERaMaterialName::GLASS_THIN:
			return vraudio::MaterialName::kGlassThin;
		case ERaMaterialName::GRASS:
			return vraudio::MaterialName::kGrass;
		case ERaMaterialName::LINOLEUM_ON_CONCRETE:
			return vraudio::MaterialName::kLinoleumOnConcrete;
		case ERaMaterialName::MARBLE:
			return vraudio::MaterialName::kMarble;
		case ERaMaterialName::METAL:
			return vraudio::MaterialName::kMarble;
		case ERaMaterialName::PARQUET_ONCONCRETE:
			return vraudio::MaterialName::kParquetOnConcrete;
		case ERaMaterialName::PLASTER_ROUGH:
			return vraudio::MaterialName::kPlasterRough;
		case ERaMaterialName::PLASTER_SMOOTH:
			return vraudio::MaterialName::kPlasterSmooth;
		case ERaMaterialName::PLYWOOD_PANEL:
			return vraudio::MaterialName::kPlywoodPanel;
		case ERaMaterialName::POLISHED_CONCRETE_OR_TILE:
			return vraudio::MaterialName::kPolishedConcreteOrTile;
		case ERaMaterialName::SHEETROCK:
			return vraudio::MaterialName::kSheetrock;
		case ERaMaterialName::WATER_OR_ICE_SURFACE:
			return vraudio::MaterialName::kWaterOrIceSurface;
		case ERaMaterialName::WOOD_CEILING:
			return vraudio::MaterialName::kWoodCeiling;
		case ERaMaterialName::WOOD_PANEL:
			return vraudio::MaterialName::kWoodPanel;
		case ERaMaterialName::UNIFORM:
			return vraudio::MaterialName::kUniform;
		default:
			UE_LOGF(LogResonanceAudio, Error, "Acoustic Material does not exist. Returning Transparent Material.")
			return vraudio::MaterialName::kTransparent;
		}
	}

/*       RESONANCE AUDIO                UNREAL
           Y                             Z
           |					         |    X
		   |						     |   /
		   |						     |  /
		   |						     | /
		   |_______________X			 |/_______________Y
		  /
		 /
		/
	   Z
*/
	FVector ConvertToResonanceAudioCoordinates(const FVector& UnrealVector)
	{
		FVector ResonanceAudioVector;
		ResonanceAudioVector.X = UnrealVector.Y;
		ResonanceAudioVector.Y = UnrealVector.Z;
		ResonanceAudioVector.Z = -UnrealVector.X;
		return ResonanceAudioVector * SCALE_FACTOR;
	}


	FVector ConvertToResonanceAudioCoordinates(const Audio::FChannelPositionInfo& ChannelPositionInfo)
	{
		FVector ResonanceAudioVector;
		ResonanceAudioVector.X = ChannelPositionInfo.Radius * FMath::Sin(ChannelPositionInfo.Azimuth) * FMath::Cos(ChannelPositionInfo.Elevation);
		ResonanceAudioVector.Y = ChannelPositionInfo.Radius * FMath::Sin(ChannelPositionInfo.Azimuth) * FMath::Sin(ChannelPositionInfo.Elevation);
		ResonanceAudioVector.Z = ChannelPositionInfo.Radius * FMath::Cos(ChannelPositionInfo.Azimuth);

		return ResonanceAudioVector * SCALE_FACTOR;
	}

	FQuat ConvertToResonanceAudioRotation(const FQuat& UnrealQuat)
	{
		FQuat ResonanceAudioQuat;
		ResonanceAudioQuat.X = -UnrealQuat.Y;
		ResonanceAudioQuat.Y = -UnrealQuat.Z;
		ResonanceAudioQuat.Z = UnrealQuat.X;
		ResonanceAudioQuat.W = UnrealQuat.W;
		return ResonanceAudioQuat;
	}

}  // namespace ResonanceAudio
