//
// Copyright Google Inc. 2017. All rights reserved.
//

#pragma once

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include "api/resonance_audio_api.h"
#include "platforms/common/room_properties.h"
#include "platforms/common/room_effects_utils.h"
THIRD_PARTY_INCLUDES_END


#include "UObject/ObjectMacros.h"
#include "ResonanceAudioConstants.h"
#include "ResonanceAudioEnums.h"
#include "ISoundfieldFormat.h"

namespace ResonanceAudio
{
	typedef vraudio::ResonanceAudioApi::SourceId RaSourceId;
	typedef vraudio::RoomProperties RaRoomProperties;
	typedef vraudio::ReflectionProperties RaReflectionProperties;
	typedef vraudio::ReverbProperties RaReverbProperties;

	// Lifecycle of a ResonanceAudioApi is managed by a threadsafe TSharedPtr
	typedef TSharedPtr<vraudio::ResonanceAudioApi, ESPMode::ThreadSafe> FResonanceAudioApiSharedPtr;

	// Attempts to load the dynamic library pertaining to the given platform, performing some basic error checking.
	// Returns handle to dynamic library or nullptr on error.
	void* LoadResonanceAudioDynamicLibrary();

	// Calls the CreateVrAudioApi method either from the given dynamic library or directly in the case of static linkage.
	vraudio::ResonanceAudioApi* CreateResonanceAudioApi(void* DynamicLibraryHandle, size_t NumChannels, size_t NumFrames, int SampleRate);

	// Invalid Source ID.
	const RaSourceId RA_INVALID_SOURCE_ID = vraudio::ResonanceAudioApi::kInvalidSourceId;

	// Converts between Unreal enum and Resonance Audio room surface coefficient.
	vraudio::MaterialName ConvertToResonanceMaterialName(ERaMaterialName UnrealMaterialName);

	// Converts between Unreal and Resonance Audio position coordinates.
	FVector ConvertToResonanceAudioCoordinates(const FVector& UnrealVector);
	FVector ConvertToResonanceAudioCoordinates(const Audio::FChannelPositionInfo& ChannelPositionInfo);

	// Converts between Unreal and Resonance Audio rotation quaternions.
	FQuat ConvertToResonanceAudioRotation(const FQuat& UnrealQuat);

} // namespace ResonanceAudio
