//
// Copyright Google Inc. 2017. All rights reserved.
//

#pragma once

#include "Math/Color.h"

namespace ResonanceAudio
{
	// Resonance Audio assets base color.
	const FColor ASSET_COLOR = FColor(0, 198, 246);

	// Number of surfaces in a shoe-box room.
	constexpr int NUM_SURFACES = 6;

	// Conversion factor between Resonance Audio and Unreal world distance units (1cm in Unreal = 0.01m in Resonance Audio).
	constexpr float SCALE_FACTOR = 0.01f;
}
