//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioDirectivityVisualizer.h"
#include "ResonanceAudioConstants.h"

#include "UObject/ConstructorHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioDirectivityVisualizer)

AResonanceAudioDirectivityVisualizer::AResonanceAudioDirectivityVisualizer()
	: Material(nullptr)
	, Settings(nullptr)
{
	// Make sure visualization is not displayed in the actual game.
	Super::SetActorHiddenInGame(true);

#if SUPPORTS_PROCEDURAL_MESH
	Mesh = CreateDefaultSubobject<UProceduralMeshComponent>(TEXT("VisualizationMesh"));
	RootComponent = Mesh;

	// Disable collision data.
	Mesh->ContainsPhysicsTriMeshData(false);
#endif // SUPPORTS_PROCEDURAL_MESH
}

void AResonanceAudioDirectivityVisualizer::DrawPattern()
{
	Vertices.Empty();
	Triangles.Empty();
	Normals.Empty();
	VertexColors.Empty();

	// Make the default directivity visualization radius equal 2m (set arbitrarily, based on how well the visualization renders in the viewport).
	const float FINAL_SCALE = 200.0f * Settings->Scale;
	const FVector NORMAL_VECTOR = FVector(0, 0, 1);

	for (int i = 0; i < CIRCLE_SECTIONS; ++i)
	{
		const float Angle1 = static_cast<float>(i * 2) * PI / CIRCLE_SECTIONS;
		const float Angle2 = static_cast<float>((i + 1) * 2) * PI / CIRCLE_SECTIONS;

		const float Gain1 = FINAL_SCALE * FMath::Pow(FMath::Abs((1.0f - Settings->Pattern) + Settings->Pattern * FMath::Cos(Angle1)), Settings->Sharpness);
		const float Gain2 = FINAL_SCALE * FMath::Pow(FMath::Abs((1.0f - Settings->Pattern) + Settings->Pattern * FMath::Cos(Angle2)), Settings->Sharpness);

		Vertices.Add(FVector(0.0f, 0.0f, 0.0f));
		Vertices.Add(FVector(Gain1 * FMath::Cos(Angle1), Gain1 * FMath::Sin(Angle1), 0.0f));
		Vertices.Add(FVector(Gain2 * FMath::Cos(Angle2), Gain2 * FMath::Sin(Angle2), 0.0f));

		for (int j = 0; j < 3; ++j)
		{
			Triangles.Add(3 * i + j);
			Normals.Add(NORMAL_VECTOR);
			VertexColors.Add(ResonanceAudio::ASSET_COLOR);
		}
	}

#if SUPPORTS_PROCEDURAL_MESH
	Mesh->CreateMeshSection(0, Vertices, Triangles, Normals, UV0, VertexColors, Tangents, false);
	Mesh->SetMaterial(0, Material);
#endif // SUPPORTS_PROCEDURAL_MESH
}


