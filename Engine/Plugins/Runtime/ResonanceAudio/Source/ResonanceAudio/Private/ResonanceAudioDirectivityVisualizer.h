//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Materials/Material.h"
#include "ProceduralMeshComponent.h"
#include "ResonanceAudioSpatializationSourceSettings.h"
#include "ResonanceAudioDirectivityVisualizer.generated.h"

#define UE_API RESONANCEAUDIO_API

class UResonanceAudioSpatializationSourceSettings;

UCLASS(MinimalAPI)
class AResonanceAudioDirectivityVisualizer : public AActor
{
	GENERATED_BODY()

public:
	// Sets default values for this actor's properties.
	UE_API AResonanceAudioDirectivityVisualizer();

	// Draws given directivity pattern as a 2D mesh based on Settings.
	UE_API void DrawPattern();

	void SetSettings(UResonanceAudioSpatializationSourceSettings* InSettings) { Settings = InSettings; };
	UResonanceAudioSpatializationSourceSettings* GetSettings() const { return Settings; };

private:
	// Controls the smoothness of mesh visualization.
	const int CIRCLE_SECTIONS = 128;

	// Arrays required by Procedural Mesh Component.
	TArray<FVector> Vertices;
	TArray<int32> Triangles;
	TArray<FVector> Normals;
	TArray<FColor> VertexColors;

	// Optional arrays.
	TArray<FVector2D> UV0;
	TArray<FProcMeshTangent> Tangents;

#if SUPPORTS_PROCEDURAL_MESH
	UProceduralMeshComponent* Mesh;
#endif

	UPROPERTY()
	TObjectPtr<UMaterial> Material;

	UPROPERTY()
	TObjectPtr<UResonanceAudioSpatializationSourceSettings> Settings;
};

#undef UE_API
