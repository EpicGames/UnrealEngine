# Default owners for everything in the repo.
* @anokta @haroonq

# Wwise plugin.
platforms/wwise/*     @ralphyon

