# [Resonance Audio](https://developers.google.com/resonance-audio) Source Code [![Travis CI](https://travis-ci.org/resonance-audio/resonance-audio.svg?branch=master)](https://travis-ci.org/resonance-audio/resonance-audio)

This is the official open source project for the Resonance Audio SDK. This
repository consists of the full source code of the Resonance Audio C++ library,
as well as the platform integrations into [Unity](https://unity3d.com/),
[FMOD](https://www.fmod.com/),
[Wwise](https://www.audiokinetic.com/products/wwise/) and DAW tools.

Resonance Audio started as a Google product and has since graduated to open
source. It is supported by members of our [steering
committee](#steering-committee) who are also project committers.

In this document there are some quick instructions for how to build the SDK from
source code.

For more detailed documentation about using the SDK, visit our [developer
docs](https://developers.google.com/resonance-audio/). If you are interested in
contributing to the project, please read the [Contributing to Resonance
Audio](#contributing-to-resonance-audio) section below.

## Build Instructions

Clone the repository:

    git clone https://github.com/resonance-audio/resonance-audio $YOUR_LOCAL_REPO

### Software Requirements

In addition to the system C++ software development platform tools / toolchains,
the following software is required to build and install the Resonance Audio
SDKs:

-   [CMake](https://cmake.org/download/)
-   [Git (with Git-Bash on Windows)](https://git-scm.com/downloads)
-   [Mercurial](https://www.mercurial-scm.org/downloads)

_Note: For Windows builds, [Visual Studio
2015](https://www.visualstudio.com/vs/older-downloads/) is recommended._

### Third Party Dependencies

All third party dependencies must be installed into the `third_party` subfolder
in the repository. To simplify the installation, bash scripts are located within
the `third_party` that automatically clone, build and install the required third
party source code.

_Note: On Windows, these scripts can be executed in the Git-Bash console (which
gets installed as part of Git for Windows)._

#### Core Dependencies ([pffft](https://bitbucket.org/jpommier/pffft), [eigen](https://bitbucket.org/eigen/eigen), [googletest](https://github.com/google/googletest), [SADIE Binaural Measurements](https://www.york.ac.uk/sadie-project/database_old.html))

To clone the dependencies into the repository, run:

    ./$YOUR_LOCAL_REPO/third_party/clone_core_deps.sh

_Note: These dependencies do *not* need to be built, since their source code is
directly pulled in from the build scripts._

#### [Unity](https://unity3d.com/) Platform Dependencies ([nativeaudioplugins](https://bitbucket.org/Unity-Technologies/nativeaudioplugins), [embree](https://github.com/embree/embree), [ogg](https://github.com/xiph/ogg), [vorbis](https://github.com/xiph/vorbis))

The Unity plugin integrates additional tools to estimate reverberation from game
geometry and to capture Ambisonic soundfields from a game scene. These features
require the Embree, libOgg and libVorbis libraries to be *prebuilt*.

To clone and build the additional Unity dependencies, run:

    ./$YOUR_LOCAL_REPO/third_party/clone_build_install_unity_deps.sh

#### [FMOD](https://www.fmod.com/) Platform Dependencies ([FMOD Low Level API](https://www.fmod.com/download#fmodstudio))

To add the additional FMOD dependencies, download and install the [FMOD Studio
API](https://www.fmod.com/download#fmodstudio) (which includes the FMOD Low
Level API).

_Note: On Linux, unzip the downloaded package within the `third_party` subfolder
and rename its folder to `fmod`._

#### [Wwise](https://www.audiokinetic.com/products/wwise/) Platform Dependencies ([WwiseIncludes](https://github.com/audiokinetic/WwiseIncludes))

To clone the additional Wwise dependencies, run:

    ./$YOUR_LOCAL_REPO/third_party/clone_wwise_deps.sh

The Wwise Authoring Plugin (Windows only) also requires the [Microsoft
Foundation Classes SDK](https://docs.microsoft.com/en-gb/cpp/mfc/mfc-and-atl).
To install the SDK on Windows:

1.  Open the `Control Panel`
2.  Select `Programs->Programs and Features`
3.  Right-click on `Microsoft Visual C++ Build Tools`, and select `Change`
4.  Install `MFC SDK`

#### DAW Tools Dependencies ([VST2 Audio Plug-Ins SDK](https://www.steinberg.net/vst3sdk))

To add the additional DAW Tools dependencies, download the [Steinberg's VST
3.X.X Audio Plug-Ins SDK](https://www.steinberg.net/vst3sdk) (which
includes the VST2 Audio Plug-Ins SDK) and extract the package into `third_party`
subfolder.

### Build Resonance Audio SDKs

This repository provides the `build.sh` script in the root folder that
configures the build targets, triggers the compilation and installs the
artifacts for the specified platform into the target installation folder.

The script provides the following flags:

-   t=|--target=
    -   `RESONANCE_AUDIO_API`: Builds the Resonance Audio API
    -   `RESONANCE_AUDIO_TESTS`: Runs the Resonance Audio unit tests
    -   `GEOMETRICAL_ACOUSTICS_TESTS`: Runs the geometrical acoustics specific
        unit tests.
    -   `UNITY_PLUGIN`: Builds the Resonance Audio plugin for Unity
    -   `WWISE_AUTHORING_PLUGIN`: Builds the Resonance Audio authoring plugin
        for Wwise
    -   `WWISE_SOUND_ENGINE_PLUGIN`: Builds the Resonance Audio sound engine
        plugin for Wwise
    -   `FMOD_PLUGIN`: Builds the Resonance Audio plugin for FMOD
    -   `VST_MONITOR_PLUGIN`: Builds the Resonance Audio VST Monitor Plugin
-   p=|--profile=
    -   `Debug`: Debug build
    -   `RelWithDebInfo`: Release build with debug information
    -   `Release`: Release build
-   --msvc_dynamic_runtime
    -   Enables dynamic linking against the run-time library on Windows (`/MD`,
        `/MDd`). By default, all Windows builds are statically linked against
        the run-time library (`/MT`, `/MTd`). Note that the third party
        dependencies must be compiled with the same options to avoid library
        conflicts.
-   --verbose_make
    -   Enables verbose make/build output.
-   --android_toolchain
    -   Enables the Android NDK toolchain to target Android builds (may require
        adjustments to `ANDROID_NDK`, `ANDROID_NATIVE_API_LEVEL` and
        `ANDROID_ABI` script variables). For more information, see project
        documentation at https://github.com/taka-no-me/android-cmake.
-   --ios_os_toolchain
    -   Enables the iOS OS toolchain. For more information, see project
        documentation at https://github.com/leetal/ios-cmake.
-   --ios_simulator_toolchain
    -   Enables the iOS Simulator toolchain. For more information, see project
        documentation at https://github.com/leetal/ios-cmake

##### E.g.

To build and run the Resonance Audio unit tests:

    ./$YOUR_LOCAL_REPO/build.sh -t=RESONANCE_AUDIO_TESTS

## Citations

If you find Resonance Audio useful and would like to cite it in your publication, please use:

Gorzel, M., Allen, A., Kelly, I., Kammerl, J., Gugnormusler, A., Yeh, H., and Boland, F., *"Efficient Encoding and Decoding of Binaural Sound with Resonance Audio"*, In proc. of the AES International Conference on Immersive and Interactive Audio, March 2019

The full paper is available (open access) at: http://www.aes.org/e-lib/browse.cfm?elib=20446 ([BibTeX](http://www.aes.org/e-lib/browse.cfm?elib=20446&fmt=bibtex))

## Contributing to Resonance Audio

If you would like to contribute changes to the Resonance Audio project, please
make a pull request for one of our project committers to review.

### Steering Committee

The Resonance Audio project is overseen by a steering committee established to
help guide the technical direction of the project in collaboration with the
entire developer community.

The intention of the steering committee is to cultivate collaboration across the
developer community for improving the project and ensuring Resonance Audio
continues to work well for everyone.

The committee will lead the Resonance Audio project in major decisions by
consensus and ensure that Resonance Audio can meet its goals as a truly open
source project.

The steering committee consists of the following members (company name ordered):

-   Martin Dufour, Audiokinetic
-   Aaron McLeran, Epic Games
-   Mathew Block, Firelight Technologies
-   Alper Gungormusler, Google
-   Eric Mauskopf, Google
-   Haroon Qureshi, Google
-   Ian Kelly, Google
-   Julius Kammerl, Google
-   Marcin Gorzel, Google
-   Damien Kelly, Google (YouTube)
-   Jean-Marc Jot, Magic Leap
-   Michael Berg, Unity Technologies

Affiliations are listed for identification purposes only; steering committee
members do not represent their employers or academic institutions.
