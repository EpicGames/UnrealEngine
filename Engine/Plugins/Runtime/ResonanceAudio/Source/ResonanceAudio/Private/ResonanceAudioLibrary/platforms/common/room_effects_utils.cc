/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "platforms/common/room_effects_utils.h"

#include <algorithm>
#include <cmath>
#include <numeric>

namespace vraudio {

namespace {

// Air absorption coefficients at 20 degrees Celsius and 50% relative humidity,
// according to:
// http://www.music.mcgill.ca/~gary/courses/papers/Moorer-Reverb-CMJ-1979.pdf
// These coefficients have been extrapolated to other frequencies by fitting
// an exponential curve:
//
// m = a + be^(-cx) where:
//
// a = -0.00259118
// b = 0.003173474
// c = -0.0002491554
//
const float kAirAbsorptionCoefficients[]{0.0006f, 0.0006f, 0.0007f,
                                         0.0008f, 0.0010f, 0.0015f,
                                         0.0026f, 0.0060f, 0.0207f};

// Room materials with the corresponding absorption coefficients.
const RoomMaterial kRoomMaterials[static_cast<size_t>(
    MaterialName::kNumMaterialNames)] = {
    {MaterialName::kTransparent,
     // 31.25 62.5  125   250   500   1000  2000  4000  8000 Hz.
     {1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f, 1.0f}},
    {MaterialName::kAcousticCeilingTiles,
     {0.672f, 0.675f, 0.700f, 0.660f, 0.720f, 0.920f, 0.880f, 0.750f, 1.000f}},
    {MaterialName::kBrickBare,
     {0.030f, 0.030f, 0.030f, 0.030f, 0.030f, 0.040f, 0.050f, 0.070f, 0.140f}},

    {MaterialName::kBrickPainted,
     {0.006f, 0.007f, 0.010f, 0.010f, 0.020f, 0.020f, 0.020f, 0.030f, 0.060f}},

    {MaterialName::kConcreteBlockCoarse,
     {0.360f, 0.360f, 0.360f, 0.440f, 0.310f, 0.290f, 0.390f, 0.250f, 0.500f}},

    {MaterialName::kConcreteBlockPainted,
     {0.092f, 0.090f, 0.100f, 0.050f, 0.060f, 0.070f, 0.090f, 0.080f, 0.160f}},

    {MaterialName::kCurtainHeavy,
     {0.073f, 0.106f, 0.140f, 0.350f, 0.550f, 0.720f, 0.700f, 0.650f, 1.000f}},

    {MaterialName::kFiberGlassInsulation,
     {0.193f, 0.220f, 0.220f, 0.820f, 0.990f, 0.990f, 0.990f, 0.990f, 1.000f}},

    {MaterialName::kGlassThin,
     {0.180f, 0.169f, 0.180f, 0.060f, 0.040f, 0.030f, 0.020f, 0.020f, 0.040f}},

    {MaterialName::kGlassThick,
     {0.350f, 0.350f, 0.350f, 0.250f, 0.180f, 0.120f, 0.070f, 0.040f, 0.080f}},

    {MaterialName::kGrass,
     {0.05f, 0.05f, 0.15f, 0.25f, 0.40f, 0.55f, 0.60f, 0.60f, 0.60f}},

    {MaterialName::kLinoleumOnConcrete,
     {0.020f, 0.020f, 0.020f, 0.030f, 0.030f, 0.030f, 0.030f, 0.020f, 0.040f}},

    {MaterialName::kMarble,
     {0.010f, 0.010f, 0.010f, 0.010f, 0.010f, 0.010f, 0.020f, 0.020f, 0.040f}},

    {MaterialName::kMetal,
     {0.030f, 0.035f, 0.04f, 0.04f, 0.05f, 0.05f, 0.05f, 0.07f, 0.09f}},

    {MaterialName::kParquetOnConcrete,
     {0.028f, 0.030f, 0.040f, 0.040f, 0.070f, 0.060f, 0.060f, 0.070f, 0.140f}},

    {MaterialName::kPlasterRough,
     {0.017f, 0.018f, 0.020f, 0.030f, 0.040f, 0.050f, 0.040f, 0.030f, 0.060f}},

    {MaterialName::kPlasterSmooth,
     {0.011f, 0.012f, 0.013f, 0.015f, 0.020f, 0.030f, 0.040f, 0.050f, 0.100f}},

    {MaterialName::kPlywoodPanel,
     {0.40f, 0.34f, 0.28f, 0.22f, 0.17f, 0.09f, 0.10f, 0.11f, 0.22f}},

    {MaterialName::kPolishedConcreteOrTile,
     {0.008f, 0.008f, 0.010f, 0.010f, 0.015f, 0.020f, 0.020f, 0.020f, 0.040f}},

    {MaterialName::kSheetrock,
     {0.290f, 0.279f, 0.290f, 0.100f, 0.050f, 0.040f, 0.070f, 0.090f, 0.180f}},

    {MaterialName::kWaterOrIceSurface,
     {0.006f, 0.006f, 0.008f, 0.008f, 0.013f, 0.015f, 0.020f, 0.025f, 0.050f}},

    {MaterialName::kWoodCeiling,
     {0.150f, 0.147f, 0.150f, 0.110f, 0.100f, 0.070f, 0.060f, 0.070f, 0.140f}},

    {MaterialName::kWoodPanel,
     {0.280f, 0.280f, 0.280f, 0.220f, 0.170f, 0.090f, 0.100f, 0.110f, 0.220f}},

    {MaterialName::kUniform,
     {0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f, 0.5f}}};

// Frequency threshold for low pass filtering. This is the -3dB frequency.
const float kCutoffFrequency = 800.0f;

// Number of averaging bands for computing the average absorption coefficient.
const int kNumAveragingBands = 3;

// Average absorption coefficients are computed based on 3 octave bands (500Hz,
// 1kHz, 2kHz) from the user specified materials. The 500Hz band has index 4.
const int kStartingBand = 4;

// Default scaling factor of the reverberation tail. This value is
// multiplied by the user-defined factor in order to allow for the change of
// gain in the reverb tail from the UI. Updates to the gain value are made to
// match the previous reverb implementation's loudness.
const float kDefaultReverbGain = 0.045f;

// Constant used in Eyring's equation to compute RT60.
const float kEyringConstant = 0.161f;

// Computes RT60 time in the given frequency band according to Eyring.
inline float ComputeRt60Eyring(float total_area, float mean_absorption,
                               size_t band, float volume) {
  return kEyringConstant * volume /
         (-total_area * std::log(1.0f - mean_absorption) +
          4.0f * kAirAbsorptionCoefficients[band] * volume);
}

inline std::vector<float> ComputeShoeBoxSurfaceAreas(
    const RoomProperties& room_properties) {
  const float room_width = room_properties.dimensions[0];
  const float room_height = room_properties.dimensions[1];
  const float room_depth = room_properties.dimensions[2];
  const float left_right_area = room_height * room_depth;
  const float top_bottom_area = room_width * room_depth;
  const float front_back_area = room_width * room_height;
  return std::vector<float>{left_right_area, left_right_area, top_bottom_area,
                            top_bottom_area, front_back_area, front_back_area};
}

// Generates average reflection coefficients for each room model surface.
//
// @param room_properties Struct containing properties of the shoe-box room
//     model.
// @param coefficients Reflection coefficients for each surface.
void GenerateReflectionCoefficients(const RoomProperties& room_properties,
                                    float* coefficients) {
  DCHECK(coefficients);
  // Loop through all the surfaces and compute the average absorption
  // coefficient for 3 bands (500Hz, 1kHz and 2kHz).
  for (size_t surface = 0; surface < kNumRoomSurfaces; ++surface) {
    const size_t material_index =
        static_cast<size_t>(room_properties.material_names[surface]);
    // Absorption coefficients in all bands for the current surface.
    const auto& absorption_coefficients =
        kRoomMaterials[material_index].absorption_coefficients;
    // Compute average absorption coefficients for each surface.
    float average_absorption_coefficient =
        std::accumulate(std::begin(absorption_coefficients) + kStartingBand,
                        std::begin(absorption_coefficients) + kStartingBand +
                            kNumAveragingBands,
                        0.0f) /
        static_cast<float>(kNumAveragingBands);
    // Compute a reflection coefficient for each surface.
    coefficients[surface] =
        std::min(1.0f, std::sqrt(1.0f - average_absorption_coefficient));
  }
}

// Uses the Eyring's equation to estimate RT60 values (reverb time in seconds)
// in |kNumReverbOctaveBands| octave bands, including the correction for air
// absorption. The equation is applied as defined in:
// https://arauacustica.com/files/publicaciones_relacionados/pdf_esp_26.pdf
//

void GenerateRt60Values(const RoomProperties& room_properties,
                        float* rt60_values) {
  DCHECK(rt60_values);
  // Compute the shoe-box room volume.
  const float room_volume = room_properties.dimensions[0] *
                            room_properties.dimensions[1] *
                            room_properties.dimensions[2];
  if (room_volume < std::numeric_limits<float>::epsilon()) {
    // RT60 values will be all zeros, if the room volume is zero.
    return;
  }

  // Compute surface areas of the shoe-box room.
  const std::vector<float> all_surface_areas =
      ComputeShoeBoxSurfaceAreas(room_properties);
  const float total_area =
      std::accumulate(all_surface_areas.begin(), all_surface_areas.end(), 0.0f);
  DCHECK_GT(total_area, 0.0f);
  // Loop through each band and compute the RT60 values.
  for (size_t band = 0; band < kNumReverbOctaveBands; ++band) {
    // Initialize the effective absorbing area.
    float absorbing_area = 0.0f;
    for (size_t surface = 0; surface < kNumRoomSurfaces; ++surface) {
      const size_t material_index =
          static_cast<size_t>(room_properties.material_names[surface]);
      // Compute the effective absorbing area based on the absorption
      // coefficients for the current band and all the surfaces.
      absorbing_area +=
          kRoomMaterials[material_index].absorption_coefficients[band] *
          all_surface_areas[surface];
    }
    DCHECK_GT(absorbing_area, 0.0f);
    const float mean_absorption = std::min(absorbing_area / total_area, 1.0f);

    // Compute RT60 time in this band according to Eyring.
    rt60_values[band] =
        ComputeRt60Eyring(total_area, mean_absorption, band, room_volume);
  }
}

// Modifies the RT60 values by the given |brightness_modifier| and
// |time_scaler|.
void ModifyRT60Values(float brightness_modifier, float time_scaler,
                      float* rt60_values) {
  DCHECK(rt60_values);
  for (size_t band = 0; band < kNumReverbOctaveBands; ++band) {
    // Linearly scale calculated reverb times according to the user specified
    // |brightness_modifier| and |time_scaler| values.
    rt60_values[band] *=
        (1.0f + brightness_modifier * static_cast<float>(band + 1) /
                    static_cast<float>(kNumReverbOctaveBands)) *
        time_scaler;
  }
}

}  // namespace

ReflectionProperties ComputeReflectionProperties(
    const RoomProperties& room_properties) {
  ReflectionProperties reflection_properties;
  std::copy(std::begin(room_properties.position),
            std::end(room_properties.position),
            std::begin(reflection_properties.room_position));
  std::copy(std::begin(room_properties.rotation),
            std::end(room_properties.rotation),
            std::begin(reflection_properties.room_rotation));
  std::copy(std::begin(room_properties.dimensions),
            std::end(room_properties.dimensions),
            std::begin(reflection_properties.room_dimensions));
  reflection_properties.cutoff_frequency = kCutoffFrequency;
  GenerateReflectionCoefficients(room_properties,
                                 reflection_properties.coefficients);
  reflection_properties.gain = room_properties.reflection_scalar;
  return reflection_properties;
}

ReverbProperties ComputeReverbProperties(
    const RoomProperties& room_properties) {
  ReverbProperties reverb_properties;
  GenerateRt60Values(room_properties, reverb_properties.rt60_values);
  ModifyRT60Values(room_properties.reverb_brightness,
                   room_properties.reverb_time, reverb_properties.rt60_values);
  reverb_properties.gain = kDefaultReverbGain * room_properties.reverb_gain;
  return reverb_properties;
}

ReverbProperties ComputeReverbPropertiesFromRT60s(const float* rt60_values,
                                                  float brightness_modifier,
                                                  float time_scalar,
                                                  float gain_multiplier) {
  DCHECK(rt60_values);

  ReverbProperties reverb_properties;
  std::copy_n(rt60_values, kNumReverbOctaveBands,
              reverb_properties.rt60_values);
  ModifyRT60Values(brightness_modifier, time_scalar,
                   reverb_properties.rt60_values);
  reverb_properties.gain = kDefaultReverbGain * gain_multiplier;
  return reverb_properties;
}

float ComputeRoomEffectsGain(const WorldPosition& source_position,
                             const WorldPosition& room_position,
                             const WorldRotation& room_rotation,
                             const WorldPosition& room_dimensions) {
  const float room_volume =
      room_dimensions[0] * room_dimensions[1] * room_dimensions[2];
  if (room_volume < std::numeric_limits<float>::epsilon()) {
    // No room effects should be present when the room volume is zero.
    return 0.0f;
  }

  // Compute the relative source position with respect to the room.
  WorldPosition relative_source_position;
  GetRelativeDirection(room_position, room_rotation, source_position,
                       &relative_source_position);
  WorldPosition closest_position_in_room;
  GetClosestPositionInAabb(relative_source_position, room_dimensions,
                           &closest_position_in_room);
  // Shift the attenuation curve by 1.0f to avoid zero division.
  const float distance_to_room =
      1.0f + (relative_source_position - closest_position_in_room).norm();
  return 1.0f / (distance_to_room * distance_to_room);
}

RoomMaterial GetRoomMaterial(size_t material_index) {
  DCHECK_LT(material_index,
            static_cast<size_t>(MaterialName::kNumMaterialNames));
  return kRoomMaterials[material_index];
}

}  // namespace vraudio
