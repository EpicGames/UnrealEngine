/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_EFFECTS_UTILS_H_
#define RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_EFFECTS_UTILS_H_

#include <vector>

#include "api/resonance_audio_api.h"
#include "base/constants_and_types.h"
#include "base/misc_math.h"
#include "platforms/common/room_properties.h"

namespace vraudio {

// Room material properties.
struct RoomMaterial {
  // Material surface integer identifier.
  MaterialName name;

  // An array of absorption coefficients defined in octave bands.
  float absorption_coefficients[kNumReverbOctaveBands];
};

// Generates |ReflectionProperties| based on given |room_properties|.
//
// @param room_properties Room properties.
ReflectionProperties ComputeReflectionProperties(
    const RoomProperties& room_properties);

// Generates |ReverbProperties| based on given |room_properties|.
//
// @param room_properties Room properties.
ReverbProperties ComputeReverbProperties(const RoomProperties& room_properties);

// Generates |ReverbProperties| by directly setting the RT60 values, subject
// to modifications by |brightness_modifier| and |time_scaler|.
//
// @param rt60_values RT60 values.
// @param brightness_modifier Modifier adjusting the brightness of reverb.
// @param time_scalar Modifier scaling the reverb time.
// @param gain_multiplier Modifier scaling the reverb gain.
ReverbProperties ComputeReverbPropertiesFromRT60s(const float* rt60_values,
                                                  float brightness_modifier,
                                                  float time_scalar,
                                                  float gain_multiplier);

// Calculates the gain value for |source_position| with respect to the given
// room properties. The sound level for the room effects will remain the same
// inside the room, otherwise, it will decrease with a linear ramp from the
// closest point on the room.
//
// @param source_position World position of the source.
// @param room_position Center position of the room.
// @param room_rotation Orientation of the room.
// @param room_dimensions Dimensions of the room..
// @return Attenuation (gain) value in range [0.0f, 1.0f].
float ComputeRoomEffectsGain(const WorldPosition& source_position,
                             const WorldPosition& room_position,
                             const WorldRotation& room_rotation,
                             const WorldPosition& room_dimensions);

// Gets the room material properties from a material index.
//
// @param material_index Index of the material.
RoomMaterial GetRoomMaterial(size_t material_index);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_EFFECTS_UTILS_H_
