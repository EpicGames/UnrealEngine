/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Copyright 2017 Google Inc. All rights reserved.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//      http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS-IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.
#ifndef RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_PROPERTIES_H_
#define RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_PROPERTIES_H_

namespace vraudio {

// Room surface material names, used to set room properties.
// Note that this enum is C-compatible by design to be used across external
// C/C++ and C# implementations.
enum MaterialName {
  kTransparent = 0,
  kAcousticCeilingTiles,
  kBrickBare,
  kBrickPainted,
  kConcreteBlockCoarse,
  kConcreteBlockPainted,
  kCurtainHeavy,
  kFiberGlassInsulation,
  kGlassThin,
  kGlassThick,
  kGrass,
  kLinoleumOnConcrete,
  kMarble,
  kMetal,
  kParquetOnConcrete,
  kPlasterRough,
  kPlasterSmooth,
  kPlywoodPanel,
  kPolishedConcreteOrTile,
  kSheetrock,
  kWaterOrIceSurface,
  kWoodCeiling,
  kWoodPanel,
  kUniform,
  kNumMaterialNames
};

// Acoustic room properties. This struct can be used to describe an acoustic
// environment with a given geometry and surface properties.
// Note that this struct is C-compatible by design to be used across external
// C/C++ and C# implementations.
struct RoomProperties {
  // Constructs |RoomProperties| with the default values.
  RoomProperties()
      : position{0.0f, 0.0f, 0.0f},
        rotation{0.0f, 0.0f, 0.0f, 1.0f},
        dimensions{0.0f, 0.0f, 0.0f},
        material_names{MaterialName::kTransparent, MaterialName::kTransparent,
                       MaterialName::kTransparent, MaterialName::kTransparent,
                       MaterialName::kTransparent, MaterialName::kTransparent},
        reflection_scalar(1.0f),
        reverb_gain(1.0f),
        reverb_time(1.0f),
        reverb_brightness(0.0f) {}

  // Center position of the room in world space, uses right-handed coordinate
  // system.
  float position[3];

  // Rotation (quaternion) of the room in world space, uses right-handed
  // coordinate system.
  float rotation[4];

  // Size of the shoebox room in world space, uses right-handed coordinate
  // system.
  float dimensions[3];

  // Material name of each surface of the shoebox room in this order:
  // [0] (-)ive x-axis wall (left)
  // [1] (+)ive x-axis wall (right)
  // [2] (-)ive y-axis wall (bottom)
  // [3] (+)ive y-axis wall (top)
  // [4] (-)ive z-axis wall (front)
  // [5] (+)ive z-axis wall (back)
  MaterialName material_names[6];

  // User defined uniform scaling factor for all reflection coefficients.
  float reflection_scalar;

  // User defined reverb tail gain multiplier.
  float reverb_gain;

  // Adjusts the reverberation time across all frequency bands. RT60 values
  // are multiplied by this factor. Has no effect when set to 1.0f.
  float reverb_time;

  // Controls the slope of a line from the lowest to the highest RT60 values
  // (increases high frequency RT60s when positive, decreases when negative).
  // Has no effect when set to 0.0f.
  float reverb_brightness;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_PLATFORM_COMMON_ROOM_PROPERTIES_H_
