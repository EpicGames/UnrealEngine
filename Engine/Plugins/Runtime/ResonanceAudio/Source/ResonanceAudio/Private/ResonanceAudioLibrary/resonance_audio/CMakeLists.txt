# Copyright 2018 Google Inc. All Rights Reserved.
#
# Licensed under the Apache License, Version 2.0 (the "License");
# you may not use this file except in compliance with the License.
# You may obtain a copy of the License at
#
#     http://www.apache.org/licenses/LICENSE-2.0
#
# Unless required by applicable law or agreed to in writing, software
# distributed under the License is distributed on an "AS-IS" BASIS,
# WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
# See the License for the specific language governing permissions and
# limitations under the License.
#

# Build PFFFT.
set(PFFFT_DIR "${PROJECT_SOURCE_DIR}/third_party/pffft/" CACHE PATH "Path to pffft library")
set(PFFFT_INCLUDE_DIR ${PFFFT_DIR})
set(PFFFT_SOURCE
        ${PFFFT_DIR}/fftpack.c
        ${PFFFT_DIR}/fftpack.h
        ${PFFFT_DIR}/pffft.c
        ${PFFFT_DIR}/pffft.h
        )
add_library(PffftObj OBJECT ${PFFFT_SOURCE})
target_include_directories(PffftObj PRIVATE ${PFFFT_INCLUDE_DIR}/)

if (UNIX)
    set_target_properties(PffftObj PROPERTIES COMPILE_FLAGS -fPIC)
endif (UNIX)

# Build SADIE HRTF database.
set(SADIE_HRTFS_DIR "${PROJECT_SOURCE_DIR}/third_party/SADIE_hrtf_database/generated/" CACHE PATH "Path to SADIE_hrtf_database library")
set(SADIE_HRTFS_INCLUDE_DIR ${SADIE_HRTFS_DIR})
set(SADIE_HRTFS_SOURCE
        ${SADIE_HRTFS_DIR}/hrtf_assets.cc
        ${SADIE_HRTFS_DIR}/hrtf_assets.h
        )
add_library(SadieHrtfsObj OBJECT ${SADIE_HRTFS_SOURCE})

# Build Resonance Audio.
set(RA_SOURCES
        ${PROJECT_SOURCE_DIR}/platforms/common/utils.cc
        ${PROJECT_SOURCE_DIR}/platforms/common/utils.h
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_binaural_decoder.h
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_binaural_decoder.cc
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_codec.h
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_codec_impl.h
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_lookup_table.cc
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_lookup_table.h
        ${RA_SOURCE_DIR}/ambisonics/ambisonic_spread_coefficients.h
        ${RA_SOURCE_DIR}/ambisonics/associated_legendre_polynomials_generator.cc
        ${RA_SOURCE_DIR}/ambisonics/associated_legendre_polynomials_generator.h
        ${RA_SOURCE_DIR}/ambisonics/foa_rotator.cc
        ${RA_SOURCE_DIR}/ambisonics/foa_rotator.h
        ${RA_SOURCE_DIR}/ambisonics/hoa_rotator.cc
        ${RA_SOURCE_DIR}/ambisonics/hoa_rotator.h
        ${RA_SOURCE_DIR}/ambisonics/stereo_from_soundfield_converter.cc
        ${RA_SOURCE_DIR}/ambisonics/stereo_from_soundfield_converter.h
        ${RA_SOURCE_DIR}/ambisonics/utils.h
        ${RA_SOURCE_DIR}/api/binaural_surround_renderer.cc
        ${RA_SOURCE_DIR}/api/binaural_surround_renderer.h
        ${RA_SOURCE_DIR}/api/resonance_audio_api.cc
        ${RA_SOURCE_DIR}/api/resonance_audio_api.h
        ${RA_SOURCE_DIR}/base/aligned_allocator.h
        ${RA_SOURCE_DIR}/base/audio_buffer.cc
        ${RA_SOURCE_DIR}/base/audio_buffer.h
        ${RA_SOURCE_DIR}/base/channel_view.cc
        ${RA_SOURCE_DIR}/base/channel_view.h
        ${RA_SOURCE_DIR}/base/constants_and_types.h
        ${RA_SOURCE_DIR}/base/integral_types.h
        ${RA_SOURCE_DIR}/base/logging.h
        ${RA_SOURCE_DIR}/base/misc_math.cc
        ${RA_SOURCE_DIR}/base/misc_math.h
        ${RA_SOURCE_DIR}/base/object_transform.h
        ${RA_SOURCE_DIR}/base/simd_macros.h
        ${RA_SOURCE_DIR}/base/simd_utils.cc
        ${RA_SOURCE_DIR}/base/simd_utils.h
        ${RA_SOURCE_DIR}/base/source_parameters.h
        ${RA_SOURCE_DIR}/base/spherical_angle.cc
        ${RA_SOURCE_DIR}/base/spherical_angle.h
        ${RA_SOURCE_DIR}/base/unique_ptr_wrapper.h
        ${RA_SOURCE_DIR}/config/global_config.h
        ${RA_SOURCE_DIR}/config/source_config.cc
        ${RA_SOURCE_DIR}/config/source_config.h
        ${RA_SOURCE_DIR}/dsp/biquad_filter.cc
        ${RA_SOURCE_DIR}/dsp/biquad_filter.h
        ${RA_SOURCE_DIR}/dsp/channel_converter.cc
        ${RA_SOURCE_DIR}/dsp/channel_converter.h
        ${RA_SOURCE_DIR}/dsp/circular_buffer.cc
        ${RA_SOURCE_DIR}/dsp/circular_buffer.h
        ${RA_SOURCE_DIR}/dsp/delay_filter.cc
        ${RA_SOURCE_DIR}/dsp/delay_filter.h
        ${RA_SOURCE_DIR}/dsp/distance_attenuation.cc
        ${RA_SOURCE_DIR}/dsp/distance_attenuation.h
        ${RA_SOURCE_DIR}/dsp/fft_manager.cc
        ${RA_SOURCE_DIR}/dsp/fft_manager.h
        ${RA_SOURCE_DIR}/dsp/filter_coefficient_generators.cc
        ${RA_SOURCE_DIR}/dsp/filter_coefficient_generators.h
        ${RA_SOURCE_DIR}/dsp/fir_filter.cc
        ${RA_SOURCE_DIR}/dsp/fir_filter.h
        ${RA_SOURCE_DIR}/dsp/gain.cc
        ${RA_SOURCE_DIR}/dsp/gain.h
        ${RA_SOURCE_DIR}/dsp/gain_mixer.cc
        ${RA_SOURCE_DIR}/dsp/gain_mixer.h
        ${RA_SOURCE_DIR}/dsp/gain_processor.cc
        ${RA_SOURCE_DIR}/dsp/gain_processor.h
        ${RA_SOURCE_DIR}/dsp/mixer.cc
        ${RA_SOURCE_DIR}/dsp/mixer.h
        ${RA_SOURCE_DIR}/dsp/mono_pole_filter.cc
        ${RA_SOURCE_DIR}/dsp/mono_pole_filter.h
        ${RA_SOURCE_DIR}/dsp/multi_channel_iir.cc
        ${RA_SOURCE_DIR}/dsp/multi_channel_iir.h
        ${RA_SOURCE_DIR}/dsp/near_field_processor.cc
        ${RA_SOURCE_DIR}/dsp/near_field_processor.h
        ${RA_SOURCE_DIR}/dsp/occlusion_calculator.cc
        ${RA_SOURCE_DIR}/dsp/occlusion_calculator.h
        ${RA_SOURCE_DIR}/dsp/partitioned_fft_filter.cc
        ${RA_SOURCE_DIR}/dsp/partitioned_fft_filter.h
        ${RA_SOURCE_DIR}/dsp/reflection.h
        ${RA_SOURCE_DIR}/dsp/reflections_processor.cc
        ${RA_SOURCE_DIR}/dsp/reflections_processor.h
        ${RA_SOURCE_DIR}/dsp/resampler.cc
        ${RA_SOURCE_DIR}/dsp/resampler.h
        ${RA_SOURCE_DIR}/dsp/reverb_onset_compensator.cc
        ${RA_SOURCE_DIR}/dsp/reverb_onset_compensator.h
        ${RA_SOURCE_DIR}/dsp/reverb_onset_update_processor.cc
        ${RA_SOURCE_DIR}/dsp/reverb_onset_update_processor.h
        ${RA_SOURCE_DIR}/dsp/sh_hrir_creator.cc
        ${RA_SOURCE_DIR}/dsp/sh_hrir_creator.h
        ${RA_SOURCE_DIR}/dsp/shoe_box_room.cc
        ${RA_SOURCE_DIR}/dsp/shoe_box_room.h
        ${RA_SOURCE_DIR}/dsp/spectral_reverb.cc
        ${RA_SOURCE_DIR}/dsp/spectral_reverb.h
        ${RA_SOURCE_DIR}/dsp/spectral_reverb_constants_and_tables.h
        ${RA_SOURCE_DIR}/dsp/stereo_panner.cc
        ${RA_SOURCE_DIR}/dsp/stereo_panner.h
        ${RA_SOURCE_DIR}/dsp/utils.cc
        ${RA_SOURCE_DIR}/dsp/utils.h
        ${RA_SOURCE_DIR}/graph/ambisonic_binaural_decoder_node.cc
        ${RA_SOURCE_DIR}/graph/ambisonic_binaural_decoder_node.h
        ${RA_SOURCE_DIR}/graph/ambisonic_mixing_encoder_node.cc
        ${RA_SOURCE_DIR}/graph/ambisonic_mixing_encoder_node.h
        ${RA_SOURCE_DIR}/graph/binaural_surround_renderer_impl.cc
        ${RA_SOURCE_DIR}/graph/binaural_surround_renderer_impl.h
        ${RA_SOURCE_DIR}/graph/buffered_source_node.cc
        ${RA_SOURCE_DIR}/graph/buffered_source_node.h
        ${RA_SOURCE_DIR}/graph/foa_rotator_node.cc
        ${RA_SOURCE_DIR}/graph/foa_rotator_node.h
        ${RA_SOURCE_DIR}/graph/gain_mixer_node.cc
        ${RA_SOURCE_DIR}/graph/gain_mixer_node.h
        ${RA_SOURCE_DIR}/graph/gain_node.cc
        ${RA_SOURCE_DIR}/graph/gain_node.h
        ${RA_SOURCE_DIR}/graph/graph_manager.cc
        ${RA_SOURCE_DIR}/graph/graph_manager.h
        ${RA_SOURCE_DIR}/graph/graph_manager_config.h
        ${RA_SOURCE_DIR}/graph/hoa_rotator_node.cc
        ${RA_SOURCE_DIR}/graph/hoa_rotator_node.h
        ${RA_SOURCE_DIR}/graph/mixer_node.cc
        ${RA_SOURCE_DIR}/graph/mixer_node.h
        ${RA_SOURCE_DIR}/graph/mono_from_soundfield_node.cc
        ${RA_SOURCE_DIR}/graph/mono_from_soundfield_node.h
        ${RA_SOURCE_DIR}/graph/near_field_effect_node.cc
        ${RA_SOURCE_DIR}/graph/near_field_effect_node.h
        ${RA_SOURCE_DIR}/graph/occlusion_node.cc
        ${RA_SOURCE_DIR}/graph/occlusion_node.h
        ${RA_SOURCE_DIR}/graph/reflections_node.cc
        ${RA_SOURCE_DIR}/graph/reflections_node.h
        ${RA_SOURCE_DIR}/graph/resonance_audio_api_impl.cc
        ${RA_SOURCE_DIR}/graph/resonance_audio_api_impl.h
        ${RA_SOURCE_DIR}/graph/reverb_node.cc
        ${RA_SOURCE_DIR}/graph/reverb_node.h
        ${RA_SOURCE_DIR}/graph/source_graph_config.h
        ${RA_SOURCE_DIR}/graph/source_parameters_manager.cc
        ${RA_SOURCE_DIR}/graph/source_parameters_manager.h
        ${RA_SOURCE_DIR}/graph/stereo_mixing_panner_node.cc
        ${RA_SOURCE_DIR}/graph/stereo_mixing_panner_node.h
        ${RA_SOURCE_DIR}/graph/system_settings.h
        ${RA_SOURCE_DIR}/node/node.h
        ${RA_SOURCE_DIR}/node/processing_node.cc
        ${RA_SOURCE_DIR}/node/processing_node.h
        ${RA_SOURCE_DIR}/node/publisher_node.h
        ${RA_SOURCE_DIR}/node/sink_node.cc
        ${RA_SOURCE_DIR}/node/sink_node.h
        ${RA_SOURCE_DIR}/node/source_node.cc
        ${RA_SOURCE_DIR}/node/source_node.h
        ${RA_SOURCE_DIR}/node/subscriber_node.h
        ${PROJECT_SOURCE_DIR}/platforms/common/room_effects_utils.cc
        ${PROJECT_SOURCE_DIR}/platforms/common/room_effects_utils.h
        ${RA_SOURCE_DIR}/utils/buffer_crossfader.cc
        ${RA_SOURCE_DIR}/utils/buffer_crossfader.h
        ${RA_SOURCE_DIR}/utils/buffer_partitioner.cc
        ${RA_SOURCE_DIR}/utils/buffer_partitioner.h
        ${RA_SOURCE_DIR}/utils/buffer_unpartitioner.cc
        ${RA_SOURCE_DIR}/utils/buffer_unpartitioner.h
        ${RA_SOURCE_DIR}/utils/lockless_task_queue.cc
        ${RA_SOURCE_DIR}/utils/lockless_task_queue.h
        ${RA_SOURCE_DIR}/utils/planar_interleaved_conversion.cc
        ${RA_SOURCE_DIR}/utils/planar_interleaved_conversion.h
        ${RA_SOURCE_DIR}/utils/pseudoinverse.h
        ${RA_SOURCE_DIR}/utils/sample_type_conversion.cc
        ${RA_SOURCE_DIR}/utils/sample_type_conversion.h
        ${RA_SOURCE_DIR}/utils/semi_lockless_fifo.h
        ${RA_SOURCE_DIR}/utils/sum_and_difference_processor.cc
        ${RA_SOURCE_DIR}/utils/sum_and_difference_processor.h
        ${RA_SOURCE_DIR}/utils/threadsafe_fifo.h
        ${RA_SOURCE_DIR}/utils/wav.cc
        ${RA_SOURCE_DIR}/utils/wav.h
        ${RA_SOURCE_DIR}/utils/wav_reader.cc
        ${RA_SOURCE_DIR}/utils/wav_reader.h
        )

add_library(ResonanceAudioObj OBJECT ${RA_SOURCES})
target_include_directories(ResonanceAudioObj PRIVATE ${PROJECT_SOURCE_DIR}/resonance_audio/)
target_include_directories(ResonanceAudioObj PRIVATE ${EIGEN3_INCLUDE_DIR}/)
target_include_directories(ResonanceAudioObj PRIVATE ${PFFFT_INCLUDE_DIR}/)


if (BUILD_RESONANCE_AUDIO_API)
    # Build shared library
    add_library(ResonanceAudioShared SHARED $<TARGET_OBJECTS:ResonanceAudioObj>
            $<TARGET_OBJECTS:SadieHrtfsObj>
            $<TARGET_OBJECTS:PffftObj>)

    # Build static library
    add_library(ResonanceAudioStatic STATIC $<TARGET_OBJECTS:ResonanceAudioObj>
            $<TARGET_OBJECTS:SadieHrtfsObj>
            $<TARGET_OBJECTS:PffftObj>)

    set(RA_INSTALL_DIR "${INSTALL_DIR}/resonance_audio/")
    install(TARGETS ResonanceAudioStatic ResonanceAudioShared
            LIBRARY DESTINATION ${RA_INSTALL_DIR}/lib
            ARCHIVE DESTINATION ${RA_INSTALL_DIR}/lib
            RUNTIME DESTINATION ${RA_INSTALL_DIR}/lib
            PUBLIC_HEADER DESTINATION ${RA_INSTALL_DIR}/include)

    set(RA_PUBLIC_HEADERS
            ${RA_SOURCE_DIR}/api/binaural_surround_renderer.h
            ${RA_SOURCE_DIR}/api/resonance_audio_api.h
            )
    install(FILES ${RA_PUBLIC_HEADERS}
            DESTINATION ${RA_INSTALL_DIR}/include)
endif (BUILD_RESONANCE_AUDIO_API)


if (BUILD_RESONANCE_AUDIO_TESTS)
    set(RA_TESTS
            ${RA_SOURCE_DIR}/ambisonics/ambisonic_binaural_decoder_test.cc
            ${RA_SOURCE_DIR}/ambisonics/ambisonic_codec_impl_test.cc
            ${RA_SOURCE_DIR}/ambisonics/ambisonic_lookup_table_test.cc
            ${RA_SOURCE_DIR}/ambisonics/associated_legendre_polynomials_generator_test.cc
            ${RA_SOURCE_DIR}/ambisonics/foa_rotator_test.cc
            ${RA_SOURCE_DIR}/ambisonics/hoa_rotator_test.cc
            ${RA_SOURCE_DIR}/ambisonics/stereo_from_soundfield_converter_test.cc
            ${RA_SOURCE_DIR}/ambisonics/utils_test.cc
            ${RA_SOURCE_DIR}/base/aligned_allocator_test.cc
            ${RA_SOURCE_DIR}/base/audio_buffer_test.cc
            ${RA_SOURCE_DIR}/base/channel_view_test.cc
            ${RA_SOURCE_DIR}/base/misc_math_test.cc
            ${RA_SOURCE_DIR}/base/simd_utils_test.cc
            ${RA_SOURCE_DIR}/base/spherical_angle_test.cc
            ${RA_SOURCE_DIR}/dsp/biquad_filter_test.cc
            ${RA_SOURCE_DIR}/dsp/channel_converter_test.cc
            ${RA_SOURCE_DIR}/dsp/circular_buffer_test.cc
            ${RA_SOURCE_DIR}/dsp/delay_filter_test.cc
            ${RA_SOURCE_DIR}/dsp/distance_attenuation_test.cc
            ${RA_SOURCE_DIR}/dsp/fft_manager_test.cc
            ${RA_SOURCE_DIR}/dsp/filter_coefficient_generators_test.cc
            ${RA_SOURCE_DIR}/dsp/fir_filter_test.cc
            ${RA_SOURCE_DIR}/dsp/gain_mixer_test.cc
            ${RA_SOURCE_DIR}/dsp/gain_processor_test.cc
            ${RA_SOURCE_DIR}/dsp/gain_test.cc
            ${RA_SOURCE_DIR}/dsp/mixer_test.cc
            ${RA_SOURCE_DIR}/dsp/mono_pole_filter_test.cc
            ${RA_SOURCE_DIR}/dsp/multi_channel_iir_test.cc
            ${RA_SOURCE_DIR}/dsp/occlusion_calculator_test.cc
            ${RA_SOURCE_DIR}/dsp/partitioned_fft_filter_test.cc
            ${RA_SOURCE_DIR}/dsp/reflections_processor_test.cc
            ${RA_SOURCE_DIR}/dsp/resampler_test.cc
            ${RA_SOURCE_DIR}/dsp/shoe_box_room_test.cc
            ${RA_SOURCE_DIR}/dsp/spectral_reverb_test.cc
            ${RA_SOURCE_DIR}/dsp/stereo_panner_test.cc
            ${RA_SOURCE_DIR}/dsp/utils_test.cc
            ${RA_SOURCE_DIR}/graph/ambisonic_mixing_encoder_node_test.cc
            ${RA_SOURCE_DIR}/graph/binaural_surround_renderer_impl_test.cc
            ${RA_SOURCE_DIR}/graph/occlusion_node_test.cc
            ${RA_SOURCE_DIR}/graph/gain_mixer_node_test.cc
            ${RA_SOURCE_DIR}/graph/gain_node_test.cc
            ${RA_SOURCE_DIR}/graph/mixer_node_test.cc
            ${RA_SOURCE_DIR}/graph/source_parameters_manager_test.cc
            ${RA_SOURCE_DIR}/node/audio_nodes_test.cc
            ${RA_SOURCE_DIR}/node/node_test.cc
            ${PROJECT_SOURCE_DIR}/platforms/common/room_effects_utils_test.cc
            ${RA_SOURCE_DIR}/utils/buffer_crossfader_test.cc
            ${RA_SOURCE_DIR}/utils/buffer_partitioner_test.cc
            ${RA_SOURCE_DIR}/utils/buffer_unpartitioner_test.cc
            ${RA_SOURCE_DIR}/utils/lockless_task_queue_test.cc
            ${RA_SOURCE_DIR}/utils/planar_interleaved_conversion_test.cc
            ${RA_SOURCE_DIR}/utils/pseudoinverse_test.cc
            ${RA_SOURCE_DIR}/utils/sample_type_conversion_test.cc
            ${RA_SOURCE_DIR}/utils/sum_and_difference_processor_test.cc
            ${RA_SOURCE_DIR}/utils/test_util.cc
            ${RA_SOURCE_DIR}/utils/test_util.h
            ${RA_SOURCE_DIR}/utils/test_util_test.cc
            )

    # Unit Tests target.
    add_executable(ResonanceAudioUnitTests ${RA_TESTS}
                                           $<TARGET_OBJECTS:gtest>
                                           $<TARGET_OBJECTS:ResonanceAudioObj>
                                           $<TARGET_OBJECTS:SadieHrtfsObj>
                                           $<TARGET_OBJECTS:PffftObj>)

    target_include_directories(ResonanceAudioUnitTests PRIVATE "${EIGEN3_INCLUDE_DIR}/")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${PFFFT_INCLUDE_DIR}/")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${PROJECT_SOURCE_DIR}/resonance_audio/")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${GTEST_DIR}/googlemock/")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${GTEST_DIR}/googlemock/include")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${GTEST_DIR}/googletest/")
    target_include_directories(ResonanceAudioUnitTests PRIVATE "${GTEST_DIR}/googletest/include")

    if (NOT WIN32)
        find_package(Threads REQUIRED)
        target_link_libraries(ResonanceAudioUnitTests pthread)
    endif (NOT WIN32)

    add_test(NAME runResonanceAudioUnitTest COMMAND $<TARGET_FILE:ResonanceAudioUnitTests>)

    add_custom_command(
            TARGET ResonanceAudioUnitTests
            COMMENT "Run resonance audio tests"
            POST_BUILD
            COMMAND ResonanceAudioUnitTests
    )
endif (BUILD_RESONANCE_AUDIO_TESTS)
