/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "ambisonics/ambisonic_binaural_decoder.h"

#include "ambisonics/utils.h"
#include "base/constants_and_types.h"


namespace vraudio {

AmbisonicBinauralDecoder::AmbisonicBinauralDecoder(const AudioBuffer& sh_hrirs,
                                                   size_t frames_per_buffer,
                                                   FftManager* fft_manager)
    : fft_manager_(fft_manager),
      freq_input_(kNumMonoChannels, NextPowTwo(frames_per_buffer) * 2),
      filtered_input_(kNumMonoChannels, frames_per_buffer) {
  CHECK(fft_manager_);
  CHECK_NE(frames_per_buffer, 0U);
  const size_t num_channels = sh_hrirs.num_channels();
  const size_t filter_size = sh_hrirs.num_frames();
  CHECK_NE(num_channels, 0U);
  CHECK_NE(filter_size, 0U);
  sh_hrir_filters_.reserve(num_channels);
  for (size_t i = 0; i < num_channels; ++i) {
    sh_hrir_filters_.emplace_back(
        new PartitionedFftFilter(filter_size, frames_per_buffer, fft_manager_));
    sh_hrir_filters_[i]->SetTimeDomainKernel(sh_hrirs[i]);
  }
}

void AmbisonicBinauralDecoder::Process(const AudioBuffer& input,
                                       AudioBuffer* output) {

  DCHECK(output);
  DCHECK_EQ(kNumStereoChannels, output->num_channels());
  DCHECK_EQ(input.num_frames(), output->num_frames());
  DCHECK_EQ(input.num_channels(), sh_hrir_filters_.size());

  output->Clear();

  AudioBuffer::Channel* freq_input_channel = &freq_input_[0];
  AudioBuffer::Channel* filtered_input_channel = &filtered_input_[0];
  AudioBuffer::Channel* output_channel_0 = &(*output)[0];
  AudioBuffer::Channel* output_channel_1 = &(*output)[1];
  for (size_t channel = 0; channel < input.num_channels(); ++channel) {
    const int degree = GetPeriphonicAmbisonicDegreeForChannel(channel);
    fft_manager_->FreqFromTimeDomain(input[channel], freq_input_channel);
    sh_hrir_filters_[channel]->Filter(*freq_input_channel);
    sh_hrir_filters_[channel]->GetFilteredSignal(filtered_input_channel);
    if (degree < 0) {
      // Degree is negative: spherical harmonic is asymetric.
      // So add contributions to the left channel and subtract from the right
      // channel.
      *output_channel_0 += *filtered_input_channel;
      *output_channel_1 -= *filtered_input_channel;

    } else {
      // Degree is zero or positive: spherical harmonic is symetric.
      // So add contributions to both left and right channels.
      *output_channel_0 += *filtered_input_channel;
      *output_channel_1 += *filtered_input_channel;
    }
  }
}

}  // namespace vraudio
