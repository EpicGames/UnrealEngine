/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_AMBISONIC_BINAURAL_DECODER_H_
#define RESONANCE_AUDIO_AMBISONICS_AMBISONIC_BINAURAL_DECODER_H_

#include <vector>

#include "base/audio_buffer.h"
#include "dsp/fft_manager.h"
#include "dsp/partitioned_fft_filter.h"

namespace vraudio {

// Decodes an Ambisonic sound field, of an arbitrary order, to binaural stereo,
// by performing convolution in the spherical harmonics domain. The order (hence
// the number of channels) of the input must match the order (hence the number
// of channels) of the spherical harmonic-encoded Head Related Impulse Responses
// (HRIRs). Assumes that HRIRs are symmetric with respect to the sagittal plane.
class AmbisonicBinauralDecoder {
 public:
  // Constructs an |AmbisonicBinauralDecoder| from an |AudioBuffer| containing
  // spherical harmonic representation of HRIRs. The order of spherical
  // harmonic-encoded HRIRs (hence the number of channels) must match the order
  // of the Ambisonic sound field input.
  //
  // @param sh_hrirs |AudioBuffer| containing time-domain spherical harmonic
  //   encoded symmetric HRIRs.
  // @param frames_per_buffer Number of frames in each input/output buffer.
  // @param fft_manager Pointer to a manager to perform FFT transformations.
  AmbisonicBinauralDecoder(const AudioBuffer& sh_hrirs,
                           size_t frames_per_buffer, FftManager* fft_manager);

  // Processes an Ambisonic sound field input and outputs a binaurally decoded
  // stereo buffer.
  //
  // @param input Input buffer to be processed.
  // @param output Pointer to a stereo output buffer.
  void Process(const AudioBuffer& input, AudioBuffer* output);

 private:
  // Manager for all FFT related functionality (not owned).
  FftManager* const fft_manager_;

  // Spherical Harmonic HRIR filter kernels.
  std::vector<std::unique_ptr<PartitionedFftFilter>> sh_hrir_filters_;

  // Frequency domain representation of the input signal.
  PartitionedFftFilter::FreqDomainBuffer freq_input_;

  // Temporary audio buffer to store the convolution output for asymmetric or
  // symmetric spherical harmonic HRIR.
  AudioBuffer filtered_input_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_AMBISONIC_BINAURAL_DECODER_H_
