/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_H_
#define RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_H_

#include <vector>

#include "base/audio_buffer.h"
#include "base/spherical_angle.h"

namespace vraudio {
// An encoder/decoder for ambisonic sound fields. It supports variable ambisonic
// order, ACN channel sequencing and SN3D normalization.
class AmbisonicCodec {
 public:
  virtual ~AmbisonicCodec() = default;
  // Encodes the given buffer of decoded vectors (unencoded data).
  //
  // @param input |AudioBuffer| of decoded (unencoded) data.
  // @param output |AudioBuffer| to store the encoded data.
  virtual void EncodeBuffer(const AudioBuffer& input, AudioBuffer* output) = 0;

  // Decodes the given |AudioBuffer| of planar (non-interleaved) encoded data.
  //
  // @param input |AudioBuffer| of encoded data.
  // @param output |AudioBuffer| to store the decoded data.
  virtual void DecodeBuffer(const AudioBuffer& input, AudioBuffer* output) = 0;

  // Returns the maximum periphonic ambisonic order that this codec supports.
  virtual int ambisonic_order() const = 0;

  // Returns the number of angles for this codec.
  virtual size_t num_angles() const = 0;

  // Returns the maximum number of spherical harmonics computed by this codec.
  virtual size_t num_spherical_harmonics() const = 0;

  // Returns the spherical angles currently used to define the encoder/decoder
  // matrices.
  virtual const std::vector<SphericalAngle>& angles() const = 0;

  // Sets the spherical angles used to build the encoder/decoder matrices.
  virtual void set_angles(const std::vector<SphericalAngle>& angles) = 0;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_H_
