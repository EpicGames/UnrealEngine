/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_IMPL_H_
#define RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_IMPL_H_

#include <cmath>
#include <vector>

#include "ResonanceEigenHelper.h"
#include "ambisonics/ambisonic_codec.h"
#include "ambisonics/associated_legendre_polynomials_generator.h"
#include "ambisonics/utils.h"
#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/spherical_angle.h"
#include "utils/pseudoinverse.h"

namespace vraudio {
// An encoder/decoder for ambisonic sound fields. It supports variable ambisonic
// order, ACN channel sequencing and SN3D normalization.
//
// @tparam NumAngles Used to fix the number of angles to be encoded/decoded at
//      compile-time; use |Eigen::Dynamic| to indicate run-time variability.
// @tparam NumSphericalHarmonics Used to fix the number of spherical harmonic
//     components at compile time; use |Eigen::Dynamic| to indicate run-time
//     variability.
template <int NumAngles = Eigen::Dynamic,
          int NumSphericalHarmonics = Eigen::Dynamic>
class AmbisonicCodecImpl : public AmbisonicCodec {
 public:
  // Spherical harmonics encoder matrix.
  typedef Eigen::Matrix<float, NumSphericalHarmonics, NumAngles> EncoderMatrix;
  // Spherical harmonics decoder matrix.
  typedef Eigen::Matrix<float, NumAngles, NumSphericalHarmonics> DecoderMatrix;
  // Spherical harmonics encoding of a frame or collection of frames (i.e., a
  // vector of spherical harmonics).
  typedef Eigen::Matrix<float, NumSphericalHarmonics, 1> EncodedVector;
  // Decoded sequence of values for each angle / mono frame (i.e., a vector with
  // a decoded mono frame for each angle).
  typedef Eigen::Matrix<float, NumAngles, 1> DecodedVector;

  // Creates a codec with the given |ambisonic_order| and spherical |angles| to
  // compute encoder/decoder matrices.
  AmbisonicCodecImpl(int ambisonic_order,
                     const std::vector<SphericalAngle>& angles);

  // Implements |AmbisonicCodec|.
  void EncodeBuffer(const AudioBuffer& input, AudioBuffer* output) override;
  void DecodeBuffer(const AudioBuffer& input, AudioBuffer* output) override;
  int ambisonic_order() const override;
  size_t num_angles() const override;
  size_t num_spherical_harmonics() const override;
  const std::vector<SphericalAngle>& angles() const override;
  void set_angles(const std::vector<SphericalAngle>& angles) override;

  // Encodes the given vector.

  void Encode(const Eigen::Ref<const Eigen::MatrixXf> decoded_vector,
              Eigen::Ref<Eigen::MatrixXf> encoded_vector);

  // Decodes the given vector.

  void Decode(const Eigen::Ref<const Eigen::MatrixXf> encoded_vector,
              Eigen::Ref<Eigen::MatrixXf> decoded_vector);

  // Gets the ambisonic sound field encoder matrix.
  const Eigen::Ref<const Eigen::MatrixXf> GetEncoderMatrix();

  // Gets the ambisonic sound field decoder matrix.
  const Eigen::Ref<const Eigen::MatrixXf> GetDecoderMatrix();

  // Necessary due to Eigen's alignment requirements on some platforms.
  EIGEN_MAKE_ALIGNED_OPERATOR_NEW

 private:
  // Returns the unnormalized spherical harmonic
  // Y_degree^order(azimuth, elevation).
  float UnnormalizedSphericalHarmonic(int degree, int order,
                                      const SphericalAngle& angle) const;

  // The maximum-ordered ambisonic sound field handled by this codec. In the
  // case of a periphonic codec, this is the order of the ambisonic sound field.
  const int ambisonic_order_;
  // Spherical angles used to compute spherical harmonics. For example, for a
  // decoder, virtual loudspeaker positions; for an encoder, the position(s) of
  // virtual sources relative to the listener.
  std::vector<SphericalAngle> angles_;
  // Current spherical harmonics encoder matrix if encoder_matrix_invalid_ is
  // false.
  EncoderMatrix encoder_matrix_;
  // True if encoder_matrix_ needs to be recomputed.
  bool encoder_matrix_invalid_;
  // Current spherical harmonics decoder matrix if encoder_matrix_invalid_ is
  // false.
  DecoderMatrix decoder_matrix_;
  // True if decoder_matrix_ needs to be recomputed.
  bool decoder_matrix_invalid_;
  // The associated Legendre polynomial generator for this codec.
  AssociatedLegendrePolynomialsGenerator alp_generator_;
  // Temporary storage for associated Legendre polynomials generated.
  std::vector<float> associated_legendre_polynomials_temp_;
};

template <int NumAngles, int NumSphericalHarmonics>
AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::AmbisonicCodecImpl(
    int ambisonic_order, const std::vector<SphericalAngle>& angles)
    : ambisonic_order_(ambisonic_order),
      alp_generator_(ambisonic_order, false, false) {
  DCHECK_GE(ambisonic_order_, 0);
  set_angles(angles);
}

template <int NumAngles, int NumSphericalHarmonics>
void AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::Encode(
    const Eigen::Ref<const Eigen::MatrixXf> decoded_vector,
    Eigen::Ref<Eigen::MatrixXf> encoded_vector) {
  encoded_vector.noalias() = GetEncoderMatrix() * decoded_vector;
}

template <int NumAngles, int NumSphericalHarmonics>
void AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::Decode(
    const Eigen::Ref<const Eigen::MatrixXf> encoded_vector,
    Eigen::Ref<Eigen::MatrixXf> decoded_vector) {
  decoded_vector.noalias() = GetDecoderMatrix() * encoded_vector;
}

template <int NumAngles, int NumSphericalHarmonics>
void AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::EncodeBuffer(
    const AudioBuffer& input, AudioBuffer* output) {
  CHECK(output);
  CHECK_EQ(input.num_channels(), num_angles());
  CHECK_EQ(output->num_channels(), num_spherical_harmonics());
  CHECK_EQ(input.num_frames(), output->num_frames());

  Eigen::Map<const Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic,
                                 Eigen::RowMajor>,
             Eigen::Aligned>
      unencoded_buffer(&input[0][0], num_angles(), output->GetChannelStride());

  Eigen::Map<
      Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::RowMajor>,
      Eigen::Aligned>
      encoded_buffer(&(*output)[0][0], num_spherical_harmonics(),
                     input.GetChannelStride());

  encoded_buffer.noalias() = GetEncoderMatrix() * unencoded_buffer;
}

template <int NumAngles, int NumSphericalHarmonics>
void AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::DecodeBuffer(
    const AudioBuffer& input, AudioBuffer* output) {
  CHECK(output);
  CHECK_EQ(input.num_channels(), num_spherical_harmonics());
  CHECK_EQ(output->num_channels(), num_angles());
  CHECK_EQ(input.num_frames(), output->num_frames());

  Eigen::Map<const Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic,
                                 Eigen::RowMajor>,
             Eigen::Aligned>
      encoded_buffer(&input[0][0], num_spherical_harmonics(),
                     input.GetChannelStride());

  Eigen::Map<
      Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::RowMajor>,
      Eigen::Aligned>
      decoded_buffer(&(*output)[0][0], num_angles(),
                     output->GetChannelStride());

  decoded_buffer.noalias() = GetDecoderMatrix() * encoded_buffer;
}

template <int NumAngles, int NumSphericalHarmonics>
const Eigen::Ref<const Eigen::MatrixXf>
AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::GetEncoderMatrix() {
  if (encoder_matrix_invalid_) {
    encoder_matrix_ = EncoderMatrix(
        GetNumPeriphonicComponents(ambisonic_order_), angles_.size());
    for (int col = 0; col < encoder_matrix_.cols(); col++) {
      const SphericalAngle& angle = angles_[col];
      associated_legendre_polynomials_temp_ =
          alp_generator_.Generate(std::sin(angle.elevation()));
      // Compute the actual spherical harmonics using the generated polynomials.
      for (int degree = 0; degree <= ambisonic_order_; degree++) {
        for (int order = -degree; order <= degree; order++) {
          const int row = AcnSequence(degree, order);
          if (row == -1) {
            // Skip this spherical harmonic.
            continue;
          }
          encoder_matrix_(row, col) =
              Sn3dNormalization(degree, order) *
              UnnormalizedSphericalHarmonic(degree, order, angle);
        }
      }
    }
    encoder_matrix_invalid_ = false;
  }
  return encoder_matrix_;
}

template <int NumAngles, int NumSphericalHarmonics>
const Eigen::Ref<const Eigen::MatrixXf>
AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::GetDecoderMatrix() {
  if (decoder_matrix_invalid_) {
    decoder_matrix_ = Pseudoinverse<Eigen::MatrixXf>(GetEncoderMatrix());
    // Condition number of the encoding/decoding matrices. We use the fact that
    // the decoding matrix is already a (pseudo)-inverse of the encoding matrix.
    const float condition_number =
        static_cast<float>(GetEncoderMatrix().norm() * decoder_matrix_.norm());
    const float num_rows = static_cast<float>(GetEncoderMatrix().rows());
    const float num_cols = static_cast<float>(GetEncoderMatrix().cols());
    if (condition_number >
        1.0f / (std::max(num_rows, num_cols) * kEpsilonFloat)) {
      LOG(WARNING) << "Ambisonic decoding matrix is ill-conditioned. Results "
                   << "may be inaccurate.";
    }
    decoder_matrix_invalid_ = false;
  }
  return decoder_matrix_;
}

template <int NumAngles, int NumSphericalHarmonics>
int AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::ambisonic_order()
    const {
  return ambisonic_order_;
}

template <int NumAngles, int NumSphericalHarmonics>
size_t AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::num_angles()
    const {
  return angles_.size();
}

template <int NumAngles, int NumSphericalHarmonics>
size_t AmbisonicCodecImpl<
    NumAngles, NumSphericalHarmonics>::num_spherical_harmonics() const {
  // Return the worst-case scenario (the number of coefficients for a
  // periphonic sound field).
  return GetNumPeriphonicComponents(ambisonic_order_);
}

template <int NumAngles, int NumSphericalHarmonics>
const std::vector<SphericalAngle>&
AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::angles() const {
  return angles_;
}

template <int NumAngles, int NumSphericalHarmonics>
void AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::set_angles(
    const std::vector<SphericalAngle>& angles) {
  CHECK_GT(angles.size(), 0);

  angles_ = angles;
  encoder_matrix_invalid_ = decoder_matrix_invalid_ = true;
}

template <int NumAngles, int NumSphericalHarmonics>
float AmbisonicCodecImpl<NumAngles, NumSphericalHarmonics>::
    UnnormalizedSphericalHarmonic(int degree, int order,
                                  const SphericalAngle& angle) const {
  const float last_term =
      (order >= 0) ? std::cos(static_cast<float>(order) * angle.azimuth())
                   : std::sin(static_cast<float>(-order) * angle.azimuth());
  return associated_legendre_polynomials_temp_[alp_generator_.GetIndex(
             degree, std::abs(order))] *
         last_term;
}

// Codec for a single source.
template <int NumSphericalHarmonics = Eigen::Dynamic>
using MonoAmbisonicCodec = AmbisonicCodecImpl<1, NumSphericalHarmonics>;

// Codec for a N-speaker first-order periphonic setup.
template <int NumAngles>
using FirstOrderPeriphonicAmbisonicCodec =
    AmbisonicCodecImpl<NumAngles, GetNumPeriphonicComponentsStatic<1>::value>;

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_AMBISONIC_CODEC_IMPL_H_
