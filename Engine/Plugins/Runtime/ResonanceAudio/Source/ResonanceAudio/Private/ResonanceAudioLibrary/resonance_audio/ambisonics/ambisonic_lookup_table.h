/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_AMBISONIC_LOOKUP_TABLE_H_
#define RESONANCE_AUDIO_AMBISONICS_AMBISONIC_LOOKUP_TABLE_H_

#include <vector>

#include "base/spherical_angle.h"

namespace vraudio {

// Represents a lookup table for encoding of Ambisonic periphonic sound fields.
// Supports arbitrary Ambisonic order and uses AmbiX convention (ACN channel
// sequencing, SN3D normalization).
class AmbisonicLookupTable {
 public:
  // Creates Ambisonic (AmbiX) encoder lookup table given the
  // |max_ambisonic_order| used by the client. AmbiX convention uses ACN channel
  // sequencing and SN3D normalization.
  explicit AmbisonicLookupTable(int max_ambisonic_order);

  // Gets spherical harmonic encoding coefficients for a given order and
  // writes them to |encoding_coeffs|.
  //
  // @param ambisonic_order Ambisonic order of the encoded sound source.
  // @param source_direction Direction of a sound source in spherical
  //     coordinates.
  // @param source_spread_deg Encoded sound source angular spread in degrees.
  // @param encoding_coeffs Pointer to a vector of Ambisonic encoding
  //     coefficients.
  void GetEncodingCoeffs(int ambisonic_order,
                         const SphericalAngle& source_direction,
                         float source_spread_deg,
                         std::vector<float>* encoding_coeffs) const;

 private:
  // Computes a lookup table of encoding coefficients for one quadrant of the
  // sphere.
  void ComputeEncoderTable();

  // Computes a table of spherical harmonics symmetry information for all
  // cartesian axes. Value of 1 indicates that the current spherical harmonic is
  // symmetric with respect to the current axis. Value of -1 indicates that the
  // current spherical harmonic is anti-symmetric with respect to the current
  // axis.
  void ComputeSymmetriesTable();

  // Returns the unnormalized spherical harmonic coefficient:
  // Y_degree^order(azimuth, elevation).
  //
  // @param alp_value Associated Legendre polynomial for the given degree and
  //     order evaluated at sin elevation angle: P_degree^order(sin(elevation)).
  // @param order Order of the Associated Legendre polynomial.
  // @param azimuth_rad Azimuth angle in radians.
  // @return Unnormalized spherical harmonic coefficient.
  float UnnormalizedSphericalHarmonic(float alp_value, int order,
                                      float azimuth_rad);

  // Ambisonic order.
  const int max_ambisonic_order_;

  // Maximum number of coefficients to be stored in the lookup table equal to
  // the number of Ambisonic channels for |max_ambisonic_order_| - 1. This is
  // because we do not need to store the coefficient for the first spherical
  // harmonic coefficient as it is always 1.
  const size_t max_num_coeffs_in_table_;

  // Lookup table for storing Ambisonic encoding coefficients for one quadrant
  // of the sphere.
  std::vector<float> encoder_table_;

  // Lookup table for storing information about spherical harmonic symmetries.
  std::vector<float> symmetries_table_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_AMBISONIC_LOOKUP_TABLE_H_
