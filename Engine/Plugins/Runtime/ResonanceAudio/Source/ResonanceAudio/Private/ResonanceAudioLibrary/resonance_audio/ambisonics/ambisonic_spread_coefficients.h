/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_AMBISONIC_SPREAD_COEFFICIENTS_H_
#define RESONANCE_AUDIO_AMBISONICS_AMBISONIC_SPREAD_COEFFICIENTS_H_

namespace vraudio {

// Lookup table with gain coefficients to control Ambisonic sound source spread
// at different orders, up to and including order 3. For information on how the
// spread control coefficients have been derived please refer to Matlab code as
// well as the corresponding paper.
static const float kSpreadCoeffs[5089] = {
    1.0f,         1.0019f,      0.998092f,    1.01441f,     0.985357f,
    1.02704f,     0.972176f,    1.03975f,     0.958555f,    1.05252f,
    0.944499f,    1.06532f,     0.930017f,    1.07813f,     0.915119f,
    1.09091f,     0.899819f,    1.10365f,     0.884131f,    1.11631f,
    0.868073f,    1.12886f,     0.851665f,    1.14128f,     0.834929f,
    1.15354f,     0.817888f,    1.16561f,     0.800569f,    1.17747f,
    0.782999f,    1.1891f,      0.76521f,     1.20046f,     0.747231f,
    1.21155f,     0.729095f,    1.22234f,     0.710837f,    1.23281f,
    0.692489f,    1.24296f,     0.674086f,    1.25276f,     0.655664f,
    1.26221f,     0.637256f,    1.27131f,     0.618896f,    1.28003f,
    0.600619f,    1.28839f,     0.582456f,    1.29637f,     0.564439f,
    1.30399f,     0.546597f,    1.31123f,     0.528959f,    1.31811f,
    0.511551f,    1.32463f,     0.494397f,    1.3308f,      0.47752f,
    1.33663f,     0.46094f,     1.34212f,     0.444677f,    1.34729f,
    0.428746f,    1.35214f,     0.413161f,    1.3567f,      0.397935f,
    1.36096f,     0.383079f,    1.36495f,     0.3686f,      1.36867f,
    0.354505f,    1.37215f,     0.340799f,    1.37538f,     0.327486f,
    1.37839f,     0.314566f,    1.38118f,     0.302039f,    1.38378f,
    0.289906f,    1.38618f,     0.278163f,    1.38841f,     0.266808f,
    1.39047f,     0.255835f,    1.39238f,     0.24524f,     1.39414f,
    0.235017f,    1.39576f,     0.225159f,    1.39726f,     0.21566f,
    1.39864f,     0.206511f,    1.39991f,     0.197704f,    1.40108f,
    0.189232f,    1.40215f,     0.181084f,    1.40314f,     0.173254f,
    1.40405f,     0.165731f,    1.40488f,     0.158507f,    1.40565f,
    0.151572f,    1.40635f,     0.144918f,    1.40699f,     0.138535f,
    1.40758f,     0.132414f,    1.40812f,     0.126546f,    1.40861f,
    0.120923f,    1.40906f,     0.115536f,    1.40947f,     0.110376f,
    1.40985f,     0.105436f,    1.4102f,      0.100705f,    1.41051f,
    0.0961781f,   1.4108f,      0.0918459f,   1.41107f,     0.087701f,
    1.41131f,     0.0837362f,   1.41153f,     0.0799443f,   1.41173f,
    0.0763185f,   1.41191f,     0.0728519f,   1.41208f,     0.069538f,
    1.41223f,     0.0663707f,   1.41237f,     0.0633438f,   1.41249f,
    0.0604515f,   1.41261f,     0.0576881f,   1.41271f,     0.0550481f,
    1.41281f,     0.0525264f,   1.4129f,      0.0501178f,   1.41298f,
    0.0478176f,   1.41305f,     0.045621f,    1.41312f,     0.0435236f,
    1.41318f,     0.0415211f,   1.41323f,     0.0396092f,   1.41328f,
    0.0377841f,   1.41333f,     0.0360419f,   1.41337f,     0.034379f,
    1.4134f,      0.0327918f,   1.41344f,     0.031277f,    1.41347f,
    0.0298315f,   1.4135f,      0.028452f,    1.41352f,     0.0271356f,
    1.41355f,     0.0258796f,   1.41357f,     0.0246811f,   1.41359f,
    0.0235377f,   1.41361f,     0.0224468f,   1.41362f,     0.0214061f,
    1.41364f,     0.0204132f,   1.41365f,     0.0194661f,   1.41366f,
    0.0185626f,   1.41367f,     0.0177008f,   1.41368f,     0.0168787f,
    1.41369f,     0.0160946f,   1.4137f,      0.0153468f,   1.41371f,
    0.0146335f,   1.41372f,     0.0139531f,   1.41372f,     0.0133043f,
    1.41373f,     0.0126855f,   1.41373f,     0.0120953f,   1.41374f,
    0.0115325f,   1.41374f,     0.0109958f,   1.41375f,     0.0104839f,
    1.41375f,     0.00999584f,  1.41375f,     0.00953039f,  1.41376f,
    0.00908654f,  1.41376f,     0.0086633f,   1.41376f,     0.00825972f,
    1.41376f,     0.00787488f,  1.41376f,     0.00750793f,  1.41377f,
    0.00715803f,  1.41377f,     0.0068244f,   1.41377f,     0.00650628f,
    1.41377f,     0.00620296f,  1.41377f,     0.00591376f,  1.41377f,
    0.00563801f,  1.41377f,     0.00537509f,  1.41378f,     0.00512441f,
    1.41378f,     0.0048854f,   1.41378f,     0.00465752f,  1.41378f,
    0.00444026f,  1.41378f,     0.00423311f,  1.41378f,     0.00403562f,
    1.41378f,     0.00384732f,  1.41378f,     0.0036678f,   1.41378f,
    0.00349665f,  1.41378f,     0.00333348f,  1.41378f,     0.00317791f,
    1.41378f,     0.00302959f,  1.41378f,     0.00288819f,  1.41378f,
    0.00275338f,  1.41378f,     0.00262486f,  1.41378f,     0.00250233f,
    1.41378f,     0.00238552f,  1.41378f,     0.00227415f,  1.41378f,
    0.00216798f,  1.41378f,     0.00206677f,  1.41378f,     0.00197027f,
    1.41378f,     0.00187828f,  1.41378f,     0.00179058f,  1.41378f,
    0.00170697f,  1.41378f,     0.00162727f,  1.41378f,     0.00155128f,
    1.41378f,     0.00147884f,  1.41378f,     0.00140979f,  1.41378f,
    0.00134395f,  1.41378f,     0.00128119f,  1.41378f,     0.00122136f,
    1.41378f,     0.00116432f,  1.41378f,     0.00110994f,  1.41378f,
    0.0010581f,   1.41378f,     0.00100869f,  1.41378f,     0.000961575f,
    1.41378f,     0.000916665f, 1.41378f,     0.000873851f, 1.41378f,
    0.000833036f, 1.41378f,     0.000794127f, 1.41378f,     0.000757035f,
    1.41378f,     0.000721675f, 1.41378f,     0.000687966f, 1.41378f,
    0.000655831f, 1.41378f,     0.000625197f, 1.41378f,     0.000595994f,
    1.41378f,     0.000568154f, 1.41378f,     0.000541615f, 1.41378f,
    0.000516315f, 1.41378f,     0.000492197f, 1.41378f,     0.000469205f,
    1.41378f,     0.000447287f, 1.41378f,     0.000426393f, 1.41378f,
    0.000406474f, 1.41378f,     0.000387486f, 1.41378f,     0.000369385f,
    1.41378f,     0.000352129f, 1.41378f,     0.00033568f,  1.41378f,
    0.000319998f, 1.41378f,     0.00030505f,  1.41378f,     0.000290799f,
    1.41378f,     0.000277214f, 1.41378f,     0.000264264f, 1.41378f,
    0.000251918f, 1.41378f,     0.00024015f,  1.41378f,     0.000228931f,
    1.41378f,     0.000218236f, 1.41378f,     0.000208041f, 1.41378f,
    0.000198322f, 1.41378f,     0.000189056f, 1.41378f,     0.000180224f,
    1.41378f,     0.000171805f, 1.41378f,     0.000163778f, 1.41378f,
    0.000156127f, 1.41378f,     0.000148833f, 1.41378f,     0.00014188f,
    1.41378f,     0.000135252f, 1.41378f,     0.000128933f, 1.41378f,
    0.000122909f, 1.41378f,     0.000117167f, 1.41378f,     0.000111693f,
    1.41378f,     0.000106475f, 1.41378f,     0.000101501f, 1.41378f,
    9.67588e-05f, 1.41378f,     9.22384e-05f, 1.41378f,     8.79291e-05f,
    1.41378f,     8.38211e-05f, 1.41378f,     7.99051e-05f, 1.41378f,
    7.6172e-05f,  1.41378f,     7.26133e-05f, 1.41378f,     6.92209e-05f,
    1.41378f,     6.5987e-05f,  1.41378f,     6.29041e-05f, 1.41378f,
    5.99653e-05f, 1.41378f,     5.71637e-05f, 1.41378f,     5.44931e-05f,
    1.41378f,     5.19472e-05f, 1.41378f,     4.95202e-05f, 1.41378f,
    4.72067e-05f, 1.41378f,     4.50012e-05f, 1.41378f,     4.28988e-05f,
    1.41378f,     4.08946e-05f, 1.41378f,     3.8984e-05f,  1.41378f,
    3.71627e-05f, 1.41378f,     3.54264e-05f, 1.41378f,     3.37713e-05f,
    1.41378f,     3.21935e-05f, 1.41378f,     3.06895e-05f, 1.41378f,
    2.92557e-05f, 1.41378f,     2.78888e-05f, 1.41378f,     2.65859e-05f,
    1.41378f,     2.53438e-05f, 1.41378f,     2.41597e-05f, 1.41378f,
    2.3031e-05f,  1.41378f,     2.1955e-05f,  1.41378f,     2.09292e-05f,
    1.41378f,     1.99514e-05f, 1.41378f,     1.90193e-05f, 1.41378f,
    1.81307e-05f, 1.41378f,     1.72837e-05f, 1.41378f,     1.64762e-05f,
    1.41378f,     1.57064e-05f, 1.41378f,     1.49726e-05f, 1.41378f,
    1.42731e-05f, 1.41378f,     1.36062e-05f, 1.41378f,     1.29706e-05f,
    1.41378f,     1.23646e-05f, 1.41378f,     1.17869e-05f, 1.41378f,
    1.12362e-05f, 1.41378f,     1.07113e-05f, 1.41378f,     1.02108e-05f,
    1.41378f,     9.73377e-06f, 1.41378f,     9.27901e-06f, 1.41378f,
    8.84549e-06f, 1.41378f,     8.43223e-06f, 1.41378f,     8.03827e-06f,
    1.41378f,     7.66273e-06f, 1.41378f,     7.30472e-06f, 1.41378f,
    6.96344e-06f, 1.41378f,     6.63811e-06f, 1.41378f,     6.32798e-06f,
    1.41378f,     6.03233e-06f, 1.41378f,     5.7505e-06f,  1.41378f,
    5.48184e-06f, 1.41378f,     5.22572e-06f, 1.41378f,     4.98158e-06f,
    1.41378f,     4.74884e-06f, 1.41378f,     4.52697e-06f, 1.41378f,
    4.31547e-06f, 1.41378f,     4.11385e-06f, 1.41378f,     3.92165e-06f,
    1.41378f,     3.73843e-06f, 1.41378f,     3.56377e-06f, 1.41378f,
    3.39727e-06f, 1.41378f,     3.23855e-06f, 1.41378f,     3.08724e-06f,
    1.41378f,     2.94301e-06f, 1.41378f,     2.80551e-06f, 1.41378f,
    2.67443e-06f, 1.41378f,     2.54948e-06f, 1.41378f,     2.43037e-06f,
    1.41378f,     2.31682e-06f, 1.41378f,     2.20858e-06f, 1.41378f,
    2.1054e-06f,  1.41378f,     2.00703e-06f, 1.41378f,     1.91326e-06f,
    1.41378f,     1.82388e-06f, 1.41378f,     1.73866e-06f, 1.41378f,
    1.65743e-06f, 1.41378f,     1.58e-06f,    1.41378f,     1.50618e-06f,
    1.41378f,     1.43581e-06f, 1.41378f,     1.36873e-06f, 1.41378f,
    1.30478e-06f, 1.41378f,     1.24382e-06f, 1.41378f,     1.18571e-06f,
    1.01483f,     1.00465f,     0.970383f,    1.03865f,     1.01155f,
    0.9215f,      1.06314f,     1.0179f,      0.869372f,    1.08819f,
    1.02356f,     0.813901f,    1.11364f,     1.02839f,     0.755004f,
    1.13935f,     1.03225f,     0.692625f,    1.16513f,     1.03497f,
    0.626735f,    1.19076f,     1.0364f,      0.557346f,    1.21602f,
    1.03638f,     0.484517f,    1.24065f,     1.03474f,     0.408362f,
    1.2644f,      1.03133f,     0.329059f,    1.28697f,     1.02601f,
    0.246853f,    1.3081f,      1.01866f,     0.162061f,    1.3275f,
    1.00918f,     0.0750718f,   1.34495f,     0.99753f,     0.0f,
    1.36175f,     0.984802f,    0.0f,         1.37869f,     0.971629f,
    0.0f,         1.39575f,     0.958015f,    0.0f,         1.4129f,
    0.943967f,    0.0f,         1.43008f,     0.929493f,    0.0f,
    1.44728f,     0.914603f,    0.0f,         1.46444f,     0.899312f,
    0.0f,         1.48154f,     0.883633f,    0.0f,         1.49853f,
    0.867584f,    0.0f,         1.51538f,     0.851185f,    0.0f,
    1.53206f,     0.834458f,    0.0f,         1.54851f,     0.817427f,
    0.0f,         1.56472f,     0.800117f,    0.0f,         1.58064f,
    0.782558f,    0.0f,         1.59624f,     0.764779f,    0.0f,
    1.6115f,      0.74681f,     0.0f,         1.62638f,     0.728685f,
    0.0f,         1.64086f,     0.710436f,    0.0f,         1.65492f,
    0.692098f,    0.0f,         1.66854f,     0.673706f,    0.0f,
    1.6817f,      0.655294f,    0.0f,         1.69439f,     0.636896f,
    0.0f,         1.7066f,      0.618548f,    0.0f,         1.71831f,
    0.600281f,    0.0f,         1.72953f,     0.582128f,    0.0f,
    1.74025f,     0.564121f,    0.0f,         1.75047f,     0.546289f,
    0.0f,         1.76019f,     0.528661f,    0.0f,         1.76943f,
    0.511262f,    0.0f,         1.77819f,     0.494118f,    0.0f,
    1.78647f,     0.477251f,    0.0f,         1.79429f,     0.46068f,
    0.0f,         1.80166f,     0.444426f,    0.0f,         1.8086f,
    0.428504f,    0.0f,         1.81511f,     0.412928f,    0.0f,
    1.82123f,     0.397711f,    0.0f,         1.82695f,     0.382863f,
    0.0f,         1.8323f,      0.368392f,    0.0f,         1.8373f,
    0.354306f,    0.0f,         1.84196f,     0.340607f,    0.0f,
    1.84631f,     0.327301f,    0.0f,         1.85035f,     0.314388f,
    0.0f,         1.8541f,      0.301869f,    0.0f,         1.85758f,
    0.289743f,    0.0f,         1.86081f,     0.278006f,    0.0f,
    1.8638f,      0.266657f,    0.0f,         1.86657f,     0.255691f,
    0.0f,         1.86912f,     0.245102f,    0.0f,         1.87149f,
    0.234885f,    0.0f,         1.87367f,     0.225033f,    0.0f,
    1.87568f,     0.215538f,    0.0f,         1.87753f,     0.206394f,
    0.0f,         1.87924f,     0.197593f,    0.0f,         1.8808f,
    0.189125f,    0.0f,         1.88225f,     0.180982f,    0.0f,
    1.88357f,     0.173156f,    0.0f,         1.88479f,     0.165638f,
    0.0f,         1.88591f,     0.158418f,    0.0f,         1.88694f,
    0.151487f,    0.0f,         1.88788f,     0.144836f,    0.0f,
    1.88874f,     0.138457f,    0.0f,         1.88953f,     0.132339f,
    0.0f,         1.89025f,     0.126475f,    0.0f,         1.89091f,
    0.120855f,    0.0f,         1.89152f,     0.115471f,    0.0f,
    1.89207f,     0.110314f,    0.0f,         1.89258f,     0.105376f,
    0.0f,         1.89305f,     0.100649f,    0.0f,         1.89347f,
    0.0961239f,   0.0f,         1.89386f,     0.0917941f,   0.0f,
    1.89421f,     0.0876516f,   0.0f,         1.89453f,     0.083689f,
    0.0f,         1.89483f,     0.0798993f,   0.0f,         1.8951f,
    0.0762755f,   0.0f,         1.89534f,     0.0728108f,   0.0f,
    1.89557f,     0.0694989f,   0.0f,         1.89577f,     0.0663333f,
    0.0f,         1.89596f,     0.0633081f,   0.0f,         1.89613f,
    0.0604174f,   0.0f,         1.89628f,     0.0576555f,   0.0f,
    1.89642f,     0.0550171f,   0.0f,         1.89655f,     0.0524968f,
    0.0f,         1.89667f,     0.0500896f,   0.0f,         1.89678f,
    0.0477907f,   0.0f,         1.89687f,     0.0455953f,   0.0f,
    1.89696f,     0.0434991f,   0.0f,         1.89704f,     0.0414977f,
    0.0f,         1.89712f,     0.0395869f,   0.0f,         1.89718f,
    0.0377628f,   0.0f,         1.89724f,     0.0360216f,   0.0f,
    1.8973f,      0.0343596f,   0.0f,         1.89735f,     0.0327733f,
    0.0f,         1.8974f,      0.0312594f,   0.0f,         1.89744f,
    0.0298146f,   0.0f,         1.89748f,     0.0284359f,   0.0f,
    1.89751f,     0.0271203f,   0.0f,         1.89754f,     0.025865f,
    0.0f,         1.89757f,     0.0246672f,   0.0f,         1.8976f,
    0.0235244f,   0.0f,         1.89762f,     0.0224342f,   0.0f,
    1.89764f,     0.021394f,    0.0f,         1.89766f,     0.0204017f,
    0.0f,         1.89768f,     0.0194551f,   0.0f,         1.8977f,
    0.0185521f,   0.0f,         1.89771f,     0.0176908f,   0.0f,
    1.89773f,     0.0168692f,   0.0f,         1.89774f,     0.0160856f,
    0.0f,         1.89775f,     0.0153381f,   0.0f,         1.89776f,
    0.0146252f,   0.0f,         1.89777f,     0.0139453f,   0.0f,
    1.89778f,     0.0132968f,   0.0f,         1.89778f,     0.0126783f,
    0.0f,         1.89779f,     0.0120885f,   0.0f,         1.8978f,
    0.011526f,    0.0f,         1.8978f,      0.0109896f,   0.0f,
    1.89781f,     0.010478f,    0.0f,         1.89781f,     0.00999021f,
    0.0f,         1.89782f,     0.00952502f,  0.0f,         1.89782f,
    0.00908142f,  0.0f,         1.89782f,     0.00865842f,  0.0f,
    1.89783f,     0.00825506f,  0.0f,         1.89783f,     0.00787044f,
    0.0f,         1.89783f,     0.00750369f,  0.0f,         1.89784f,
    0.00715399f,  0.0f,         1.89784f,     0.00682055f,  0.0f,
    1.89784f,     0.00650262f,  0.0f,         1.89784f,     0.00619947f,
    0.0f,         1.89784f,     0.00591042f,  0.0f,         1.89785f,
    0.00563483f,  0.0f,         1.89785f,     0.00537206f,  0.0f,
    1.89785f,     0.00512152f,  0.0f,         1.89785f,     0.00488265f,
    0.0f,         1.89785f,     0.0046549f,   0.0f,         1.89785f,
    0.00443776f,  0.0f,         1.89785f,     0.00423073f,  0.0f,
    1.89785f,     0.00403334f,  0.0f,         1.89785f,     0.00384516f,
    0.0f,         1.89785f,     0.00366574f,  0.0f,         1.89785f,
    0.00349468f,  0.0f,         1.89786f,     0.0033316f,   0.0f,
    1.89786f,     0.00317612f,  0.0f,         1.89786f,     0.00302788f,
    0.0f,         1.89786f,     0.00288656f,  0.0f,         1.89786f,
    0.00275183f,  0.0f,         1.89786f,     0.00262338f,  0.0f,
    1.89786f,     0.00250092f,  0.0f,         1.89786f,     0.00238417f,
    0.0f,         1.89786f,     0.00227287f,  0.0f,         1.89786f,
    0.00216676f,  0.0f,         1.89786f,     0.0020656f,   0.0f,
    1.89786f,     0.00196916f,  0.0f,         1.89786f,     0.00187722f,
    0.0f,         1.89786f,     0.00178957f,  0.0f,         1.89786f,
    0.00170601f,  0.0f,         1.89786f,     0.00162635f,  0.0f,
    1.89786f,     0.00155041f,  0.0f,         1.89786f,     0.00147801f,
    0.0f,         1.89786f,     0.00140899f,  0.0f,         1.89786f,
    0.00134319f,  0.0f,         1.89786f,     0.00128047f,  0.0f,
    1.89786f,     0.00122067f,  0.0f,         1.89786f,     0.00116366f,
    0.0f,         1.89786f,     0.00110932f,  0.0f,         1.89786f,
    0.00105751f,  0.0f,         1.89786f,     0.00100812f,  0.0f,
    1.89786f,     0.000961033f, 0.0f,         1.89786f,     0.000916148f,
    0.0f,         1.89786f,     0.000873358f, 0.0f,         1.89786f,
    0.000832566f, 0.0f,         1.89786f,     0.000793679f, 0.0f,
    1.89786f,     0.000756608f, 0.0f,         1.89786f,     0.000721268f,
    0.0f,         1.89786f,     0.000687578f, 0.0f,         1.89786f,
    0.000655462f, 0.0f,         1.89786f,     0.000624845f, 0.0f,
    1.89786f,     0.000595658f, 0.0f,         1.89786f,     0.000567834f,
    0.0f,         1.89786f,     0.00054131f,  0.0f,         1.89786f,
    0.000516024f, 0.0f,         1.89786f,     0.000491919f, 0.0f,
    1.89786f,     0.000468941f, 0.0f,         1.89786f,     0.000447035f,
    0.0f,         1.89786f,     0.000426152f, 0.0f,         1.89786f,
    0.000406245f, 0.0f,         1.89786f,     0.000387268f, 0.0f,
    1.89786f,     0.000369177f, 0.0f,         1.89786f,     0.000351931f,
    0.0f,         1.89786f,     0.000335491f, 0.0f,         1.89786f,
    0.000319818f, 0.0f,         1.89786f,     0.000304878f, 0.0f,
    1.89786f,     0.000290635f, 0.0f,         1.89786f,     0.000277058f,
    0.0f,         1.89786f,     0.000264115f, 0.0f,         1.89786f,
    0.000251776f, 0.0f,         1.89786f,     0.000240014f, 0.0f,
    1.89786f,     0.000228802f, 0.0f,         1.89786f,     0.000218113f,
    0.0f,         1.89786f,     0.000207923f, 0.0f,         1.89786f,
    0.00019821f,  0.0f,         1.89786f,     0.00018895f,  0.0f,
    1.89786f,     0.000180123f, 0.0f,         1.89786f,     0.000171708f,
    0.0f,         1.89786f,     0.000163686f, 0.0f,         1.89786f,
    0.000156039f, 0.0f,         1.89786f,     0.000148749f, 0.0f,
    1.89786f,     0.0001418f,   0.0f,         1.89786f,     0.000135175f,
    0.0f,         1.89786f,     0.00012886f,  0.0f,         1.89786f,
    0.00012284f,  0.0f,         1.89786f,     0.000117101f, 0.0f,
    1.89786f,     0.00011163f,  0.0f,         1.89786f,     0.000106415f,
    0.0f,         1.89786f,     0.000101444f, 0.0f,         1.89786f,
    9.67043e-05f, 0.0f,         1.89786f,     9.21864e-05f, 0.0f,
    1.89786f,     8.78795e-05f, 0.0f,         1.89786f,     8.37739e-05f,
    0.0f,         1.89786f,     7.98601e-05f, 0.0f,         1.89786f,
    7.61291e-05f, 0.0f,         1.89786f,     7.25724e-05f, 0.0f,
    1.89786f,     6.91819e-05f, 0.0f,         1.89786f,     6.59498e-05f,
    0.0f,         1.89786f,     6.28686e-05f, 0.0f,         1.89786f,
    5.99315e-05f, 0.0f,         1.89786f,     5.71315e-05f, 0.0f,
    1.89786f,     5.44624e-05f, 0.0f,         1.89786f,     5.19179e-05f,
    0.0f,         1.89786f,     4.94923e-05f, 0.0f,         1.89786f,
    4.71801e-05f, 0.0f,         1.89786f,     4.49758e-05f, 0.0f,
    1.89786f,     4.28746e-05f, 0.0f,         1.89786f,     4.08715e-05f,
    0.0f,         1.89786f,     3.8962e-05f,  0.0f,         1.89786f,
    3.71417e-05f, 0.0f,         1.89786f,     3.54065e-05f, 0.0f,
    1.89786f,     3.37523e-05f, 0.0f,         1.89786f,     3.21754e-05f,
    0.0f,         1.89786f,     3.06722e-05f, 0.0f,         1.89786f,
    2.92392e-05f, 0.0f,         1.89786f,     2.78731e-05f, 0.0f,
    1.89786f,     2.65709e-05f, 0.0f,         1.89786f,     2.53295e-05f,
    0.0f,         1.89786f,     2.41461e-05f, 0.0f,         1.89786f,
    2.3018e-05f,  0.0f,         1.89786f,     2.19426e-05f, 0.0f,
    1.89786f,     2.09175e-05f, 0.0f,         1.89786f,     1.99402e-05f,
    0.0f,         1.89786f,     1.90086e-05f, 0.0f,         1.89786f,
    1.81205e-05f, 0.0f,         1.89786f,     1.72739e-05f, 0.0f,
    1.89786f,     1.64669e-05f, 0.0f,         1.89786f,     1.56975e-05f,
    0.0f,         1.89786f,     1.49642e-05f, 0.0f,         1.89786f,
    1.4265e-05f,  0.0f,         1.89786f,     1.35986e-05f, 0.0f,
    1.89786f,     1.29632e-05f, 0.0f,         1.89786f,     1.23576e-05f,
    0.0f,         1.89786f,     1.17802e-05f, 0.0f,         1.89786f,
    1.12299e-05f, 0.0f,         1.89786f,     1.07052e-05f, 0.0f,
    1.89786f,     1.02051e-05f, 0.0f,         1.89786f,     9.72828e-06f,
    0.0f,         1.89786f,     9.27378e-06f, 0.0f,         1.89786f,
    8.84051e-06f, 0.0f,         1.89786f,     8.42748e-06f, 0.0f,
    1.89786f,     8.03374e-06f, 0.0f,         1.89786f,     7.65841e-06f,
    0.0f,         1.89786f,     7.3006e-06f,  0.0f,         1.89786f,
    6.95952e-06f, 0.0f,         1.89786f,     6.63437e-06f, 0.0f,
    1.89786f,     6.32441e-06f, 0.0f,         1.89786f,     6.02893e-06f,
    0.0f,         1.89786f,     5.74726e-06f, 0.0f,         1.89786f,
    5.47875e-06f, 0.0f,         1.89786f,     5.22278e-06f, 0.0f,
    1.89786f,     4.97877e-06f, 0.0f,         1.89786f,     4.74616e-06f,
    0.0f,         1.89786f,     4.52442e-06f, 0.0f,         1.89786f,
    4.31304e-06f, 0.0f,         1.89786f,     4.11153e-06f, 0.0f,
    1.89786f,     3.91944e-06f, 0.0f,         1.89786f,     3.73632e-06f,
    0.0f,         1.89786f,     3.56176e-06f, 0.0f,         1.89786f,
    3.39536e-06f, 0.0f,         1.89786f,     3.23672e-06f, 0.0f,
    1.89786f,     3.0855e-06f,  0.0f,         1.89786f,     2.94135e-06f,
    0.0f,         1.89786f,     2.80393e-06f, 0.0f,         1.89786f,
    2.67293e-06f, 0.0f,         1.89786f,     2.54805e-06f, 0.0f,
    1.89786f,     2.429e-06f,   0.0f,         1.89786f,     2.31552e-06f,
    0.0f,         1.89786f,     2.20734e-06f, 0.0f,         1.89786f,
    2.10421e-06f, 0.0f,         1.89786f,     2.0059e-06f,  0.0f,
    1.89786f,     1.91218e-06f, 0.0f,         1.89786f,     1.82285e-06f,
    0.0f,         1.89786f,     1.73768e-06f, 0.0f,         1.89786f,
    1.6565e-06f,  0.0f,         1.89786f,     1.57911e-06f, 0.0f,
    1.89786f,     1.50533e-06f, 0.0f,         1.89786f,     1.435e-06f,
    0.0f,         1.89786f,     1.36796e-06f, 0.0f,         1.89786f,
    1.30405e-06f, 0.0f,         1.89786f,     1.24312e-06f, 0.0f,
    1.89786f,     1.18504e-06f, 0.0f,         1.00324f,     1.00216f,
    0.999152f,    0.990038f,    1.03237f,     1.02119f,     0.990433f,
    0.898572f,    1.06269f,     1.04023f,     0.979161f,    0.799806f,
    1.094f,       1.05895f,     0.964976f,    0.693603f,    1.126f,
    1.07701f,     0.947526f,    0.57989f,     1.15835f,     1.09398f,
    0.926474f,    0.45869f,     1.19059f,     1.10944f,     0.901512f,
    0.330158f,    1.22223f,     1.12289f,     0.87237f,     0.194621f,
    1.25268f,     1.13384f,     0.838839f,    0.0526136f,   1.28199f,
    1.14236f,     0.801199f,    0.0f,         1.31207f,     1.15021f,
    0.760839f,    0.0f,         1.34301f,     1.15742f,     0.717799f,
    0.0f,         1.37465f,     1.16386f,     0.671999f,    0.0f,
    1.40681f,     1.16935f,     0.623371f,    0.0f,         1.43929f,
    1.17374f,     0.571868f,    0.0f,         1.47185f,     1.17684f,
    0.517465f,    0.0f,         1.50423f,     1.17846f,     0.460174f,
    0.0f,         1.53613f,     1.17844f,     0.400043f,    0.0f,
    1.56725f,     1.17657f,     0.337165f,    0.0f,         1.59725f,
    1.1727f,      0.271688f,    0.0f,         1.62577f,     1.16665f,
    0.203815f,    0.0f,         1.65245f,     1.15828f,     0.133806f,
    0.0f,         1.67697f,     1.14751f,     0.0619832f,   0.0f,
    1.69901f,     1.13426f,     0.0f,         0.0f,         1.72022f,
    1.11979f,     0.0f,         0.0f,         1.74163f,     1.10481f,
    0.0f,         0.0f,         1.76318f,     1.08933f,     0.0f,
    0.0f,         1.78484f,     1.07336f,     0.0f,         0.0f,
    1.80655f,     1.0569f,      0.0f,         0.0f,         1.82827f,
    1.03997f,     0.0f,         0.0f,         1.84995f,     1.02258f,
    0.0f,         0.0f,         1.87155f,     1.00475f,     0.0f,
    0.0f,         1.89302f,     0.986504f,    0.0f,         0.0f,
    1.91431f,     0.967857f,    0.0f,         0.0f,         1.93537f,
    0.948837f,    0.0f,         0.0f,         1.95615f,     0.929471f,
    0.0f,         0.0f,         1.97662f,     0.90979f,     0.0f,
    0.0f,         1.99674f,     0.889823f,    0.0f,         0.0f,
    2.01645f,     0.869607f,    0.0f,         0.0f,         2.03572f,
    0.849175f,    0.0f,         0.0f,         2.05452f,     0.828565f,
    0.0f,         0.0f,         2.07282f,     0.807816f,    0.0f,
    0.0f,         2.09058f,     0.786964f,    0.0f,         0.0f,
    2.10779f,     0.766051f,    0.0f,         0.0f,         2.12441f,
    0.745115f,    0.0f,         0.0f,         2.14044f,     0.724196f,
    0.0f,         0.0f,         2.15586f,     0.703332f,    0.0f,
    0.0f,         2.17065f,     0.682561f,    0.0f,         0.0f,
    2.18482f,     0.661921f,    0.0f,         0.0f,         2.19836f,
    0.641445f,    0.0f,         0.0f,         2.21128f,     0.621169f,
    0.0f,         0.0f,         2.22356f,     0.601125f,    0.0f,
    0.0f,         2.23523f,     0.581341f,    0.0f,         0.0f,
    2.24629f,     0.561847f,    0.0f,         0.0f,         2.25675f,
    0.542667f,    0.0f,         0.0f,         2.26663f,     0.523826f,
    0.0f,         0.0f,         2.27594f,     0.505344f,    0.0f,
    0.0f,         2.28471f,     0.487239f,    0.0f,         0.0f,
    2.29294f,     0.469528f,    0.0f,         0.0f,         2.30066f,
    0.452225f,    0.0f,         0.0f,         2.30789f,     0.435342f,
    0.0f,         0.0f,         2.31465f,     0.418888f,    0.0f,
    0.0f,         2.32097f,     0.40287f,     0.0f,         0.0f,
    2.32686f,     0.387294f,    0.0f,         0.0f,         2.33234f,
    0.372164f,    0.0f,         0.0f,         2.33745f,     0.357481f,
    0.0f,         0.0f,         2.34219f,     0.343246f,    0.0f,
    0.0f,         2.34659f,     0.329458f,    0.0f,         0.0f,
    2.35066f,     0.316113f,    0.0f,         0.0f,         2.35444f,
    0.303208f,    0.0f,         0.0f,         2.35794f,     0.290738f,
    0.0f,         0.0f,         2.36117f,     0.278698f,    0.0f,
    0.0f,         2.36415f,     0.26708f,     0.0f,         0.0f,
    2.36691f,     0.255878f,    0.0f,         0.0f,         2.36945f,
    0.245082f,    0.0f,         0.0f,         2.37179f,     0.234685f,
    0.0f,         0.0f,         2.37394f,     0.224677f,    0.0f,
    0.0f,         2.37592f,     0.215048f,    0.0f,         0.0f,
    2.37775f,     0.20579f,     0.0f,         0.0f,         2.37942f,
    0.196891f,    0.0f,         0.0f,         2.38096f,     0.188342f,
    0.0f,         0.0f,         2.38237f,     0.180132f,    0.0f,
    0.0f,         2.38367f,     0.172251f,    0.0f,         0.0f,
    2.38486f,     0.164689f,    0.0f,         0.0f,         2.38595f,
    0.157435f,    0.0f,         0.0f,         2.38694f,     0.150479f,
    0.0f,         0.0f,         2.38786f,     0.143811f,    0.0f,
    0.0f,         2.38869f,     0.137421f,    0.0f,         0.0f,
    2.38946f,     0.131299f,    0.0f,         0.0f,         2.39016f,
    0.125435f,    0.0f,         0.0f,         2.3908f,      0.11982f,
    0.0f,         0.0f,         2.39139f,     0.114445f,    0.0f,
    0.0f,         2.39192f,     0.1093f,      0.0f,         0.0f,
    2.39241f,     0.104376f,    0.0f,         0.0f,         2.39286f,
    0.099666f,    0.0f,         0.0f,         2.39327f,     0.0951603f,
    0.0f,         0.0f,         2.39364f,     0.0908511f,   0.0f,
    0.0f,         2.39398f,     0.0867305f,   0.0f,         0.0f,
    2.39429f,     0.082791f,    0.0f,         0.0f,         2.39457f,
    0.0790251f,   0.0f,         0.0f,         2.39483f,     0.0754256f,
    0.0f,         0.0f,         2.39506f,     0.0719858f,   0.0f,
    0.0f,         2.39528f,     0.0686988f,   0.0f,         0.0f,
    2.39547f,     0.0655584f,   0.0f,         0.0f,         2.39565f,
    0.0625583f,   0.0f,         0.0f,         2.39582f,     0.0596925f,
    0.0f,         0.0f,         2.39596f,     0.0569554f,   0.0f,
    0.0f,         2.3961f,      0.0543413f,   0.0f,         0.0f,
    2.39622f,     0.0518451f,   0.0f,         0.0f,         2.39633f,
    0.0494615f,   0.0f,         0.0f,         2.39644f,     0.0471857f,
    0.0f,         0.0f,         2.39653f,     0.0450131f,   0.0f,
    0.0f,         2.39661f,     0.0429389f,   0.0f,         0.0f,
    2.39669f,     0.0409591f,   0.0f,         0.0f,         2.39676f,
    0.0390693f,   0.0f,         0.0f,         2.39682f,     0.0372656f,
    0.0f,         0.0f,         2.39688f,     0.0355441f,   0.0f,
    0.0f,         2.39694f,     0.0339013f,   0.0f,         0.0f,
    2.39698f,     0.0323336f,   0.0f,         0.0f,         2.39703f,
    0.0308377f,   0.0f,         0.0f,         2.39707f,     0.0294103f,
    0.0f,         0.0f,         2.3971f,      0.0280484f,   0.0f,
    0.0f,         2.39714f,     0.0267489f,   0.0f,         0.0f,
    2.39717f,     0.0255092f,   0.0f,         0.0f,         2.39719f,
    0.0243265f,   0.0f,         0.0f,         2.39722f,     0.0231982f,
    0.0f,         0.0f,         2.39724f,     0.0221218f,   0.0f,
    0.0f,         2.39726f,     0.0210951f,   0.0f,         0.0f,
    2.39728f,     0.0201157f,   0.0f,         0.0f,         2.3973f,
    0.0191815f,   0.0f,         0.0f,         2.39731f,     0.0182904f,
    0.0f,         0.0f,         2.39733f,     0.0174405f,   0.0f,
    0.0f,         2.39734f,     0.0166299f,   0.0f,         0.0f,
    2.39735f,     0.0158567f,   0.0f,         0.0f,         2.39736f,
    0.0151194f,   0.0f,         0.0f,         2.39737f,     0.0144161f,
    0.0f,         0.0f,         2.39738f,     0.0137455f,   0.0f,
    0.0f,         2.39739f,     0.0131059f,   0.0f,         0.0f,
    2.3974f,      0.0124959f,   0.0f,         0.0f,         2.3974f,
    0.0119143f,   0.0f,         0.0f,         2.39741f,     0.0113596f,
    0.0f,         0.0f,         2.39741f,     0.0108306f,   0.0f,
    0.0f,         2.39742f,     0.0103262f,   0.0f,         0.0f,
    2.39742f,     0.00984523f,  0.0f,         0.0f,         2.39743f,
    0.00938658f,  0.0f,         0.0f,         2.39743f,     0.00894924f,
    0.0f,         0.0f,         2.39744f,     0.00853223f,  0.0f,
    0.0f,         2.39744f,     0.00813459f,  0.0f,         0.0f,
    2.39744f,     0.00775545f,  0.0f,         0.0f,         2.39744f,
    0.00739393f,  0.0f,         0.0f,         2.39745f,     0.00704923f,
    0.0f,         0.0f,         2.39745f,     0.00672056f,  0.0f,
    0.0f,         2.39745f,     0.00640719f,  0.0f,         0.0f,
    2.39745f,     0.00610841f,  0.0f,         0.0f,         2.39745f,
    0.00582353f,  0.0f,         0.0f,         2.39745f,     0.00555191f,
    0.0f,         0.0f,         2.39746f,     0.00529295f,  0.0f,
    0.0f,         2.39746f,     0.00504604f,  0.0f,         0.0f,
    2.39746f,     0.00481063f,  0.0f,         0.0f,         2.39746f,
    0.00458619f,  0.0f,         0.0f,         2.39746f,     0.00437221f,
    0.0f,         0.0f,         2.39746f,     0.0041682f,   0.0f,
    0.0f,         2.39746f,     0.0039737f,   0.0f,         0.0f,
    2.39746f,     0.00378826f,  0.0f,         0.0f,         2.39746f,
    0.00361147f,  0.0f,         0.0f,         2.39746f,     0.00344291f,
    0.0f,         0.0f,         2.39746f,     0.00328222f,  0.0f,
    0.0f,         2.39746f,     0.00312902f,  0.0f,         0.0f,
    2.39746f,     0.00298297f,  0.0f,         0.0f,         2.39747f,
    0.00284372f,  0.0f,         0.0f,         2.39747f,     0.00271097f,
    0.0f,         0.0f,         2.39747f,     0.00258441f,  0.0f,
    0.0f,         2.39747f,     0.00246376f,  0.0f,         0.0f,
    2.39747f,     0.00234873f,  0.0f,         0.0f,         2.39747f,
    0.00223908f,  0.0f,         0.0f,         2.39747f,     0.00213453f,
    0.0f,         0.0f,         2.39747f,     0.00203487f,  0.0f,
    0.0f,         2.39747f,     0.00193986f,  0.0f,         0.0f,
    2.39747f,     0.00184928f,  0.0f,         0.0f,         2.39747f,
    0.00176292f,  0.0f,         0.0f,         2.39747f,     0.0016806f,
    0.0f,         0.0f,         2.39747f,     0.00160212f,  0.0f,
    0.0f,         2.39747f,     0.00152731f,  0.0f,         0.0f,
    2.39747f,     0.00145598f,  0.0f,         0.0f,         2.39747f,
    0.00138799f,  0.0f,         0.0f,         2.39747f,     0.00132316f,
    0.0f,         0.0f,         2.39747f,     0.00126137f,  0.0f,
    0.0f,         2.39747f,     0.00120246f,  0.0f,         0.0f,
    2.39747f,     0.0011463f,   0.0f,         0.0f,         2.39747f,
    0.00109276f,  0.0f,         0.0f,         2.39747f,     0.00104172f,
    0.0f,         0.0f,         2.39747f,     0.000993069f, 0.0f,
    0.0f,         2.39747f,     0.000946686f, 0.0f,         0.0f,
    2.39747f,     0.000902469f, 0.0f,         0.0f,         2.39747f,
    0.000860316f, 0.0f,         0.0f,         2.39747f,     0.000820132f,
    0.0f,         0.0f,         2.39747f,     0.000781825f, 0.0f,
    0.0f,         2.39747f,     0.000745306f, 0.0f,         0.0f,
    2.39747f,     0.000710492f, 0.0f,         0.0f,         2.39747f,
    0.000677305f, 0.0f,         0.0f,         2.39747f,     0.000645667f,
    0.0f,         0.0f,         2.39747f,     0.000615507f, 0.0f,
    0.0f,         2.39747f,     0.000586756f, 0.0f,         0.0f,
    2.39747f,     0.000559347f, 0.0f,         0.0f,         2.39747f,
    0.000533218f, 0.0f,         0.0f,         2.39747f,     0.00050831f,
    0.0f,         0.0f,         2.39747f,     0.000484565f, 0.0f,
    0.0f,         2.39747f,     0.000461929f, 0.0f,         0.0f,
    2.39747f,     0.000440351f, 0.0f,         0.0f,         2.39747f,
    0.00041978f,  0.0f,         0.0f,         2.39747f,     0.00040017f,
    0.0f,         0.0f,         2.39747f,     0.000381476f, 0.0f,
    0.0f,         2.39747f,     0.000363656f, 0.0f,         0.0f,
    2.39747f,     0.000346667f, 0.0f,         0.0f,         2.39747f,
    0.000330473f, 0.0f,         0.0f,         2.39747f,     0.000315034f,
    0.0f,         0.0f,         2.39747f,     0.000300317f, 0.0f,
    0.0f,         2.39747f,     0.000286287f, 0.0f,         0.0f,
    2.39747f,     0.000272913f, 0.0f,         0.0f,         2.39747f,
    0.000260164f, 0.0f,         0.0f,         2.39747f,     0.00024801f,
    0.0f,         0.0f,         2.39747f,     0.000236423f, 0.0f,
    0.0f,         2.39747f,     0.000225378f, 0.0f,         0.0f,
    2.39747f,     0.000214849f, 0.0f,         0.0f,         2.39747f,
    0.000204812f, 0.0f,         0.0f,         2.39747f,     0.000195244f,
    0.0f,         0.0f,         2.39747f,     0.000186122f, 0.0f,
    0.0f,         2.39747f,     0.000177427f, 0.0f,         0.0f,
    2.39747f,     0.000169138f, 0.0f,         0.0f,         2.39747f,
    0.000161236f, 0.0f,         0.0f,         2.39747f,     0.000153704f,
    0.0f,         0.0f,         2.39747f,     0.000146523f, 0.0f,
    0.0f,         2.39747f,     0.000139678f, 0.0f,         0.0f,
    2.39747f,     0.000133152f, 0.0f,         0.0f,         2.39747f,
    0.000126932f, 0.0f,         0.0f,         2.39747f,     0.000121001f,
    0.0f,         0.0f,         2.39747f,     0.000115348f, 0.0f,
    0.0f,         2.39747f,     0.00010996f,  0.0f,         0.0f,
    2.39747f,     0.000104822f, 0.0f,         0.0f,         2.39747f,
    9.99252e-05f, 0.0f,         0.0f,         2.39747f,     9.52568e-05f,
    0.0f,         0.0f,         2.39747f,     9.08065e-05f, 0.0f,
    0.0f,         2.39747f,     8.65641e-05f, 0.0f,         0.0f,
    2.39747f,     8.25199e-05f, 0.0f,         0.0f,         2.39747f,
    7.86646e-05f, 0.0f,         0.0f,         2.39747f,     7.49895e-05f,
    0.0f,         0.0f,         2.39747f,     7.1486e-05f,  0.0f,
    0.0f,         2.39747f,     6.81463e-05f, 0.0f,         0.0f,
    2.39747f,     6.49625e-05f, 0.0f,         0.0f,         2.39747f,
    6.19275e-05f, 0.0f,         0.0f,         2.39747f,     5.90343e-05f,
    0.0f,         0.0f,         2.39747f,     5.62762e-05f, 0.0f,
    0.0f,         2.39747f,     5.3647e-05f,  0.0f,         0.0f,
    2.39747f,     5.11407e-05f, 0.0f,         0.0f,         2.39747f,
    4.87514e-05f, 0.0f,         0.0f,         2.39747f,     4.64738e-05f,
    0.0f,         0.0f,         2.39747f,     4.43025e-05f, 0.0f,
    0.0f,         2.39747f,     4.22327e-05f, 0.0f,         0.0f,
    2.39747f,     4.02596e-05f, 0.0f,         0.0f,         2.39747f,
    3.83787e-05f, 0.0f,         0.0f,         2.39747f,     3.65857e-05f,
    0.0f,         0.0f,         2.39747f,     3.48764e-05f, 0.0f,
    0.0f,         2.39747f,     3.3247e-05f,  0.0f,         0.0f,
    2.39747f,     3.16937e-05f, 0.0f,         0.0f,         2.39747f,
    3.0213e-05f,  0.0f,         0.0f,         2.39747f,     2.88014e-05f,
    0.0f,         0.0f,         2.39747f,     2.74558e-05f, 0.0f,
    0.0f,         2.39747f,     2.61731e-05f, 0.0f,         0.0f,
    2.39747f,     2.49503e-05f, 0.0f,         0.0f,         2.39747f,
    2.37846e-05f, 0.0f,         0.0f,         2.39747f,     2.26734e-05f,
    0.0f,         0.0f,         2.39747f,     2.16141e-05f, 0.0f,
    0.0f,         2.39747f,     2.06043e-05f, 0.0f,         0.0f,
    2.39747f,     1.96417e-05f, 0.0f,         0.0f,         2.39747f,
    1.8724e-05f,  0.0f,         0.0f,         2.39747f,     1.78492e-05f,
    0.0f,         0.0f,         2.39747f,     1.70153e-05f, 0.0f,
    0.0f,         2.39747f,     1.62203e-05f, 0.0f,         0.0f,
    2.39747f,     1.54625e-05f, 0.0f,         0.0f,         2.39747f,
    1.47401e-05f, 0.0f,         0.0f,         2.39747f,     1.40515e-05f,
    0.0f,         0.0f,         2.39747f,     1.3395e-05f,  0.0f,
    0.0f,         2.39747f,     1.27692e-05f, 0.0f,         0.0f,
    2.39747f,     1.21726e-05f, 0.0f,         0.0f,         2.39747f,
    1.16039e-05f, 0.0f,         0.0f,         2.39747f,     1.10617e-05f,
    0.0f,         0.0f,         2.39747f,     1.05449e-05f, 0.0f,
    0.0f,         2.39747f,     1.00523e-05f, 0.0f,         0.0f,
    2.39747f,     9.58263e-06f, 0.0f,         0.0f,         2.39747f,
    9.13493e-06f, 0.0f,         0.0f,         2.39747f,     8.70814e-06f,
    0.0f,         0.0f,         2.39747f,     8.3013e-06f,  0.0f,
    0.0f,         2.39747f,     7.91346e-06f, 0.0f,         0.0f,
    2.39747f,     7.54374e-06f, 0.0f,         0.0f,         2.39747f,
    7.1913e-06f,  0.0f,         0.0f,         2.39747f,     6.85532e-06f,
    0.0f,         0.0f,         2.39747f,     6.53504e-06f, 0.0f,
    0.0f,         2.39747f,     6.22972e-06f, 0.0f,         0.0f,
    2.39747f,     5.93867e-06f, 0.0f,         0.0f,         2.39747f,
    5.66121e-06f, 0.0f,         0.0f,         2.39747f,     5.39672e-06f,
    0.0f,         0.0f,         2.39747f,     5.14458e-06f, 0.0f,
    0.0f,         2.39747f,     4.90423e-06f, 0.0f,         0.0f,
    2.39747f,     4.6751e-06f,  0.0f,         0.0f,         2.39747f,
    4.45668e-06f, 0.0f,         0.0f,         2.39747f,     4.24846e-06f,
    0.0f,         0.0f,         2.39747f,     4.04997e-06f, 0.0f,
    0.0f,         2.39747f,     3.86076e-06f, 0.0f,         0.0f,
    2.39747f,     3.68038e-06f, 0.0f,         0.0f,         2.39747f,
    3.50843e-06f, 0.0f,         0.0f,         2.39747f,     3.34452e-06f,
    0.0f,         0.0f,         2.39747f,     3.18826e-06f, 0.0f,
    0.0f,         2.39747f,     3.03931e-06f, 0.0f,         0.0f,
    2.39747f,     2.89731e-06f, 0.0f,         0.0f,         2.39747f,
    2.76195e-06f, 0.0f,         0.0f,         2.39747f,     2.63291e-06f,
    0.0f,         0.0f,         2.39747f,     2.5099e-06f,  0.0f,
    0.0f,         2.39747f,     2.39263e-06f, 0.0f,         0.0f,
    2.39747f,     2.28085e-06f, 0.0f,         0.0f,         2.39747f,
    2.17429e-06f, 0.0f,         0.0f,         2.39747f,     2.0727e-06f,
    0.0f,         0.0f,         2.39747f,     1.97587e-06f, 0.0f,
    0.0f,         2.39747f,     1.88355e-06f, 0.0f,         0.0f,
    2.39747f,     1.79555e-06f, 0.0f,         0.0f,         2.39747f,
    1.71167e-06f, 0.0f,         0.0f,         2.39747f,     1.6317e-06f,
    0.0f,         0.0f,         2.39747f,     1.55546e-06f, 0.0f,
    0.0f,         2.39747f,     1.48279e-06f, 0.0f,         0.0f,
    2.39747f,     1.41352e-06f, 0.0f,         0.0f,         2.39747f,
    1.34748e-06f, 0.0f,         0.0f};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_AMBISONIC_SPREAD_COEFFICIENTS_H_
