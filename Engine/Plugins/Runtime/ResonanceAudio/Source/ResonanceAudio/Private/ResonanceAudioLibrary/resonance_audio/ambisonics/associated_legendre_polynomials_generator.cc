/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "ambisonics/associated_legendre_polynomials_generator.h"

#include <cmath>

#include "base/logging.h"
#include "base/misc_math.h"

namespace vraudio {

AssociatedLegendrePolynomialsGenerator::AssociatedLegendrePolynomialsGenerator(
    int max_degree, bool condon_shortley_phase, bool compute_negative_order)
    : max_degree_(max_degree),
      condon_shortley_phase_(condon_shortley_phase),
      compute_negative_order_(compute_negative_order) {
  DCHECK_GE(max_degree_, 0);
}

std::vector<float> AssociatedLegendrePolynomialsGenerator::Generate(
    float x) const {
  std::vector<float> values(GetNumValues());

  // Bases for the recurrence relations.
  values[GetIndex(0, 0)] = ComputeValue(0, 0, x, values);
  if (max_degree_ >= 1) values[GetIndex(1, 0)] = ComputeValue(1, 0, x, values);

  // Using recurrence relations, we now compute the rest of the values needed.
  // (degree, 0), based on (degree - 1, 0) and (degree - 2, 0):
  for (int degree = 2; degree <= max_degree_; ++degree) {
    const int order = 0;
    values[GetIndex(degree, order)] = ComputeValue(degree, order, x, values);
  }
  // (degree, degree):
  for (int degree = 1; degree <= max_degree_; ++degree) {
    const int order = degree;
    values[GetIndex(degree, order)] = ComputeValue(degree, order, x, values);
  }
  // (degree, degree - 1):
  for (int degree = 2; degree <= max_degree_; ++degree) {
    const int order = degree - 1;
    values[GetIndex(degree, order)] = ComputeValue(degree, order, x, values);
  }
  // The remaining positive orders, based on (degree - 1, order) and
  // (degree - 2, order):
  for (int degree = 3; degree <= max_degree_; ++degree) {
    for (int order = 1; order <= degree - 2; ++order) {
      values[GetIndex(degree, order)] = ComputeValue(degree, order, x, values);
    }
  }
  // (degree, -order):
  if (compute_negative_order_) {
    for (int degree = 1; degree <= max_degree_; ++degree) {
      for (int order = 1; order <= degree; ++order) {
        values[GetIndex(degree, -order)] =
            ComputeValue(degree, -order, x, values);
      }
    }
  }
  if (!condon_shortley_phase_) {
    for (int degree = 1; degree <= max_degree_; ++degree) {
      const int start_order = compute_negative_order_ ? -degree : 0;
      for (int order = start_order; order <= degree; ++order) {
        // Undo the Condon-Shortley phase.
        values[GetIndex(degree, order)] *=
            static_cast<float>(std::pow(-1, order));
      }
    }
  }
  return values;
}

size_t AssociatedLegendrePolynomialsGenerator::GetNumValues() const {
  if (compute_negative_order_)
    return (max_degree_ + 1) * (max_degree_ + 1);
  else
    return ((max_degree_ + 1) * (max_degree_ + 2)) / 2;
}

size_t AssociatedLegendrePolynomialsGenerator::GetIndex(int degree,
                                                        int order) const {
  CheckIndexValidity(degree, order);
  size_t result;
  const size_t sdegree = static_cast<size_t>(degree);
  const size_t sorder = static_cast<size_t>(order);
  if (compute_negative_order_) {
    result = sdegree * (sdegree + 1) + sorder;
  } else {
    result = (sdegree * (sdegree + 1)) / 2 + sorder;
  }
  DCHECK_GE(result, 0U);
  DCHECK_LT(result, GetNumValues());
  return result;
}

float AssociatedLegendrePolynomialsGenerator::ComputeValue(
    int degree, int order, float x, const std::vector<float>& values) const {
  CheckIndexValidity(degree, order);
  if (degree == 0 && order == 0) {
    return 1;
  } else if (degree == 1 && order == 0) {
    return x;
  } else if (degree == order) {
    return std::pow(-1.0f, static_cast<float>(degree)) *
           DoubleFactorial(2 * degree - 1) *
           std::pow((1.0f - x * x), 0.5f * static_cast<float>(degree));
  } else if (order == degree - 1) {
    return x * static_cast<float>(2 * degree - 1) *
           values[GetIndex(degree - 1, degree - 1)];
  } else if (order < 0) {
    return std::pow(-1.0f, static_cast<float>(order)) *
           Factorial(degree + order) / Factorial(degree - order) *
           values[GetIndex(degree, -order)];
  } else {
    return (static_cast<float>(2 * degree - 1) * x *
                values[GetIndex(degree - 1, order)] -
            static_cast<float>(degree - 1 + order) *
                values[GetIndex(degree - 2, order)]) /
           static_cast<float>(degree - order);
  }
}

void AssociatedLegendrePolynomialsGenerator::CheckIndexValidity(
    int degree, int order) const {
  DCHECK_GE(degree, 0);
  DCHECK_LE(degree, max_degree_);
  if (compute_negative_order_) {
    DCHECK_LE(-degree, order);
  } else {
    DCHECK_GE(order, 0);
  }
  DCHECK_LE(order, degree);
}

}  // namespace vraudio
