/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_ASSOCIATED_LEGENDRE_POLYNOMIALS_GENERATOR_H_
#define RESONANCE_AUDIO_AMBISONICS_ASSOCIATED_LEGENDRE_POLYNOMIALS_GENERATOR_H_

#include <stddef.h>
#include <vector>

namespace vraudio {

// Generates associated Legendre polynomials.
class AssociatedLegendrePolynomialsGenerator {
 public:
  // Constructs a generator for associated Legendre polynomials (ALP).
  //
  // @param max_degree The maximum ALP degree supported by this generator.
  // @param condon_shortley_phase Whether the Condon-Shortley phase, (-1)^order,
  //     should be included in the polynomials generated.
  // @param compute_negative_order Whether this generator should compute
  //     negative-ordered polynomials.
  AssociatedLegendrePolynomialsGenerator(int max_degree,
                                         bool condon_shortley_phase,
                                         bool compute_negative_order);

  // Generates the associated Legendre polynomials for the given |x|, returning
  // the computed sequence.
  //
  // @param x The abscissa (the polynomials' variable).
  // @return Output vector of computed values.
  std::vector<float> Generate(float x) const;

  // Gets the number of associated Legendre polynomials this generator produces.
  //
  // @return The number of associated Legendre polynomials this generator
  //     produces.
  size_t GetNumValues() const;

  // Gets the index into the output vector for the given |degree| and |order|.
  //
  // @param degree The polynomial's degree.
  // @param order The polynomial's order.
  // @return The index into the vector of computed values corresponding to the
  //     specified ALP.
  size_t GetIndex(int degree, int order) const;

 private:
  // Computes the ALP for (degree, order) the given |x| using recurrence
  // relations. It is assumed that the ALPs necessary for each computation are
  // already computed and stored in |values|.
  //
  // @param degree The degree of the polynomial being computed.
  // @param degree The order of the polynomial being computed.
  // @param values The previously computed values.
  // @return The computed polynomial.
  inline float ComputeValue(int degree, int order, float x,
                            const std::vector<float>& values) const;

  // Checks the validity of the given index.
  //
  // @param degree The polynomial's degree.
  // @param order The polynomial's order.
  inline void CheckIndexValidity(int degree, int order) const;

  // The maximum polynomial degree that can be computed; must be >= 0.
  int max_degree_ = 0;
  // Whether the Condon-Shortley phase, (-1)^order, should be included in the
  // polynomials generated.
  bool condon_shortley_phase_ = false;
  // Whether this generator should compute negative-ordered polynomials.
  bool compute_negative_order_ = false;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_ASSOCIATED_LEGENDRE_POLYNOMIALS_GENERATOR_H_
