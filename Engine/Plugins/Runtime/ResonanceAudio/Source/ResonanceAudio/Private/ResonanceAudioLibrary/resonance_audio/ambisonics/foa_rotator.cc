/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "ambisonics/foa_rotator.h"

#include <algorithm>

#include "base/constants_and_types.h"
#include "base/misc_math.h"

namespace vraudio {

bool FoaRotator::Process(const WorldRotation& target_rotation,
                         const AudioBuffer& input, AudioBuffer* output) {

  DCHECK(output);
  DCHECK_EQ(input.num_channels(), kNumFirstOrderAmbisonicChannels);
  DCHECK_EQ(input.num_channels(), output->num_channels());
  DCHECK_EQ(input.num_frames(), output->num_frames());

  static const WorldRotation kIdentityRotation;

  if (current_rotation_.AngularDifferenceRad(kIdentityRotation) <
          kRotationQuantizationRad &&
      target_rotation.AngularDifferenceRad(kIdentityRotation) <
          kRotationQuantizationRad) {
    return false;
  }

  if (current_rotation_.AngularDifferenceRad(target_rotation) <
      kRotationQuantizationRad) {
    // Rotate the whole input buffer frame by frame.
    Rotate(current_rotation_, 0, input.num_frames(), input, output);
    return true;
  }

  // In order to perform a smooth rotation, we divide the buffer into
  // chunks of size |kSlerpFrameInterval|.
  //

  const size_t kSlerpFrameInterval = 32;

  WorldRotation slerped_rotation;
  // Rotate the input buffer at every slerp update interval. Truncate the
  // final chunk if the input buffer is not an integer multiple of the
  // chunk size.
  for (size_t i = 0; i < input.num_frames(); i += kSlerpFrameInterval) {
    const size_t duration =
        std::min(input.num_frames() - i, kSlerpFrameInterval);
    const float interpolation_factor = static_cast<float>(i + duration) /
                                       static_cast<float>(input.num_frames());
    slerped_rotation =
        current_rotation_.slerp(interpolation_factor, target_rotation);
    // Rotate the input buffer frame by frame within the current chunk.
    Rotate(slerped_rotation, i, duration, input, output);
  }

  current_rotation_ = target_rotation;

  return true;
}

void FoaRotator::Rotate(const WorldRotation& target_rotation,
                        size_t start_location, size_t duration,
                        const AudioBuffer& input, AudioBuffer* output) {

  const AudioBuffer::Channel& input_channel_audio_space_w = input[0];
  const AudioBuffer::Channel& input_channel_audio_space_y = input[1];
  const AudioBuffer::Channel& input_channel_audio_space_z = input[2];
  const AudioBuffer::Channel& input_channel_audio_space_x = input[3];
  AudioBuffer::Channel* output_channel_audio_space_w = &(*output)[0];
  AudioBuffer::Channel* output_channel_audio_space_y = &(*output)[1];
  AudioBuffer::Channel* output_channel_audio_space_z = &(*output)[2];
  AudioBuffer::Channel* output_channel_audio_space_x = &(*output)[3];

  for (size_t frame = start_location; frame < start_location + duration;
       ++frame) {
    // Convert the current audio frame into world space position.
    temp_audio_position_(0) = input_channel_audio_space_x[frame];
    temp_audio_position_(1) = input_channel_audio_space_y[frame];
    temp_audio_position_(2) = input_channel_audio_space_z[frame];
    ConvertWorldFromAudioPosition(temp_audio_position_, &temp_world_position_);
    // Apply rotation to |world_position| and return to audio space.
    temp_rotated_world_position_ = target_rotation * temp_world_position_;

    ConvertAudioFromWorldPosition(temp_rotated_world_position_,
                                  &temp_rotated_audio_position_);
    (*output_channel_audio_space_x)[frame] =
        temp_rotated_audio_position_(0);  // X
    (*output_channel_audio_space_y)[frame] =
        temp_rotated_audio_position_(1);  // Y
    (*output_channel_audio_space_z)[frame] =
        temp_rotated_audio_position_(2);  // Z
  }
  // Copy W channel.
  std::copy_n(&input_channel_audio_space_w[start_location], duration,
              &(*output_channel_audio_space_w)[start_location]);
}

}  // namespace vraudio
