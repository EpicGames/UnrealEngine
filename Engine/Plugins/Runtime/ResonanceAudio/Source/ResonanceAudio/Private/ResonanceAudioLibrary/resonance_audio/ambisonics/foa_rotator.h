/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_FOA_ROTATOR_H_
#define RESONANCE_AUDIO_AMBISONICS_FOA_ROTATOR_H_

#include "base/audio_buffer.h"
#include "base/spherical_angle.h"

namespace vraudio {

// Rotator for first order ambisonic soundfields. It supports ACN channel
// ordering and SN3D normalization (AmbiX).
class FoaRotator {
 public:
  // @param target_rotation Target rotation to be applied to the input buffer.
  // @param input First order soundfield input buffer to be rotated.
  // @param output Pointer to output buffer.
  // @return True if rotation has been applied.
  bool Process(const WorldRotation& target_rotation, const AudioBuffer& input,
               AudioBuffer* output);

 private:
  // Method which rotates a specified chunk of data in the AudioBuffer.
  //
  // @param target_rotation Target rotation to be applied to the soundfield.
  // @param start_location Sample index in the soundfield where the rotation
  //     should begin.
  // @param duration Number of samples in soundfield to be rotated.
  // @param input First order soundfield input buffer to be rotated.
  // @param output Pointer to output buffer.
  void Rotate(const WorldRotation& target_rotation, size_t start_location,
              size_t duration, const AudioBuffer& input, AudioBuffer* output);

  // Current rotation which is used in the interpolation process in order to
  // perform a smooth rotation. Initialized with an identity matrix.
  WorldRotation current_rotation_;

  // Preallocation of temporary variables used during rotation.
  AudioPosition temp_audio_position_;
  WorldPosition temp_world_position_;
  AudioPosition temp_rotated_audio_position_;
  WorldPosition temp_rotated_world_position_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_FOA_ROTATOR_H_
