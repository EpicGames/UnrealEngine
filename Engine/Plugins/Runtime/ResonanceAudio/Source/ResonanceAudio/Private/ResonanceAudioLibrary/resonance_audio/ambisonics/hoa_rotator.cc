/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "ambisonics/hoa_rotator.h"

#include <algorithm>
#include <cmath>

#include "ambisonics/utils.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"

namespace vraudio {

namespace {

// Below are the helper methods to compute SH rotation using recursion. The code
// is branched / modified from:

// maths described in the following papers:
//
// [1]  R. Green, "Spherical Harmonic Lighting: The Gritty Details", GDC 2003,
//      http://www.research.scea.com/gdc2003/spherical-harmonic-lighting.pdf
// [2]  J. Ivanic and K. Ruedenberg, "Rotation Matrices for Real Spherical
//      Harmonics. Direct Determination by Recursion", J. Phys. Chem., vol. 100,
//      no. 15, pp. 6342-6347, 1996.
//      http://pubs.acs.org/doi/pdf/10.1021/jp953350u
// [2b] Corrections to initial publication:
//      http://pubs.acs.org/doi/pdf/10.1021/jp9833350

// Kronecker Delta function.
inline float KroneckerDelta(int i, int j) { return (i == j) ? 1.0f : 0.0f; }

// [2] uses an odd convention of referring to the rows and columns using
// centered indices, so the middle row and column are (0, 0) and the upper
// left would have negative coordinates.
//
// This is a convenience function to allow us to access an Eigen::MatrixXf
// in the same manner, assuming r is a (2l+1)x(2l+1) matrix.
float GetCenteredElement(const Eigen::MatrixXf& r, int i, int j) {
  // The shift to go from [-l, l] to [0, 2l] is (rows - 1) / 2 = l,
  // (since the matrix is assumed to be square, rows == cols).
  const int offset = (static_cast<int>(r.rows()) - 1) / 2;
  return r(i + offset, j + offset);
}

// Helper function defined in [2] that is used by the functions U, V, W.
// This should not be called on its own, as U, V, and W (and their coefficients)
// select the appropriate matrix elements to access arguments |a| and |b|.
float P(int i, int a, int b, int l, const std::vector<Eigen::MatrixXf>& r) {
  if (b == l) {
    return GetCenteredElement(r[1], i, 1) *
               GetCenteredElement(r[l - 1], a, l - 1) -
           GetCenteredElement(r[1], i, -1) *
               GetCenteredElement(r[l - 1], a, -l + 1);
  } else if (b == -l) {
    return GetCenteredElement(r[1], i, 1) *
               GetCenteredElement(r[l - 1], a, -l + 1) +
           GetCenteredElement(r[1], i, -1) *
               GetCenteredElement(r[l - 1], a, l - 1);
  } else {
    return GetCenteredElement(r[1], i, 0) * GetCenteredElement(r[l - 1], a, b);
  }
}

// The functions U, V, and W should only be called if the correspondingly
// named coefficient u, v, w from the function ComputeUVWCoeff() is non-zero.
// When the coefficient is 0, these would attempt to access matrix elements that
// are out of bounds. The vector of rotations, |r|, must have the |l - 1|
// previously completed band rotations. These functions are valid for |l >= 2|.

float U(int m, int n, int l, const std::vector<Eigen::MatrixXf>& r) {
  // Although [1, 2] split U into three cases for m == 0, m < 0, m > 0
  // the actual values are the same for all three cases.
  return P(0, m, n, l, r);
}

float V(int m, int n, int l, const std::vector<Eigen::MatrixXf>& r) {
  if (m == 0) {
    return P(1, 1, n, l, r) + P(-1, -1, n, l, r);
  } else if (m > 0) {
    const float d = KroneckerDelta(m, 1);
    return P(1, m - 1, n, l, r) * std::sqrt(1 + d) -
           P(-1, -m + 1, n, l, r) * (1 - d);
  } else {
    // Note there is apparent errata in [1,2,2b] dealing with this particular
    // case. [2b] writes it should be P*(1-d)+P*(1-d)^0.5
    // [1] writes it as P*(1+d)+P*(1-d)^0.5, but going through the math by hand,
    // you must have it as P*(1-d)+P*(1+d)^0.5 to form a 2^.5 term, which
    // parallels the case where m > 0.
    const float d = KroneckerDelta(m, -1);
    return P(1, m + 1, n, l, r) * (1 - d) +
           P(-1, -m - 1, n, l, r) * std::sqrt(1 + d);
  }
}

float W(int m, int n, int l, const std::vector<Eigen::MatrixXf>& r) {
  if (m == 0) {
    // Whenever this happens, w is also 0 so W can be anything.
    return 0.0f;
  } else if (m > 0) {
    return P(1, m + 1, n, l, r) + P(-1, -m - 1, n, l, r);
  } else {
    return P(1, m - 1, n, l, r) - P(-1, -m + 1, n, l, r);
  }
}

// Calculates the coefficients applied to the U, V, and W functions. Because
// their equations share many common terms they are computed simultaneously.
void ComputeUVWCoeff(int m, int n, int l, float* u, float* v, float* w) {
  const float d = KroneckerDelta(m, 0);
  const float denom = (abs(n) == l ? static_cast<float>(2 * l * (2 * l - 1))
                                   : static_cast<float>((l + n) * (l - n)));
  const float one_over_denom = 1.0f / denom;

  *u = std::sqrt(static_cast<float>((l + m) * (l - m)) * one_over_denom);
  *v = 0.5f *
       std::sqrt((1.0f + d) * static_cast<float>(l + abs(m) - 1) *
                 (static_cast<float>(l + abs(m))) * one_over_denom) *
       (1.0f - 2.0f * d);
  *w = -0.5f *
       std::sqrt(static_cast<float>(l - abs(m) - 1) *
                 (static_cast<float>(l - abs(m))) * one_over_denom) *
       (1.0f - d);
}

// Calculates the (2l+1)x(2l+1) rotation matrix for the band l.
// This uses the matrices computed for band 1 and band l-1 to compute the
// matrix for band l. |rotations| must contain the previously computed l-1
// rotation matrices.
//
// This implementation comes from p. 5 (6346), Table 1 and 2 in [2] taking
// into account the corrections from [2b].
void ComputeBandRotation(int l, std::vector<Eigen::MatrixXf>* rotations) {
  // The lth band rotation matrix has rows and columns equal to the number of
  // coefficients within that band (-l <= m <= l implies 2l + 1 coefficients).
  Eigen::MatrixXf rotation(2 * l + 1, 2 * l + 1);
  for (int m = -l; m <= l; ++m) {
    for (int n = -l; n <= l; ++n) {
      float u, v, w;
      ComputeUVWCoeff(m, n, l, &u, &v, &w);

      // The functions U, V, W are only safe to call if the coefficients
      // u, v, w are not zero.
      if (std::abs(u) > 0.0f) u *= U(m, n, l, *rotations);
      if (std::abs(v) > 0.0f) v *= V(m, n, l, *rotations);
      if (std::abs(w) > 0.0f) w *= W(m, n, l, *rotations);

      rotation(m + l, n + l) = (u + v + w);
    }
  }
  (*rotations)[l] = rotation;
}

}  // namespace

HoaRotator::HoaRotator(int ambisonic_order)
    : ambisonic_order_(ambisonic_order),
      rotation_matrices_(ambisonic_order_ + 1),
      rotation_matrix_(
          static_cast<int>(GetNumPeriphonicComponents(ambisonic_order)),
          static_cast<int>(GetNumPeriphonicComponents(ambisonic_order))) {
  DCHECK_GE(ambisonic_order_, 2);

  // Initialize rotation sub-matrices.
  // Order 0  matrix (first band) is simply the 1x1 identity.
  Eigen::MatrixXf r(1, 1);
  r(0, 0) = 1.0f;
  rotation_matrices_[0] = r;
  // All the other ambisonic orders (bands) are set to identity matrices of
  // corresponding sizes.
  for (int l = 1; l <= ambisonic_order_; ++l) {
    const size_t submatrix_size = GetNumNthOrderPeriphonicComponents(l);
    r.resize(submatrix_size, submatrix_size);
    rotation_matrices_[l] = r.setIdentity();
  }
  // Initialize the final rotation matrix to an identity matrix.
  rotation_matrix_.setIdentity();
}

bool HoaRotator::Process(const WorldRotation& target_rotation,
                         const AudioBuffer& input, AudioBuffer* output) {

  DCHECK(output);
  DCHECK_EQ(input.num_channels(), GetNumPeriphonicComponents(ambisonic_order_));
  DCHECK_EQ(input.num_channels(), output->num_channels());
  DCHECK_EQ(input.num_frames(), output->num_frames());

  static const WorldRotation kIdentityRotation;

  if (current_rotation_.AngularDifferenceRad(kIdentityRotation) <
          kRotationQuantizationRad &&
      target_rotation.AngularDifferenceRad(kIdentityRotation) <
          kRotationQuantizationRad) {
    return false;
  }

  const size_t channel_stride = input.GetChannelStride();

  typedef Eigen::Matrix<float, Eigen::Dynamic, Eigen::Dynamic, Eigen::RowMajor>
      RowMajorMatrixf;

  const Eigen::Map<const RowMajorMatrixf, Eigen::Aligned, Eigen::OuterStride<>>
      input_matrix(input[0].begin(), static_cast<int>(input.num_channels()),
                   static_cast<int>(input.num_frames()),
                   Eigen::OuterStride<>(static_cast<int>(channel_stride)));

  Eigen::Map<RowMajorMatrixf, Eigen::Aligned, Eigen::OuterStride<>>
      output_matrix((*output)[0].begin(),
                    static_cast<int>(input.num_channels()),
                    static_cast<int>(input.num_frames()),
                    Eigen::OuterStride<>(static_cast<int>(channel_stride)));

  if (current_rotation_.AngularDifferenceRad(target_rotation) <
      kRotationQuantizationRad) {
    output_matrix = rotation_matrix_ * input_matrix;
    return true;
  }

  // In order to perform a smooth rotation, we divide the buffer into
  // chunks of size |kSlerpFrameInterval|.
  //

  const size_t kSlerpFrameInterval = 32;

  WorldRotation slerped_rotation;
  // Rotate the input buffer at every slerp update interval. Truncate the
  // final chunk if the input buffer is not an integer multiple of the
  // chunk size.
  for (size_t i = 0; i < input.num_frames(); i += kSlerpFrameInterval) {
    const size_t duration =
        std::min(input.num_frames() - i, kSlerpFrameInterval);
    const float interpolation_factor = static_cast<float>(i + duration) /
                                       static_cast<float>(input.num_frames());
    UpdateRotationMatrix(
        current_rotation_.slerp(interpolation_factor, target_rotation));
    output_matrix.block(0 /* first channel */, i, output->num_channels(),
                        duration) =
        rotation_matrix_ * input_matrix.block(0 /* first channel */, i,
                                              input.num_channels(), duration);
  }
  current_rotation_ = target_rotation;

  return true;
}

void HoaRotator::UpdateRotationMatrix(const WorldRotation& rotation) {


  // There is no need to update 0th order 1-element sub-matrix.
  // First order sub-matrix can be updated directly from the WorldRotation
  // quaternion. However, we must account for the flipped left-right and
  // front-back axis in the World coordinates.
  AudioRotation rotation_audio_space;
  ConvertAudioFromWorldRotation(rotation, &rotation_audio_space);
  rotation_matrices_[1] = rotation_audio_space.toRotationMatrix();
  rotation_matrix_.block(1, 1, 3, 3) = rotation_matrices_[1];

  // Sub-matrices for the remaining orders are updated recursively using the
  // equations provided in [2, 2b]. An example final rotation matrix composed of
  // sub-matrices of orders 0 to 3 has the following structure:
  //
  // X | 0 0 0 | 0 0 0 0 0 | 0 0 0 0 0 0 0
  // -------------------------------------
  // 0 | X X X | 0 0 0 0 0 | 0 0 0 0 0 0 0
  // 0 | X X X | 0 0 0 0 0 | 0 0 0 0 0 0 0
  // 0 | X X X | 0 0 0 0 0 | 0 0 0 0 0 0 0
  // -------------------------------------
  // 0 | 0 0 0 | X X X X X | 0 0 0 0 0 0 0
  // 0 | 0 0 0 | X X X X X | 0 0 0 0 0 0 0
  // 0 | 0 0 0 | X X X X X | 0 0 0 0 0 0 0
  // 0 | 0 0 0 | X X X X X | 0 0 0 0 0 0 0
  // 0 | 0 0 0 | X X X X X | 0 0 0 0 0 0 0
  // -------------------------------------
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  // 0 | 0 0 0 | 0 0 0 0 0 | X X X X X X X
  //
  for (int current_order = 2; current_order <= ambisonic_order_;
       ++current_order) {
    ComputeBandRotation(current_order, &rotation_matrices_);
    const int index = current_order * current_order;
    const int size =
        static_cast<int>(GetNumNthOrderPeriphonicComponents(current_order));
    rotation_matrix_.block(index, index, size, size) =
        rotation_matrices_[current_order];
  }
}

}  // namespace vraudio
