/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_HOA_ROTATOR_H_
#define RESONANCE_AUDIO_AMBISONICS_HOA_ROTATOR_H_

#include <vector>


#include "ResonanceEigenHelper.h"
#include "base/audio_buffer.h"
#include "base/misc_math.h"

namespace vraudio {

// Rotator for higher order ambisonic sound fields. It supports ACN channel
// ordering and SN3D normalization (AmbiX).
class HoaRotator {
 public:
  // Constructs a sound field rotator of an arbitrary ambisonic order.
  //
  // @param ambisonic_order Order of ambisonic sound field.
  explicit HoaRotator(int ambisonic_order);

  // Performs a smooth inplace rotation of a sound field buffer from
  // |current_rotation_| to |target_rotation|.
  //
  // @param target_rotation Target rotation to be applied to the input buffer.
  // @param input Higher order sound field input buffer to be rotated.
  // @param output Pointer to output buffer.
  // @return True if rotation has been applied.
  bool Process(const WorldRotation& target_rotation, const AudioBuffer& input,
               AudioBuffer* output);

 private:
  // Updates the rotation matrix with using supplied WorldRotation.
  //
  // @param rotation World rotation.
  void UpdateRotationMatrix(const WorldRotation& rotation);

  // Order of the ambisonic sound field handled by the rotator.
  const int ambisonic_order_;

  // Current rotation which is used in the interpolation process in order to
  // compute new rotation matrix. Initialized with an identity rotation.
  WorldRotation current_rotation_;

  // Spherical harmonics rotation sub-matrices for each order.
  std::vector<Eigen::MatrixXf> rotation_matrices_;

  // Final spherical harmonics rotation matrix.
  Eigen::MatrixXf rotation_matrix_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_HOA_ROTATOR_H_
