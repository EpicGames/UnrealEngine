/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "ambisonics/stereo_from_soundfield_converter.h"

#include "base/constants_and_types.h"
#include "dsp/gain.h"

namespace vraudio {

namespace {

const float kMidSideChannelGain = 0.5f;

}  // namespace

void StereoFromSoundfield(const AudioBuffer& soundfield_input,
                          AudioBuffer* stereo_output) {
  DCHECK(stereo_output);
  DCHECK_EQ(kNumStereoChannels, stereo_output->num_channels());
  DCHECK_EQ(soundfield_input.num_frames(), stereo_output->num_frames());
  DCHECK_GE(soundfield_input.num_channels(), kNumFirstOrderAmbisonicChannels);
  const AudioBuffer::Channel& channel_audio_space_w = soundfield_input[0];
  const AudioBuffer::Channel& channel_audio_space_y = soundfield_input[1];
  AudioBuffer::Channel* left_channel_output = &(*stereo_output)[0];
  AudioBuffer::Channel* right_channel_output = &(*stereo_output)[1];
  // Left = 0.5 * (Mid + Side).
  *left_channel_output = channel_audio_space_w;
  *left_channel_output += channel_audio_space_y;
  ConstantGain(0 /* no offset */, kMidSideChannelGain, *left_channel_output,
               left_channel_output, false /* accumulate_output */);
  // Right = 0.5 * (Mid - Side).
  *right_channel_output = channel_audio_space_w;
  *right_channel_output -= channel_audio_space_y;
  ConstantGain(0 /* no offset */, kMidSideChannelGain, *right_channel_output,
               right_channel_output, false /* accumulate_output */);
}

}  // namespace vraudio
