/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_STEREO_FROM_SOUNDFIELD_CONVERTER_H_
#define RESONANCE_AUDIO_AMBISONICS_STEREO_FROM_SOUNDFIELD_CONVERTER_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Performs Ambisonic to stereo decode by performing Mid-Side (M-S) matrixing.
// Soundfield format assumed is Ambix (ACN/SN3D). Only first order channels
// are used for the stereo decode.
//
// @param soundfield_input Soundfield of arbitrary order.
// @param stereo_output Pointer to stereo output buffer.
void StereoFromSoundfield(const AudioBuffer& soundfield_input,
                          AudioBuffer* stereo_output);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_STEREO_FROM_SOUNDFIELD_CONVERTER_H_
