/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_AMBISONICS_UTILS_H_
#define RESONANCE_AUDIO_AMBISONICS_UTILS_H_

#include <cmath>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"

namespace vraudio {

// Returns ACN channel sequence from a degree and order of a spherical harmonic.
inline int AcnSequence(int degree, int order) {
  DCHECK_GE(degree, 0);
  DCHECK_LE(-degree, order);
  DCHECK_LE(order, degree);

  return degree * degree + degree + order;
}

// Returns normalization factor for Schmidt semi-normalized spherical harmonics
// used in AmbiX.
inline float Sn3dNormalization(int degree, int order) {
  DCHECK_GE(degree, 0);
  DCHECK_LE(-degree, order);
  DCHECK_LE(order, degree);
  return std::sqrt((2.0f - ((order == 0) ? 1.0f : 0.0f)) *
                   Factorial(degree - std::abs(order)) /
                   Factorial(degree + std::abs(order)));
}

// Returns the number of spherical harmonics for a periphonic ambisonic sound
// field of |ambisonic_order| at compile-time.
// We have to use template metaprogramming because MSVC12 doesn't support
// constexpr.
template <size_t AmbisonicOrder>
struct GetNumPeriphonicComponentsStatic {
  enum { value = (AmbisonicOrder + 1) * (AmbisonicOrder + 1) };
};

// Returns the number of spherical harmonics for a periphonic ambisonic sound
// field of |ambisonic_order|.
inline size_t GetNumPeriphonicComponents(int ambisonic_order) {
  const size_t sambisonic_order = static_cast<size_t>(ambisonic_order);
  return (sambisonic_order + 1) * (sambisonic_order + 1);
}

// Returns the number of periphonic spherical harmonics (SHs) for a particular
// Ambisonic order. E.g. number of 1st, 2nd or 3rd degree SHs in a 3rd order
// sound field.
inline size_t GetNumNthOrderPeriphonicComponents(int ambisonic_order) {
  if (ambisonic_order == 0) return 1;
  return static_cast<size_t>(GetNumPeriphonicComponents(ambisonic_order) -
                             GetNumPeriphonicComponents(ambisonic_order - 1));
}

// Calculates the ambisonic order of a periphonic sound field with the given
// number of spherical harmonics.
inline int GetPeriphonicAmbisonicOrder(size_t num_components) {
  DCHECK_GT(num_components, 0);
  const int ambisonic_order = static_cast<int>(std::sqrt(num_components)) - 1;
  // Detect when num_components is not square.
  DCHECK_EQ((ambisonic_order + 1) * (ambisonic_order + 1),
            static_cast<int>(num_components));
  return ambisonic_order;
}

// Calculates the order of the current spherical harmonic channel as the integer
// part of a square root of the channel number. Please note, that in Ambisonics
// the terms 'order' (usually denoted as 'n') and 'degree' (usually denoted as
// 'm') are used in the opposite meaning as in more traditional maths or physics
// conventions:
// [1] C. Nachbar, F. Zotter, E. Deleflie, A. Sontacchi, "AMBIX - A SUGGESTED
//     AMBISONICS FORMAT", Proc. of the 2nd Ambisonics Symposium, June 2-3 2011,
//     Lexington, KY, https://goo.gl/jzt4Yy.
inline int GetPeriphonicAmbisonicOrderForChannel(size_t channel) {
  return static_cast<int>(sqrtf(static_cast<float>(channel)));
}

// Calculates the degree of the current spherical harmonic channel. Please note,
// that in Ambisonics the terms 'order' (usually denoted as 'n') and 'degree'
// (usually denoted as 'm') are used in the opposite meaning as in more
// traditional maths or physics conventions:
// [1] C. Nachbar, F. Zotter, E. Deleflie, A. Sontacchi, "AMBIX - A SUGGESTED
//     AMBISONICS FORMAT", Proc. of the 2nd Ambisonics Symposium, June 2-3 2011,
//     Lexington, KY, https://goo.gl/jzt4Yy.
inline int GetPeriphonicAmbisonicDegreeForChannel(size_t channel) {
  const int order = GetPeriphonicAmbisonicOrderForChannel(channel);
  return static_cast<int>(channel) - order * (order + 1);
}

// Returns whether the given |num_channels| corresponds to a valid ambisonic
// order configuration.
inline bool IsValidAmbisonicOrder(size_t num_channels) {
  if (num_channels == 0) {
    return false;
  }
  // Number of channels must be a square number for valid ambisonic orders.
  const size_t sqrt_num_channels = static_cast<size_t>(std::sqrt(num_channels));
  return num_channels == sqrt_num_channels * sqrt_num_channels;
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_AMBISONICS_UTILS_H_
