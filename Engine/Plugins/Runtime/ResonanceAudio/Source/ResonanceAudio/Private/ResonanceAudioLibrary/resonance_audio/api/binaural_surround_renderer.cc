/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "api/binaural_surround_renderer.h"

#include "base/logging.h"
#include "graph/binaural_surround_renderer_impl.h"

namespace vraudio {

BinauralSurroundRenderer* BinauralSurroundRenderer::Create(
    size_t frames_per_buffer, int sample_rate_hz,
    SurroundFormat surround_format) {
  std::unique_ptr<BinauralSurroundRendererImpl> renderer(
      new BinauralSurroundRendererImpl(frames_per_buffer, sample_rate_hz));
  if (!renderer->Init(surround_format)) {
    return nullptr;
  }
  return renderer.release();
}

}  // namespace vraudio
