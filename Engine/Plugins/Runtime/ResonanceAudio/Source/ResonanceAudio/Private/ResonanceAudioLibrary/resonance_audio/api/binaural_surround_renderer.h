/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_API_BINAURAL_SURROUND_RENDERER_H_
#define RESONANCE_AUDIO_API_BINAURAL_SURROUND_RENDERER_H_

#include <stddef.h>
#include <stdint.h>

// Avoid dependency to base/integral_types.h
typedef int16_t int16;

namespace vraudio {

// Renders virtual surround sound as well as ambisonic soundfields to binaural
// stereo.
class BinauralSurroundRenderer {
 public:

  enum SurroundFormat {
    // Enables to initialize a yet undefined rendering mode.
    kInvalid = 0,

    // Binaurally renders a single virtual speaker at 0 degrees in front.
    kSurroundMono = 1,

    // Binaurally renders virtual stereo speakers at -30 degrees and +30
    // degrees.
    kSurroundStereo = 2,

    // Binaurally renders 5.1 surround sound according to the ITU-R BS.775-3
    // speaker configuration recommendation:
    //   - Left (L) at 30 degrees.
    //   - Right (R) at -30 degrees.
    //   - Center (C) at 0 degrees.
    //   - Low frequency effects (LFE) at front center at 0 degrees.
    //   - Left surround (LS) at 110 degrees.
    //   - Right surround (RS) at -110 degrees.
    //
    // The 5.1 channel input layout must matches AAC: L, R, C, LFE, LS, RS.
    // Note that this differs from the Vorbis/Opus 5.1 channel layout, which
    // is: L, C, R, LS, RS, LFE.
    kSurroundFiveDotOne = 3,

    // Binaurally renders 7.1 surround sound according to the ITU-R BS.775-3
    // speaker configuration recommendation:
    //   - Left (FL) at 30 degrees.
    //   - Right (FR) at -30 degrees.
    //   - Center (C) at 0 degrees.
    //   - Low frequency effects (LFE) at front center at 0 degrees.
    //   - Left surround 1 (LS1) at 90 degrees.
    //   - Right surround 1 (RS1) at -90 degrees.
    //   - Left surround 2 (LS2) at 150 degrees.
    //   - Right surround 2 (LS2) at -150 degrees.
    //
    // The 7.1 channel input layout must matches AAC: L, R, C, LFE, LS1, RS1,
    // LS2, RS2.
    // Note that this differs from the Vorbis/Opus 7.1 channel layout, which
    // is: L, C, R, LS1, RS1, LS2, RS2, LFE.
    kSurroundSevenDotOne = 10,

    // Binaurally renders first-order ambisonics
    // (AmbiX format: 4 channels, ACN channel ordering, SN3D normalization).
    kFirstOrderAmbisonics = 4,

    // Binaurally renders second-order ambisonics.
    // (AmbiX format: 9 channels, ACN channel ordering, SN3D normalization).
    kSecondOrderAmbisonics = 5,

    // Binaurally renders third-order ambisonics.
    // (AmbiX format: 16 channels, ACN channel ordering, SN3D normalization).
    kThirdOrderAmbisonics = 6,

    // Binaurally renders first-order ambisonics with a non-diegetic-stereo
    // track. The first 4 channels contain ambisonic AmbiX format.
    // (AmbiX format: 4 channels, ACN channel ordering, SN3D normalization).
    // Channel 5 to 6 contain non-diegetic-stereo.
    kFirstOrderAmbisonicsWithNonDiegeticStereo = 7,

    // Binaurally renders second-order ambisonics with a non-diegetic-stereo
    // track. The first 9 channels contain ambisonic AmbiX format.
    // (AmbiX format: 9 channels, ACN channel ordering, SN3D normalization).
    // Channel 10 to 11 contain non-diegetic-stereo.
    kSecondOrderAmbisonicsWithNonDiegeticStereo = 8,

    // Binaurally renders third-order ambisonics with a non-diegetic-stereo
    // track. The first 16 channels contain ambisonic AmbiX format.
    // (AmbiX format: 16 channels, ACN channel ordering, SN3D normalization).
    // Channel 17 to 18 contain non-diegetic-stereo.
    kThirdOrderAmbisonicsWithNonDiegeticStereo = 9,

    // Note: Next available value is: 11
  };


  virtual ~BinauralSurroundRenderer() = default;

  // Factory method to create a |BinauralSurroundRenderer| instance. Caller must
  // take ownership of returned instance and destroy it via operator delete.
  //
  // @param frames_per_buffer Number of frames in output buffer.
  // @param sample_rate_hz Sample rate of audio buffers.
  // @param surround_format Input surround sound format.
  // @param return |BinauralSurroundRenderer| instance, nullptr if creation
  //    fails.
  static BinauralSurroundRenderer* Create(size_t frames_per_buffer,
                                          int sample_rate_hz,
                                          SurroundFormat surround_format);

  // Enables the stereo speaker mode. When activated, it disables HRTF-based
  // filtering and switches to computationally cheaper stereo-panning. This
  // helps to avoid HRTF-based coloring effects when stereo speakers are used
  // and reduces computational complexity when headphone-based HRTF filtering is
  // not needed. By default the stereo speaker mode is disabled.
  //
  // @param enabled Flag to enable stereo speaker mode.
  virtual void SetStereoSpeakerMode(bool enabled) = 0;

  // Returns the number of frames the input buffer is currently able to consume.
  //
  // @return Number of available frames in input buffer.
  virtual size_t GetNumAvailableFramesInInputBuffer() const = 0;

  // Adds interleaved int16 audio data to the renderer. If enough data has been
  // provided for an output buffer to be generated then it will be immediately
  // available via |Get[Interleaved|Planar]StereoOutputBuffer|. The input data
  // is copied into an internal buffer which allows the caller to re-use the
  // input buffer immediately. The available space in the internal buffer can be
  // obtained via |GetAvailableInputSizeSamples|.
  //
  // @param input_buffer_ptr Pointer to interleaved input data.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  // @return The number of consumed frames.
  virtual size_t AddInterleavedInput(const int16* input_buffer_ptr,
                                     size_t num_channels,
                                     size_t num_frames) = 0;

  // Adds interleaved floating point audio data to the renderer. If enough data
  // has been provided for an output buffer to be generated then it will be
  // immediately available via |Get[Interleaved|Planar]StereoOutputBuffer|. The
  // input data is copied into an internal buffer which allows the caller to
  // re-use the input buffer immediately. The available space in the internal
  // buffer can be obtained via |GetAvailableInputSizeSamples|.
  //
  // @param input_buffer_ptr Pointer to interleaved input data.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  // @return The number of consumed frames.
  virtual size_t AddInterleavedInput(const float* input_buffer_ptr,
                                     size_t num_channels,
                                     size_t num_frames) = 0;

  // Adds planar int16 audio data to the renderer. If enough data has
  // been provided for an output buffer to be generated then it will be
  // immediately available via |Get[Interleaved|Planar]StereoOutputBuffer|. The
  // input data is copied into an internal buffer which allows the caller to
  // re-use the input buffer immediately. The available space in the internal
  // buffer can be obtained via |GetAvailableInputSizeSamples|.
  //
  // @param input_buffer_ptrs Array of pointers to planar channel data.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  // @return The number of consumed frames.
  virtual size_t AddPlanarInput(const int16* const* input_buffer_ptrs,
                                size_t num_channels, size_t num_frames) = 0;

  // Adds planar floating point audio data to the renderer. If enough data has
  // been provided for an output buffer to be generated then it will be
  // immediately available via |Get[Interleaved|Planar]StereoOutputBuffer|. The
  // input data is copied into an internal buffer which allows the caller to
  // re-use the input buffer immediately. The available space in the internal
  // buffer can be obtained via |GetAvailableInputSizeSamples|.
  //
  // @param input_buffer_ptrs Array of pointers to planar channel data.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  // @return The number of consumed frames.
  virtual size_t AddPlanarInput(const float* const* input_buffer_ptrs,
                                size_t num_channels, size_t num_frames) = 0;

  // Returns the number of samples available in the output buffer.
  //
  // @return Number of available samples in output buffer.
  virtual size_t GetAvailableFramesInStereoOutputBuffer() const = 0;

  // Gets a processed output buffer in interleaved int16 format.
  //
  // @param output_buffer_ptr Pointer to allocated interleaved output buffer.
  // @param num_frames Size of output buffer in frames.
  // @return The number of consumed frames.
  virtual size_t GetInterleavedStereoOutput(int16* output_buffer_ptr,
                                            size_t num_frames) = 0;

  // Gets a processed output buffer in interleaved float format.
  //
  // @param output_buffer_ptr Pointer to allocated interleaved output buffer.
  // @param num_frames Size of output buffer in frames.
  // @return The number of consumed frames.
  virtual size_t GetInterleavedStereoOutput(float* output_buffer_ptr,
                                            size_t num_frames) = 0;

  // Gets a processed output buffer in planar int16 point format.
  //
  // @param output_buffer_ptrs Array of pointers to planar channel data.
  // @param num_frames Number of frames in output buffer.
  // @return The number of consumed frames.
  virtual size_t GetPlanarStereoOutput(int16** output_buffer_ptrs,
                                       size_t num_frames) = 0;

  // Gets a processed output buffer in planar floating point format.
  //
  // @param output_buffer_ptrs Array of pointers to planar channel data.
  // @param num_frames Number of frames in output buffer.
  // @return The number of consumed frames.
  virtual size_t GetPlanarStereoOutput(float** output_buffer_ptrs,
                                       size_t num_frames) = 0;

  // Removes all buffered input and processed output buffers from the buffer
  // queues.
  virtual void Clear() = 0;

  // Triggers the processing of data that has been input but not yet processed.
  // Note after calling this method, all processed output must be consumed via
  // |Get[Interleaved|Planar]StereoOutputBuffer| before adding new input
  // buffers.
  //
  // @return Whether any data was processed.
  virtual bool TriggerProcessing() = 0;

  // Updates the head rotation.
  //
  // @param w W component of quaternion.
  // @param x X component of quaternion.
  // @param y Y component of quaternion.
  // @param z Z component of quaternion.
  virtual void SetHeadRotation(float w, float x, float y, float z) = 0;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_API_BINAURAL_SURROUND_RENDERER_H_
