/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "api/resonance_audio_api.h"

#include "base/logging.h"
#include "graph/resonance_audio_api_impl.h"

namespace vraudio {

extern "C" EXPORT_API ResonanceAudioApi* CreateResonanceAudioApi(
    size_t num_channels, size_t frames_per_buffer, int sample_rate_hz) {
  return new ResonanceAudioApiImpl(num_channels, frames_per_buffer,
                                   sample_rate_hz);
}

}  // namespace vraudio
