
/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_API_RESONANCE_AUDIO_API_H_
#define RESONANCE_AUDIO_API_RESONANCE_AUDIO_API_H_

// EXPORT_API can be used to define the dllimport storage-class attribute.
#if !defined(EXPORT_API)
#define EXPORT_API
#endif

#include <cstddef>  // size_t declaration.
#include <cstdint>  // int16_t declaration.


typedef int16_t int16;

namespace vraudio {

// Rendering modes define CPU load / rendering quality balances.
// Note that this struct is C-compatible by design to be used across external
// C/C++ and C# implementations.
enum RenderingMode {
  // Stereo panning, i.e., this disables HRTF-based rendering.
  kStereoPanning = 0,
  // HRTF-based rendering using First Order Ambisonics, over a virtual array of
  // 8 loudspeakers arranged in a cube configuration around the listener's head.
  kBinauralLowQuality,
  // HRTF-based rendering using Second Order Ambisonics, over a virtual array of
  // 12 loudspeakers arranged in a dodecahedral configuration (using faces of
  // the dodecahedron).
  kBinauralMediumQuality,
  // HRTF-based rendering using Third Order Ambisonics, over a virtual array of
  // 26 loudspeakers arranged in a Lebedev grid: https://goo.gl/DX1wh3.
  kBinauralHighQuality,
  // Room effects only rendering. This disables HRTF-based rendering and direct
  // (dry) output of a sound object. Note that this rendering mode should *not*
  // be used for general-purpose sound object spatialization, as it will only
  // render the corresponding room effects of given sound objects without the
  // direct spatialization.
  kRoomEffectsOnly,
};

// Distance rolloff models used for distance attenuation.
// Note that this enum is C-compatible by design to be used across external
// C/C++ and C# implementations.
enum DistanceRolloffModel {
  // Logarithmic distance rolloff model.
  kLogarithmic = 0,
  // Linear distance rolloff model.
  kLinear,
  // Distance attenuation value will be explicitly set by the user.
  kNone,
};

// Early reflection properties of an acoustic environment.
// Note that this struct is C-compatible by design to be used across external
// C/C++ and C# implementations.
struct ReflectionProperties {
  // Default constructor initializing all data members to 0.
  ReflectionProperties()
      : room_position{0.0f, 0.0f, 0.0f},
        room_rotation{0.0f, 0.0f, 0.0f, 1.0f},
        room_dimensions{0.0f, 0.0f, 0.0f},
        cutoff_frequency(0.0f),
        coefficients{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
        gain(0.0f) {}

  // Center position of the shoebox room in world space.
  float room_position[3];

  // Rotation (quaternion) of the shoebox room in world space.
  float room_rotation[4];

  // Size of the shoebox shoebox room in world space.
  float room_dimensions[3];

  // Frequency threshold for low pass filtering (-3dB cuttoff).
  float cutoff_frequency;

  // Reflection coefficients that are stored in world space as follows:
  //  [0]  (-)ive x-axis wall (left)
  //  [1]  (+)ive x-axis wall (right)
  //  [2]  (-)ive y-axis wall (bottom)
  //  [3]  (+)ive y-axis wall (top)
  //  [4]  (-)ive z-axis wall (front)
  //  [5]  (+)ive z-axis wall (back)
  float coefficients[6];

  // Uniform reflections gain which is applied to all reflections.
  float gain;
};

// Late reverberation properties of an acoustic environment.
// Note that this struct is C-compatible by design to be used across external
// C/C++ and C# implementations.
struct ReverbProperties {
  // Default constructor initializing all data members to 0.
  ReverbProperties()
      : rt60_values{0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f, 0.0f},
        gain(0.0f) {}

  // RT60's of the reverberation tail at different octave band centre
  // frequencies in seconds.
  float rt60_values[9];

  // Reverb gain.
  float gain;
};

class ResonanceAudioApi;

// Factory method to create a |ResonanceAudioApi| instance. Caller must
// take ownership of returned instance and destroy it via operator delete.
//
// @param num_channels Number of channels of audio output.
// @param frames_per_buffer Number of frames per buffer.
// @param sample_rate_hz System sample rate.
extern "C" EXPORT_API ResonanceAudioApi* CreateResonanceAudioApi(
    size_t num_channels, size_t frames_per_buffer, int sample_rate_hz);

// The ResonanceAudioApi library renders high-quality spatial audio. It provides
// methods to binaurally render virtual sound sources with simulated room
// acoustics. In addition, it supports decoding and binaural rendering of
// ambisonic soundfields. Its implementation is single-threaded, thread-safe
// and non-blocking to be able to process raw PCM audio buffers directly on the
// audio thread while receiving parameter updates from the main/render thread.
class ResonanceAudioApi {
 public:
  // Sound object / ambisonic source identifier.
  typedef int SourceId;

  // Invalid source id that can be used to initialize handler variables during
  // class construction.
  static const SourceId kInvalidSourceId = -1;

  virtual ~ResonanceAudioApi() = default;

  // Renders and outputs an interleaved output buffer in float format.
  //
  // @param num_frames Size of output buffer in frames.
  // @param num_channels Number of channels in output buffer.
  // @param buffer_ptr Raw float pointer to audio buffer.
  // @return True if a valid output was successfully rendered, false otherwise.
  virtual bool FillInterleavedOutputBuffer(size_t num_channels,
                                           size_t num_frames,
                                           float* buffer_ptr) = 0;

  // Renders and outputs an interleaved output buffer in int16 format.
  //
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Size of output buffer in frames.
  // @param buffer_ptr Raw int16 pointer to audio buffer.
  // @return True if a valid output was successfully rendered, false otherwise.
  virtual bool FillInterleavedOutputBuffer(size_t num_channels,
                                           size_t num_frames,
                                           int16* buffer_ptr) = 0;

  // Renders and outputs a planar output buffer in float format.
  //
  // @param num_frames Size of output buffer in frames.
  // @param num_channels Number of channels in output buffer.
  // @param buffer_ptr Pointer to array of raw float pointers to each channel of
  //    the audio buffer.
  // @return True if a valid output was successfully rendered, false otherwise.
  virtual bool FillPlanarOutputBuffer(size_t num_channels, size_t num_frames,
                                      float* const* buffer_ptr) = 0;

  // Renders and outputs a planar output buffer in int16 format.
  //
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Size of output buffer in frames.
  // @param buffer_ptr Pointer to array of raw int16 pointers to each channel of
  //    the audio buffer.
  // @return True if a valid output was successfully rendered, false otherwise.
  virtual bool FillPlanarOutputBuffer(size_t num_channels, size_t num_frames,
                                      int16* const* buffer_ptr) = 0;

  // Sets listener's head position.
  //
  // @param x X coordinate of head position in world space.
  // @param y Y coordinate of head position in world space.
  // @param z Z coordinate of head position in world space.
  virtual void SetHeadPosition(float x, float y, float z) = 0;

  // Sets listener's head rotation.
  //
  // @param x X component of quaternion.
  // @param y Y component of quaternion.
  // @param z Z component of quaternion.
  // @param w W component of quaternion.
  virtual void SetHeadRotation(float x, float y, float z, float w) = 0;

  // Sets the Primary volume of the main audio output.
  //
  // @param volume Primary volume (linear) in amplitude in range [0, 1] for
  //     attenuation, range [1, inf) for gain boost.
  virtual void SetPrimaryVolume(float volume) = 0;

  // Enables the stereo speaker mode. When activated, it disables HRTF-based
  // filtering and switches to computationally cheaper stereo-panning. This
  // helps to avoid HRTF-based coloring effects when stereo speakers are used
  // and reduces computational complexity when headphone-based HRTF filtering is
  // not needed. By default the stereo speaker mode is disabled. Note that
  // stereo speaker mode overrides the |enable_hrtf| flag in
  // |CreateSoundObjectSource|.
  //
  // @param enabled Flag to enable stereo speaker mode.
  virtual void SetStereoSpeakerMode(bool enabled) = 0;

  // Creates an ambisonic source instance.
  //
  // @param num_channels Number of input channels.
  // @return Id of new ambisonic source.
  virtual SourceId CreateAmbisonicSource(size_t num_channels) = 0;

  // Creates a stereo non-spatialized source instance, which directly plays back
  // mono or stereo audio.
  //
  // @param num_channels Number of input channels.
  // @return Id of new non-spatialized source.
  virtual SourceId CreateStereoSource(size_t num_channels) = 0;

  // Creates a sound object source instance.
  //
  // @param rendering_mode Rendering mode which governs quality and performance.
  // @return Id of new sound object source.
  virtual SourceId CreateSoundObjectSource(RenderingMode rendering_mode) = 0;

  // Destroys source instance.
  //
  // @param source_id Id of source to be destroyed.
  virtual void DestroySource(SourceId id) = 0;

  // Sets the next audio buffer in interleaved float format to a sound source.
  //
  // @param source_id Id of sound source.
  // @param audio_buffer_ptr Pointer to interleaved float audio buffer.
  // @param num_channels Number of channels in interleaved audio buffer.
  // @param num_frames Number of frames per channel in interleaved audio buffer.
  virtual void SetInterleavedBuffer(SourceId source_id,
                                    const float* audio_buffer_ptr,
                                    size_t num_channels, size_t num_frames) = 0;

  // Sets the next audio buffer in interleaved int16 format to a sound source.
  //
  // @param source_id Id of sound source.
  // @param audio_buffer_ptr Pointer to interleaved int16 audio buffer.
  // @param num_channels Number of channels in interleaved audio buffer.
  // @param num_frames Number of frames per channel in interleaved audio buffer.
  virtual void SetInterleavedBuffer(SourceId source_id,
                                    const int16* audio_buffer_ptr,
                                    size_t num_channels, size_t num_frames) = 0;

  // Sets the next audio buffer in planar float format to a sound source.
  //
  // @param source_id Id of sound source.
  // @param audio_buffer_ptr Pointer to array of pointers referring to planar
  //    audio buffers for each channel.
  // @param num_channels Number of planar input audio buffers.
  // @param num_frames Number of frames per channel.
  virtual void SetPlanarBuffer(SourceId source_id,
                               const float* const* audio_buffer_ptr,
                               size_t num_channels, size_t num_frames) = 0;

  // Sets the next audio buffer in planar int16 format to a sound source.
  //
  // @param source_id Id of sound source.
  // @param audio_buffer_ptr Pointer to array of pointers referring to planar
  //    audio buffers for each channel.
  // @param num_channels Number of planar input audio buffers.
  // @param num_frames Number of frames per channel.
  virtual void SetPlanarBuffer(SourceId source_id,
                               const int16* const* audio_buffer_ptr,
                               size_t num_channels, size_t num_frames) = 0;

  // Sets the given source's distance attenuation value explicitly. The distance
  // rolloff model of the source must be set to |DistanceRolloffModel::kNone|
  // for the set value to take effect.
  //
  // @param source_id Id of source.
  // @param distance_attenuation Distance attenuation value.
  virtual void SetSourceDistanceAttenuation(SourceId source_id,
                                            float distance_attenuation) = 0;

  // Sets the given source's distance attenuation method with minimum and
  // maximum distances. Maximum distance must be greater than the minimum
  // distance for the method to be set.
  //
  // @param source_id Id of source.
  // @param rolloff Linear or logarithmic distance rolloff models.
  // @param min_distance Minimum distance to apply distance attenuation method.
  // @param max_distance Maximum distance to apply distance attenuation method.
  virtual void SetSourceDistanceModel(SourceId source_id,
                                      DistanceRolloffModel rolloff,
                                      float min_distance,
                                      float max_distance) = 0;

  // Sets the given source's position. Note that, the given position for an
  // ambisonic source is only used to determine the corresponding room effects
  // to be applied.
  //
  // @param source_id Id of source.
  // @param x X coordinate of source position in world space.
  // @param y Y coordinate of source position in world space.
  // @param z Z coordinate of source position in world space.
  virtual void SetSourcePosition(SourceId source_id, float x, float y,
                                 float z) = 0;

  // Sets the room effects contribution for the given source.
  //
  // @param source_id Id of source.
  // @param room_effects_gain Linear room effects volume in amplitude in range
  //     [0, 1] for attenuation, range [1, inf) for gain boost.
  virtual void SetSourceRoomEffectsGain(SourceId source_id,
                                        float room_effects_gain) = 0;

  // Sets the given source's rotation.
  //
  // @param source_id Id of source.
  // @param x X component of quaternion.
  // @param y Y component of quaternion.
  // @param z Z component of quaternion.
  // @param w W component of quaternion.
  virtual void SetSourceRotation(SourceId source_id, float x, float y, float z,
                                 float w) = 0;

  // Sets the given source's volume.
  //
  // @param source_id Id of source.
  // @param volume Linear source volume in amplitude in range [0, 1] for
  //     attenuation, range [1, inf) for gain boost.
  virtual void SetSourceVolume(SourceId source_id, float volume) = 0;

  // Sets the given sound object source's directivity.
  //
  // @param sound_object_source_id Id of sound object source.
  // @param alpha Weighting balance between figure of eight pattern and circular
  //     pattern for source emission in range [0, 1]. A value of 0.5 results in
  //     a cardioid pattern.
  // @param order Order applied to computed directivity. Higher values will
  //     result in narrower and sharper directivity patterns. Range [1, inf).
  virtual void SetSoundObjectDirectivity(SourceId sound_object_source_id,
                                         float alpha, float order) = 0;

  // Sets the listener's directivity with respect to the given sound object.
  // This method could be used to simulate an angular rolloff in terms of the
  // listener's orientation, given the polar pickup pattern with |alpha| and
  // |order|.
  //
  // @param sound_object_source_id Id of sound object source.
  // @param alpha Weighting balance between figure of eight pattern and circular
  //     pattern for listener's pickup in range [0, 1]. A value of 0.5 results
  //     in a cardioid pattern.
  // @param order Order applied to computed pickup pattern. Higher values will
  //     result in narrower and sharper pickup patterns. Range [1, inf).
  virtual void SetSoundObjectListenerDirectivity(
      SourceId sound_object_source_id, float alpha, float order) = 0;

  // Sets the gain (linear) of the near field effect.
  //
  // @param sound_object_source_id Id of sound object source.
  // @param gain Gain of the near field effect. Range [0, 9] (corresponding to
  //     approx. (-Inf, +20dB]).
  virtual void SetSoundObjectNearFieldEffectGain(
      SourceId sound_object_source_id, float gain) = 0;

  // Sets the given sound object source's occlusion intensity.
  //
  // @param sound_object_source_id Id of sound object source.
  // @param intensity Number of occlusions occurred for the object. The value
  //     can be set to fractional for partial occlusions. Range [0, inf).
  virtual void SetSoundObjectOcclusionIntensity(SourceId sound_object_source_id,
                                                float intensity) = 0;

  // Sets the given sound object source's spread.
  //
  // @param sound_object_source_id Id of sound object source.
  // @param spread_deg Spread in degrees.
  virtual void SetSoundObjectSpread(SourceId sound_object_source_id,
                                    float spread_deg) = 0;

  // Turns on/off the reflections and reverberation.
  virtual void EnableRoomEffects(bool enable) = 0;

  // Sets the early reflection properties of the environment.
  //
  // @param reflection_properties Reflection properties.
  virtual void SetReflectionProperties(
      const ReflectionProperties& reflection_properties) = 0;

  // Sets the late reverberation properties of the environment.
  //
  // @param reverb_properties Reverb properties.
  virtual void SetReverbProperties(
      const ReverbProperties& reverb_properties) = 0;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_API_RESONANCE_AUDIO_API_H_

