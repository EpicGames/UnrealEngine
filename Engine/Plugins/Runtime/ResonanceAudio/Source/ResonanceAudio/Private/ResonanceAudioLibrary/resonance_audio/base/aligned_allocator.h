/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_ALIGNED_ALLOCATOR_H_
#define RESONANCE_AUDIO_BASE_ALIGNED_ALLOCATOR_H_

#include <stdlib.h>

#include <algorithm>
#include <cstddef>
#include <memory>
#include <type_traits>

#include "base/simd_utils.h"

#if defined(__clang__)
_Pragma("clang diagnostic push")

// suppress clang warnings here.

#else
__pragma(warning(push))
__pragma(warning(disable: 4100))
__pragma(warning(disable: 4996))
#endif

namespace vraudio {

// Performs static assert checks on the types size and alignment parameters.
template <size_t TypeSize, size_t Alignment>
void StaticAlignmentCheck() {
  const bool alignment_is_power_of_two =
      !(Alignment == 0) && !(Alignment & (Alignment - 1));
  static_assert(alignment_is_power_of_two, "Alignment must be power of two");

  const bool type_size_is_power_of_two = !(TypeSize & (TypeSize - 1));
  static_assert(type_size_is_power_of_two, "Type size must be power of two");
}

// Returns a pointer to aligned memory.
template <typename Type, typename SizeType, typename PointerType>
PointerType AllignedMalloc(SizeType size, SizeType alignment) {
  const SizeType data_size = size * sizeof(Type);
  const SizeType offset = alignment - 1 + sizeof(PointerType);
  void* mem_block_begin = malloc(data_size + offset);
  if (mem_block_begin == nullptr) {
    return nullptr;
  }
  // Find memory aligned address.
  void** mem_block_aligned = reinterpret_cast<void**>(
      ((reinterpret_cast<SizeType>(mem_block_begin) + offset) &
       (~(alignment - 1))));
  // Save pointer to original block right before the aligned block.
  mem_block_aligned[-1] = mem_block_begin;
  return reinterpret_cast<PointerType>(mem_block_aligned);
}

// Frees memory that has been aligned with |AllignedMalloc|.
template <typename PointerType>
void AllignedFree(PointerType mem_block_aligned) {
  free(*(reinterpret_cast<void**>(mem_block_aligned) - 1));
}

// Class that allocates aligned memory. It is derived from std::allocator class
// to be used with STL containers.
//
// @tparam Type Datatype of container to allocate.
// @tparam Alignment Size of memory alignment.
template <typename Type, size_t Alignment>
class AlignedAllocator : public std::allocator<Type> {
 public:
  using Pointer = typename std::allocator_traits<std::allocator<Type>>::pointer;
  using ConstPointer = typename std::allocator_traits<std::allocator<Type>>::const_pointer;
  using SizeType = typename std::allocator_traits<std::allocator<Type>>::size_type;

  AlignedAllocator() { StaticAlignmentCheck<sizeof(Type), Alignment>(); }

  // Allocates memory for |size| elements and returns a pointer that is aligned
  // to a multiple to |Alignment|.
  //
  // @param size Number of elements to allocate.
  // @return Returns memory aligned pointer.
  Pointer allocate(SizeType size) { return allocate(size, nullptr); }

  // Allocates memory for |size| elements and returns a pointer that is aligned
  // to a multiple to |Alignment|.
  //
  // @param size Number of elements to allocate.
  // @return Returns memory aligned pointer.
  Pointer allocate(SizeType size, ConstPointer /* hint */) {

    return AllignedMalloc<Type, SizeType, Pointer>(size, Alignment);
  }

  void deallocate(Pointer mem_block_aligned, size_t size) {
    AllignedFree<Pointer>(mem_block_aligned);
  }

  // Copy constructor to support rebind operation (to make MSVC happy).
  template <typename U>
  explicit AlignedAllocator<Type, Alignment>(
      const AlignedAllocator<U, Alignment>& other) {}

  // Rebind is used to allocate container internal variables of type |U|
  // (which don't need to be aligned).
  template <typename U>
  struct rebind {
    typedef AlignedAllocator<U, Alignment> other;
  };
};

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#else
__pragma(warning(pop))
#endif

#endif  // RESONANCE_AUDIO_BASE_ALIGNED_ALLOCATOR_H_
