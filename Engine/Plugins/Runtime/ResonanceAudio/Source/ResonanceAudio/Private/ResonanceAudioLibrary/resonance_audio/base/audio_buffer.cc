/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "base/audio_buffer.h"

namespace vraudio {

AudioBuffer::AudioBuffer() : num_frames_(0), data_size_(0), source_id_(kInvalidSourceId) {}

AudioBuffer::AudioBuffer(size_t num_channels, size_t num_frames)
    : num_frames_(num_frames), data_size_(0), source_id_(kInvalidSourceId) {

  InitChannelViews(num_channels);
}

// Copy assignment from AudioBuffer.
AudioBuffer& AudioBuffer::operator=(const AudioBuffer& other) {
  if (this != &other) {
    num_frames_ = other.num_frames_;
    source_id_ = other.source_id_;
    InitChannelViews(other.num_channels());
    for (size_t i = 0; i < num_channels(); ++i) {
      channel_views_[i] = other.channel_views_[i];
    }
  }
  return *this;
}

AudioBuffer::AudioBuffer(AudioBuffer&& other) {
  num_frames_ = other.num_frames_;
  other.num_frames_ = 0;
  data_ = std::move(other.data_);
  data_size_ = other.data_size_;
  other.data_size_ = 0;
  channel_views_ = std::move(other.channel_views_);
  source_id_ = other.source_id_;
  other.source_id_ = kInvalidSourceId;
}

void AudioBuffer::InitChannelViews(size_t num_channels) {


  const size_t num_frames_to_next_channel = FindNextAlignedArrayIndex(
      num_frames_, sizeof(float), kMemoryAlignmentBytes);

  data_size_ = num_channels * num_frames_to_next_channel;
  data_.resize(data_size_);

  channel_views_.clear();
  channel_views_.reserve(num_channels);

  float* itr = data_.data();

  for (size_t i = 0; i < num_channels; ++i) {
    ChannelView new_channel_view(itr, num_frames_);
    channel_views_.push_back(new_channel_view);
    itr += num_frames_to_next_channel;
  }
}

}  // namespace vraudio
