/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_AUDIO_BUFFER_H_
#define RESONANCE_AUDIO_BASE_AUDIO_BUFFER_H_

#include <algorithm>
#include <memory>

#include "base/integral_types.h"
#include "base/aligned_allocator.h"
#include "base/channel_view.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_utils.h"


namespace vraudio {

// Audio buffer that manages multi-channel audio data in a planar data format.
// All channels are sequentially stored within a single consecutive chunk of
// memory. To access individual channel data, the array subscript operator can
// be used to obtain a |AudioBuffer::Channel|. Note that the user must guarantee
// that the AudioBuffer instance lives as long as its channel data is accessed
// via |AudioBuffer::Channel|s. Note that allocated buffers may *not* be
// initialized to zero.
//
// Examples:
//
// // Range-based for-loop over all channels and all samples.
// AudioBuffer audio_buffer(...)
// for (AudioBuffer::Channel& channel : audio_buffer) {
//   for (float& sample : channel) {
//      sample *= gain;
//   }
// }
//
// // Range-based for-loop over all channels and array subscripts-based for-loop
// // to access samples.
// AudioBuffer audio_buffer(...)
// for (AudioBuffer::Channel& channel : audio_buffer) {
//   for (size_t i = 0; i < channel.num_frames(); ++i) {
//      channel[i] *= gain;
//   }
// }
//
// // Array subscript-based for-loops over all channels samples.
// // AudioBuffer audio_buffer(...)
// for (size_t c=0; c < audio_buffer.num_channels(); ++c) {
//   // First obtain a reference to AudioBuffer::Channel.
//   AudioBuffer::Channel& channel = audio_buffer[c];
//   for (size_t i = 0; i < channel.num_frames(); ++i) {
//      channel[i] *= gain;
//   }
// }
//
// Note do *NOT* use double array subscripts to iterate over multiple samples
// since it performs a channel iterator lookup for every sample:
// for (size_t c=0; c < audio_buffer.num_channels(); ++c) {
//   for (size_t i = 0; i < channel.size(); ++i) {
//      audio_buffer[c][i] *= gain;  // *BAD*
//   }
// }
//
class AudioBuffer {
 public:
  // View on separate audio channel.
  typedef ChannelView Channel;

  // Allocator class to allocate aligned floats.
  typedef AlignedAllocator<float, kMemoryAlignmentBytes> FloatAllocator;

  // Allocator class to allocate aligned int16s.
  typedef AlignedAllocator<int16, kMemoryAlignmentBytes> Int16Allocator;

  // AlignedFloatBuffer for storing audio data.
  typedef std::vector<float, FloatAllocator> AlignedFloatVector;

  // AlignedInt16Buffer for storing audio data.
  typedef std::vector<int16, Int16Allocator> AlignedInt16Vector;

  // Default constructor initializes an empty |AudioBuffer|.
  AudioBuffer();

  // Constructor.
  //
  // @param num_channels Number of channels.
  // @param num_frames Number of frames.
  AudioBuffer(size_t num_channels, size_t num_frames);

  // Move constructor.
  AudioBuffer(AudioBuffer&& other);

  // Copy constructor is explicitly deleted to prevent accidental copies.
  // Use copy assignment operator instead.
  AudioBuffer(const AudioBuffer& other) = delete;

  // Copy assignment from AudioBuffer.
  AudioBuffer& operator=(const AudioBuffer& other);

  // Returns the number of audio channels.
  size_t num_channels() const { return channel_views_.size(); }

  // Returns the number of frames per buffer.
  size_t num_frames() const { return num_frames_; }

  // Returns this buffer's source id.
  SourceId source_id() const { return source_id_; }

  // Returns a reference to the selected ChannelView.
  Channel& operator[](size_t channel) {
    DCHECK_LT(channel, channel_views_.size());
    return channel_views_[channel];
  }

  // Returns a const reference to the selected ChannelView.
  const Channel& operator[](size_t channel) const {
    DCHECK_LT(channel, channel_views_.size());
    return channel_views_[channel];
  }

  // Copy assignment from std::vector<std::vector<float>>.
  AudioBuffer& operator=(const std::vector<std::vector<float>>& other) {
    DCHECK_EQ(other.size(), channel_views_.size());
    for (size_t channel = 0; channel < channel_views_.size(); ++channel) {
      channel_views_[channel] = other[channel];
    }
    return *this;
  }

  // += operator
  AudioBuffer& operator+=(const AudioBuffer& other) {
    DCHECK_EQ(other.num_channels(), num_channels());
    DCHECK_EQ(other.num_frames(), num_frames());
    for (size_t i = 0; i < channel_views_.size(); ++i)
      channel_views_[i] += other[i];

    return *this;
  }

  // -= operator
  AudioBuffer& operator-=(const AudioBuffer& other) {
    DCHECK_EQ(other.num_channels(), num_channels());
    DCHECK_EQ(other.num_frames(), num_frames());
    for (size_t i = 0; i < channel_views_.size(); ++i)
      channel_views_[i] -= other[i];

    return *this;
  }

  // Returns an iterator to the ChannelView of the first channel.
  std::vector<Channel>::iterator begin() { return channel_views_.begin(); }

  // Returns an iterator to the end of the ChannelView vector.
  std::vector<Channel>::iterator end() { return channel_views_.end(); }

  // Returns a const_iterator to the ChannelView of the first channel.
  std::vector<Channel>::const_iterator begin() const {
    return channel_views_.begin();
  }

  // Returns an const_iterator to the end of the ChannelView vector.
  std::vector<Channel>::const_iterator end() const {
    return channel_views_.end();
  }

  // Fills all channels with zeros and reenables |Channel|s.
  void Clear() {
    for (Channel& channel : channel_views_) {
      channel.SetEnabled(true);
      channel.Clear();
    }
  }

  // Returns the number of allocated frames per |Channel|. Note this may
  // differ from the actual size of the |Channel| to ensure alignment of all
  // |Channel|s.
  size_t GetChannelStride() const {
    return FindNextAlignedArrayIndex(num_frames_, sizeof(float),
                                     kMemoryAlignmentBytes);
  }

  // Sets the source id of which the buffer belongs to.
  void set_source_id(SourceId source_id) { source_id_ = source_id; }

 private:
  // Allocates memory and initializes vector of |ChannelView|s.
  void InitChannelViews(size_t num_channels);

  // Number of frames per buffer.
  size_t num_frames_;

  // Audio buffer that sequentially stores multiple audio channels in a planar
  // format.
  AlignedFloatVector data_;

  // Size of audio buffer.
  size_t data_size_;

  // Vector of |AudioBuffer::Channel|s.
  std::vector<Channel> channel_views_;

  // Id of a source that this buffer belongs to.
  SourceId source_id_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_AUDIO_BUFFER_H_
