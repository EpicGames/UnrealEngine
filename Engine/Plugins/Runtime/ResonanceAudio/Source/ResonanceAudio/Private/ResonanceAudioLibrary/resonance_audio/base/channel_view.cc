/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "base/channel_view.h"

#include "base/simd_utils.h"

namespace vraudio {

ChannelView& ChannelView::operator+=(const ChannelView& other) {
  DCHECK_EQ(other.size(), size_);
  DCHECK(enabled_);
  float* this_sample = begin();
  const float* other_sample = other.begin();
  AddPointwise(size_, other_sample, this_sample, this_sample);
  return *this;
}

ChannelView& ChannelView::operator-=(const ChannelView& other) {
  DCHECK_EQ(other.size(), size_);
  DCHECK(enabled_);
  float* this_sample = begin();
  const float* other_sample = other.begin();
  SubtractPointwise(size_, other_sample, this_sample, this_sample);
  return *this;
}

ChannelView& ChannelView::operator*=(const ChannelView& other) {
  DCHECK_EQ(other.size(), size_);
  DCHECK(enabled_);
  float* this_sample = begin();
  const float* other_sample = other.begin();
  MultiplyPointwise(size_, other_sample, this_sample, this_sample);
  return *this;
}

}  // namespace vraudio
