/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_CHANNEL_VIEW_H_
#define RESONANCE_AUDIO_BASE_CHANNEL_VIEW_H_

#include <algorithm>
#include <cstring>
#include <vector>

#include "base/logging.h"

namespace vraudio {

// Provides an interface to a single audio channel in |AudioBuffer|. Note that a
// |ChannelView| instance does not own the data it is initialized with.
class ChannelView {
 public:
  // @third party code - Epic Games Begin
  // This is here for this warning as of Clang 16:
  // channel_view.h(82,16): error: definition of implicit copy constructor for 'ChannelView' is deprecated because it has a user-provided copy assignment operator [-Werror,-Wdeprecated-copy-with-user-provided-copy]
  ChannelView(const ChannelView&) = default;
  // @third party code - Epic Games End

  // Array subscript operator returning a reference.
  float& operator[](size_t index) {
    DCHECK(enabled_);
    DCHECK_LT(index, size_);
    return *(begin() + index);
  }

  // Const array subscript operator returning a const reference.
  const float& operator[](size_t index) const {
    DCHECK(enabled_);
    DCHECK_LT(index, size_);
    return *(begin() + index);
  }

  // Returns the size of the channel in samples.
  size_t size() const { return size_; }

  // Returns a float pointer to the begin of the channel data.
  float* begin() {
    DCHECK(enabled_);
    return begin_itr_;
  }

  // Returns a float pointer to the end of the channel data.
  float* end() {
    DCHECK(enabled_);
    return begin_itr_ + size_;
  }

  // Returns a const float pointer to the begin of the channel data.
  const float* begin() const {
    DCHECK(enabled_);
    return begin_itr_;
  }

  // Returns a const float pointer to the end of the channel data.
  const float* end() const {
    DCHECK(enabled_);
    return begin_itr_ + size_;
  }

  // Copy assignment from float vector.
  ChannelView& operator=(const std::vector<float>& other) {
    DCHECK(enabled_);
    DCHECK_EQ(other.size(), size_);
    memcpy(begin(), other.data(), sizeof(float) * size_);
    return *this;
  }

  // Copy assignment from ChannelView.
  ChannelView& operator=(const ChannelView& other) {
    if (this != &other) {
      DCHECK(enabled_);
      DCHECK_EQ(other.size(), size_);
      memcpy(begin(), other.begin(), sizeof(float) * size_);
    }
    return *this;
  }

  // Adds a |ChannelView| to this |ChannelView|.
  ChannelView& operator+=(const ChannelView& other);

  // Subtracts a |ChannelView| from this |ChannelView|.
  ChannelView& operator-=(const ChannelView& other);

  // Pointwise multiplies a |ChannelView| with this |Channelview|.
  ChannelView& operator*=(const ChannelView& other);

  // Fills channel buffer with zeros.
  void Clear() {
    DCHECK(enabled_);
    memset(begin(), 0, sizeof(float) * size_);
  }

  // Allows for disabling the channel to prevent access to the channel data and
  // channel iterators. It is used in the |Mixer| class to prevent the copies of
  // silence |ChannelView|s. Note that |ChannelView| are enabled by default.
  //
  // @param enabled True to enable the channel.
  void SetEnabled(bool enabled) { enabled_ = enabled; }

  // Returns true if |ChannelView| is enabled.
  //
  // @return State of |enabled_| flag.
  bool IsEnabled() const { return enabled_; }

 private:
  friend class AudioBuffer;

  // Constructor is initialized with a float pointer to the first sample and the
  // size of chunk of planar channel data.
  ChannelView(float* begin_itr, size_t size)
      : begin_itr_(begin_itr), size_(size), enabled_(true) {}

  // Iterator of first and last element in channel.
  float* const begin_itr_;

  // Channel size.
  const size_t size_;

  // Flag indicating if the channel is enabled.
  bool enabled_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_CHANNEL_VIEW_H_
