/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_CONSTANTS_AND_TYPES_H_
#define RESONANCE_AUDIO_BASE_CONSTANTS_AND_TYPES_H_

#include <cmath>
#include <string>  // for size_t

#ifndef M_PI
#define DEFINED_M_PI
#define M_PI 3.14159265358979323846
#endif

namespace vraudio {

// Sound object / ambisonic source identifier.

typedef int SourceId;

// Invalid source id that can be used to initialize handler variables during
// class construction.
inline constexpr SourceId kInvalidSourceId = -1;


// Defines memory alignment of audio buffers. Note that not only the first
// element of the |data_| buffer is memory aligned but also the address of the
// first elements of the |ChannelView|s.
inline const size_t kMemoryAlignmentBytes = 64;

// Maximum Ambisonic order currently supported in vr audio, equivalent to High
// Quality sound object rendering mode. This number is limited by a) number of
// HRIR data points used in the binaural renderer; b) size of the lookup table
// controlling the angular spread of a sound source in the Ambisonic Lookup
// Table class.
inline constexpr int kMaxSupportedAmbisonicOrder = 3;

// Maximum allowed size of internal buffers.
inline const size_t kMaxSupportedNumFrames = 16384;

// Number of mono channels.
inline constexpr size_t kNumMonoChannels = 1;

// Number of stereo channels.
inline constexpr size_t kNumStereoChannels = 2;

// Number of surround 5.1 channels.
inline constexpr size_t kNumSurroundFiveDotOneChannels = 6;

// Number of surround 7.1 channels.
inline constexpr size_t kNumSurroundSevenDotOneChannels = 8;

// Number of first-order ambisonic channels.
inline constexpr size_t kNumFirstOrderAmbisonicChannels = 4;

// Number of second-order ambisonic channels.
inline constexpr size_t kNumSecondOrderAmbisonicChannels = 9;

// Number of third-order ambisonic channels.
inline constexpr size_t kNumThirdOrderAmbisonicChannels = 16;

// Number of first-order ambisonic with non-diegetic stereo channels.
inline constexpr size_t kNumFirstOrderAmbisonicWithNonDiegeticStereoChannels = 6;

// Number of second-order ambisonic with non-diegetic stereo channels.
inline constexpr size_t kNumSecondOrderAmbisonicWithNonDiegeticStereoChannels = 11;

// Number of third-order ambisonic with non-diegetic stereo channels.
inline constexpr size_t kNumThirdOrderAmbisonicWithNonDiegeticStereoChannels = 18;

// Negative 60dB in amplitude.
inline constexpr float kNegative60dbInAmplitude = 0.001f;

// Tolerated error margins for floating points.
inline constexpr double kEpsilonDouble = 1e-6;
inline constexpr float kEpsilonFloat = 1e-6f;

// Inverse square root of two (equivalent to -3dB audio signal attenuation).
inline const float kInverseSqrtTwo = 1.0f / std::sqrt(2.0f);

// Square roots.
inline const float kSqrtTwo = std::sqrt(2.0f);
inline const float kSqrtThree = std::sqrt(3.0f);

// Pi in radians.
inline constexpr float kPi = static_cast<float>(M_PI);
// Half pi in radians.
inline constexpr float kHalfPi = static_cast<float>(M_PI / 2.0);
// Two pi in radians.
inline constexpr float kTwoPi = static_cast<float>(2.0 * M_PI);

// Defines conversion factor from degrees to radians.
inline constexpr float kRadiansFromDegrees = static_cast<float>(M_PI / 180.0);

// Defines conversion factor from radians to degrees.
inline constexpr float kDegreesFromRadians = static_cast<float>(180.0 / M_PI);

// The negated natural logarithm of 1000.
inline const float kNegativeLog1000 = -std::log(1000.0f);

// The lowest octave band for computing room effects.
inline constexpr float kLowestOctaveBandHz = 31.25f;

// Number of octave bands in which room effects are computed.
inline constexpr size_t kNumReverbOctaveBands = 9;

// Centers of possible frequency bands up 8 kHz.
// ------------------------------------
// Band no.  Low     Center     High    [Frequencies in Hz]
// ------------------------------------
// 0         22        31.25     44.2
// 1         44.2      62.5      88.4
// 2         88.4      125       176.8
// 3         176.8     250       353.6
// 4         353.6     500       707.1
// 5         707.1     1000      1414.2
// 6         1414.2    2000      2828.4
// 7         2828.4    4000      5656.9
// 8         5656.9    8000      11313.7
//--------------------------------------
const float kOctaveBandCentres[kNumReverbOctaveBands] = {
    31.25f, 62.5f, 125.0f, 250.0f, 500.0f, 1000.0f, 2000.0f, 4000.0f, 8000.0f};

// Number of surfaces in a shoe-box room.
inline constexpr size_t kNumRoomSurfaces = 6;

// Speed of sound in air at 20 degrees Celsius in meters per second.
// http://www.sengpielaudio.com/calculator-speedsound.htm
inline constexpr float kSpeedOfSound = 343.0f;

// Locations of the stereo virtual loudspeakers in degrees.
inline constexpr float kStereoLeftDegrees = 90.0f;
inline constexpr float kStereoRightDegrees = -90.0f;

// Conversion factor from seconds to milliseconds.
inline constexpr float kMillisecondsFromSeconds = 1000.0f;

// Conversion factor from milliseconds to seconds.
inline constexpr float kSecondsFromMilliseconds = 0.001f;

// Conversion factor from seconds to milliseconds.
inline constexpr double kMicrosecondsFromSeconds = 1e6;

// Conversion factor from milliseconds to seconds.
inline constexpr double kSecondsFromMicroseconds = 1e-6;

// The distance threshold where the near field effect should fade in.
inline constexpr float kNearFieldThreshold = 1.0f;

// Minimum allowed distance of a near field sound source used to cap the allowed
// energy boost.
inline constexpr float kMinNearFieldDistance = 0.1f;

// Maximum gain applied by Near Field Effect to the mono source signal.
inline constexpr float kMaxNearFieldEffectGain = 9.0f;

// Number of samples across which the gain value should be interpolated for
// a unit gain change of 1.0f.

inline constexpr size_t kUnitRampLength = 2048;

// Rotation quantization which applies in ambisonic soundfield rotators.

inline constexpr float kRotationQuantizationRad = 1.0f * kRadiansFromDegrees;

}  // namespace vraudio

#ifdef DEFINED_M_PI
#undef DEFINED_M_PI
#undef M_PI
#endif

#endif  // RESONANCE_AUDIO_BASE_CONSTANTS_AND_TYPES_H_
