/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Copyright 2015 Google Inc. All Rights Reserved.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.
//
// Basic integer type definitions for various platforms

#ifndef BASE_INTEGRAL_TYPES_H_
#define BASE_INTEGRAL_TYPES_H_

// Standard typedefs
typedef signed char         schar;
typedef signed char         int8;
typedef short               int16;
typedef int                 int32;
#ifdef _MSC_VER
typedef __int64             int64;
#else
typedef long long           int64;
#endif /* _MSC_VER */

// NOTE: unsigned types are DANGEROUS in loops and other arithmetical
// places.  Use the signed types unless your variable represents a bit
// pattern (eg a hash value) or you really need the extra bit.  Do NOT
// use 'unsigned' to express "this value should always be positive";
// use assertions for this.

typedef unsigned char      uint8;
typedef unsigned short     uint16;
typedef unsigned int       uint32;
#ifdef _MSC_VER
typedef unsigned __int64   uint64;
#else
typedef unsigned long long uint64;
#endif /* _MSC_VER */

// A type to represent a Unicode code-point value. As of Unicode 4.0,
// such values require up to 21 bits.
// (For type-checking on pointers, make this explicitly signed,
// and it should always be the signed version of whatever int32 is.)
typedef signed int         char32;

//  A type to represent a natural machine word (for e.g. efficiently
// scanning through memory for checksums or index searching). Don't use
// this for storing normal integers. Ideally this would be just
// unsigned int, but our 64-bit architectures use the LP64 model
// (http://en.wikipedia.org/wiki/64-bit_computing#64-bit_data_models), hence
// their ints are only 32 bits. We want to use the same fundamental
// type on all archs if possible to preserve *printf() compatability.
typedef unsigned long      uword_t;

// long long macros to be used because gcc and vc++ use different suffixes,
// and different size specifiers in format strings
#undef GG_LONGLONG
#undef GG_ULONGLONG
#undef GG_LL_FORMAT

#ifdef _MSC_VER     /* if Visual C++ */

// VC++ long long suffixes
#define GG_LONGLONG(x) x##I64
#define GG_ULONGLONG(x) x##UI64

// Length modifier in printf format string for int64's (e.g. within %d)
#define GG_LL_FORMAT "I64"  // As in printf("%I64d", ...)
#define GG_LL_FORMAT_W L"I64"

#else   /* not Visual C++ */

#define GG_LONGLONG(x) x##LL
#define GG_ULONGLONG(x) x##ULL
#define GG_LL_FORMAT "ll"  // As in "%lld". Note that "q" is poor form also.
#define GG_LL_FORMAT_W L"ll"

#endif  // _MSC_VER


static const uint8  kuint8max  = (( uint8) 0xFF);
static const uint16 kuint16max = ((uint16) 0xFFFF);
static const uint32 kuint32max = ((uint32) 0xFFFFFFFF);
static const uint64 kuint64max = ((uint64) GG_LONGLONG(0xFFFFFFFFFFFFFFFF));
static const  int8  kint8min   = ((  int8) ~0x7F);
static const  int8  kint8max   = ((  int8) 0x7F);
static const  int16 kint16min  = (( int16) ~0x7FFF);
static const  int16 kint16max  = (( int16) 0x7FFF);
static const  int32 kint32min  = (( int32) ~0x7FFFFFFF);
static const  int32 kint32max  = (( int32) 0x7FFFFFFF);
static const  int64 kint64min  = (( int64) GG_LONGLONG(~0x7FFFFFFFFFFFFFFF));
static const  int64 kint64max  = (( int64) GG_LONGLONG(0x7FFFFFFFFFFFFFFF));


typedef uint64 Fprint;
static const Fprint kIllegalFprint = 0;
static const Fprint kMaxFprint = GG_ULONGLONG(0xFFFFFFFFFFFFFFFF);

#endif  // BASE_INTEGRAL_TYPES_H_
