/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_PLATFORM_LOGGING_H_
#define RESONANCE_AUDIO_PLATFORM_LOGGING_H_

#include <cstdlib>

#include <cassert>
#include <iostream>
#include <sstream>

#undef DCHECK
#undef DCHECK_EQ
#undef DCHECK_NE
#undef DCHECK_LE
#undef DCHECK_LT
#undef DCHECK_GE
#undef DCHECK_GT
#undef CHECK
#undef CHECK_EQ
#undef CHECK_NE
#undef CHECK_LE
#undef CHECK_LT
#undef CHECK_GE
#undef CHECK_GT
#undef CHECK_NOTNULL
#undef LOG

// This class is used to disable logging, while still allowing for log messages
// to contain '<<' expressions.
class NullLogger {
 public:
  std::ostream& GetStream() {
    static std::ostream kNullStream(nullptr);
    return kNullStream;
  }
};

#if !defined(__clang__)
__pragma(warning(disable: 4100))
#endif

#define DISABLE_CHECKS 1
#define INCLUDE_UNUSED_CODE 0

#if DISABLE_CHECKS
#define DCHECK(expr) 
#else
// If statement prevents unused variable warnings.
#define DCHECK(expr)   \
  if (false && (expr)) \
    ;                  \
  else                 \
    NullLogger().GetStream()
#endif

#define DCHECK_OP(val1, val2, op) DCHECK((val1)op(val2))

#define DCHECK_EQ(val1, val2) DCHECK_OP((val1), (val2), ==)
#define DCHECK_NE(val1, val2) DCHECK_OP((val1), (val2), !=)
#define DCHECK_LE(val1, val2) DCHECK_OP((val1), (val2), <=)
#define DCHECK_LT(val1, val2) DCHECK_OP((val1), (val2), <)
#define DCHECK_GE(val1, val2) DCHECK_OP((val1), (val2), >=)
#define DCHECK_GT(val1, val2) DCHECK_OP((val1), (val2), >)

#define ABORT_ON_FATAL_LOG 0

// This class is used to log to std::cerr.
class FatalLogger {
 public:
  FatalLogger(const char* file, int line) {
    error_string_ << file << ":" << line << ": ";
  }
  ~FatalLogger() {
    const std::string error_string = error_string_.str();
    std::cerr << error_string << std::endl;
#if ABORT_ON_FATAL_LOG
    abort();
#endif
  }
  std::ostream& GetStream() { return error_string_; }

 private:
  std::ostringstream error_string_;
};

#define CHECK(condition)                                     \
  !(condition) ? FatalLogger(__FILE__, __LINE__).GetStream() \
               : NullLogger().GetStream()

#define CHECK_OP(val1, val2, op) CHECK((val1)op(val2))

#define CHECK_EQ(val1, val2) CHECK_OP((val1), (val2), ==)
#define CHECK_NE(val1, val2) CHECK_OP((val1), (val2), !=)
#define CHECK_LE(val1, val2) CHECK_OP((val1), (val2), <=)
#define CHECK_LT(val1, val2) CHECK_OP((val1), (val2), <)
#define CHECK_GE(val1, val2) CHECK_OP((val1), (val2), >=)
#define CHECK_GT(val1, val2) CHECK_OP((val1), (val2), >)

// Helper for CHECK_NOTNULL(), using C++11 perfect forwarding.
template <typename T>
T CheckNotNull(T&& t) {
  assert(t != nullptr);
  return std::forward<T>(t);
}
#define CHECK_NOTNULL(val) CheckNotNull(val)

#define LOG(severity) NullLogger().GetStream()

#endif  // RESONANCE_AUDIO_PLATFORM_LOGGING_H_
