/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "base/misc_math.h"

namespace vraudio {

WorldPosition::WorldPosition() { setZero(); }

WorldRotation::WorldRotation() { setIdentity(); }

bool LinearLeastSquareFitting(const std::vector<float>& x_array,
                              const std::vector<float>& y_array, float* slope,
                              float* intercept, float* r_squared) {
  // The array sizes must agree.
  if (x_array.size() != y_array.size()) {
    return false;
  }

  // At least two points are needed to fit a line.
  if (x_array.size() < 2) {
    return false;
  }

  float x_sum = 0.0f;
  float y_sum = 0.0f;
  float x_square_sum = 0.0f;
  float xy_sum = 0.0f;

  for (size_t i = 0; i < x_array.size(); ++i) {
    const float x = x_array[i];
    const float y = y_array[i];
    x_sum += x;
    y_sum += y;
    x_square_sum += x * x;
    xy_sum += x * y;
  }

  const float n_inverse = 1.0f / static_cast<float>(x_array.size());
  const float x_mean = x_sum * n_inverse;
  const float y_mean = y_sum * n_inverse;
  const float x_square_mean = x_square_sum * n_inverse;
  const float xy_mean = xy_sum * n_inverse;
  const float x_mean_square = x_mean * x_mean;

  // Prevent division by zero, which means a vertical line and the slope is
  // infinite.
  if (x_square_mean == x_mean_square) {
    return false;
  }

  *slope = (xy_mean - x_mean * y_mean) / (x_square_mean - x_mean_square);
  *intercept = y_mean - *slope * x_mean;

  // Compute the coefficient of determination.
  float total_sum_of_squares = 0.0f;
  float residual_sum_of_squares = 0.0f;
  for (size_t i = 0; i < x_array.size(); ++i) {
    const float y_i = y_array[i];
    total_sum_of_squares += (y_i - y_mean) * (y_i - y_mean);
    const float y_fit = *slope * x_array[i] + *intercept;
    residual_sum_of_squares += (y_fit - y_i) * (y_fit - y_i);
  }

  if (total_sum_of_squares == 0.0f) {
    if (residual_sum_of_squares == 0.0f) {
      // A special case where all y's are equal, where the |r_squared| should
      // be 1.0, and the line is a perfectly horizontal line.
      *r_squared = 1.0f;
      return true;
    } else {
      // Division by zero.
      return false;
    }
  }

  *r_squared = 1.0f - residual_sum_of_squares / total_sum_of_squares;
  return true;
}

}  // namespace vraudio
