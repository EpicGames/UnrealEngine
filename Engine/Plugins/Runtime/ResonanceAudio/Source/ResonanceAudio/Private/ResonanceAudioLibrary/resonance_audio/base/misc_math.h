/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_MISC_MATH_H_
#define RESONANCE_AUDIO_BASE_MISC_MATH_H_

#ifndef _USE_MATH_DEFINES
#define _USE_MATH_DEFINES  // Enable MSVC math constants (e.g., M_PI).
#endif                     // _USE_MATH_DEFINES

#include <algorithm>
#include <cmath>
#include <cstdint>
#include <limits>
#include <utility>
#include <vector>

#include "base/integral_types.h"
#include "ResonanceEigenHelper.h"
#include "base/constants_and_types.h"
#include "base/logging.h"

namespace vraudio {

class WorldPosition : public Eigen::Matrix<float, 3, 1, Eigen::DontAlign> {
 public:
  // Inherits all constructors with 1-or-more arguments. Necessary because
  // MSVC12 doesn't support inheriting constructors.
  template <typename Arg1, typename... Args>
  WorldPosition(const Arg1& arg1, Args&&... args)
      : Matrix(arg1, std::forward<Args>(args)...) {}

  // Constructs a zero vector.
  WorldPosition();

  // Returns True if other |WorldPosition| differs by at least |kEpsilonFloat|.
  bool operator!=(const WorldPosition& other) const {
    return std::abs(this->x() - other.x()) > kEpsilonFloat ||
           std::abs(this->y() - other.y()) > kEpsilonFloat ||
           std::abs(this->z() - other.z()) > kEpsilonFloat;
  }
};

class WorldRotation : public Eigen::Quaternion<float, Eigen::DontAlign> {
 public:
  // Inherits all constructors with 1-or-more arguments. Necessary because
  // MSVC12 doesn't support inheriting constructors.
  template <typename Arg1, typename... Args>
  WorldRotation(const Arg1& arg1, Args&&... args)
      : Quaternion(arg1, std::forward<Args>(args)...) {}

  // Constructs an identity rotation.
  WorldRotation();

  // Returns the shortest arc between two |WorldRotation|s in radians.
  float AngularDifferenceRad(const WorldRotation& other) const {
    const Quaternion difference = this->inverse() * other;
    return static_cast<float>(Eigen::AngleAxisf(difference).angle());
  }
};

typedef Eigen::AngleAxis<float> AngleAxisf;

typedef WorldPosition AudioPosition;

typedef WorldRotation AudioRotation;

// Converts |world_position| into an equivalent audio space position.
// The world space follows the typical CG coordinate system convention:
// Positive x points right, positive y points up, negative z points forward.
// The audio space follows the ambiX coordinate system convention that is
// commonly accepted in literature [http://goo.gl/XdYNm9]:
// Positive x points forward, negative y points right, positive z points up.
// Positions in both world space and audio space are in meters.
//
// @param world_position 3D position in world space.
// @param audio_position Output 3D position in audio space.
inline void ConvertAudioFromWorldPosition(const WorldPosition& world_position,
                                          AudioPosition* audio_position) {
  DCHECK(audio_position);
  (*audio_position)(0) = -world_position[2];
  (*audio_position)(1) = -world_position[0];
  (*audio_position)(2) = world_position[1];
}

// Converts |audio_position| into an equivalent world space position.
// The world space follows the typical CG coordinate system convention:
// Positive x points right, positive y points up, negative z points forward.
// The audio space follows the ambiX coordinate system convention that is
// commonly accepted in literature [http://goo.gl/XdYNm9]:
// Positive x points forward, negative y points right, positive z points up.
// Positions in both world space and audio space are in meters.
//
// @param audio_position 3D position in audio space.
// @param world_position Output 3D position in world space.
inline void ConvertWorldFromAudioPosition(const AudioPosition& audio_position,
                                          AudioPosition* world_position) {
  DCHECK(world_position);
  (*world_position)(0) = -audio_position[1];
  (*world_position)(1) = audio_position[2];
  (*world_position)(2) = -audio_position[0];
}

// Converts |world_rotation| into an equivalent audio space rotation.
// The world space follows the typical CG coordinate system convention:
// Positive x points right, positive y points up, negative z points forward.
// The audio space follows the ambiX coordinate system convention that is
// commonly accepted in literature [http://goo.gl/XdYNm9]:
// Positive x points forward, negative y points right, positive z points up.
// Positions in both world space and audio space are in meters.
//
// @param world_rotation 3D rotation in world space.
// @param audio_rotation Output 3D rotation in audio space.
inline void ConvertAudioFromWorldRotation(const WorldRotation& world_rotation,
                                          AudioRotation* audio_rotation) {
  DCHECK(audio_rotation);
  audio_rotation->w() = world_rotation.w();
  audio_rotation->x() = -world_rotation.x();
  audio_rotation->y() = world_rotation.y();
  audio_rotation->z() = -world_rotation.z();
}

// Returns the relative direction vector |from_position| and |to_position| by
// rotating the relative position vector with respect to |from_rotation|.
//
// @param from_position Origin position of the direction.
// @param from_rotation Origin orientation of the direction.
// @param to_position Target position of the direction.
// @param relative_direction Relative direction vector (not normalized).
inline void GetRelativeDirection(const WorldPosition& from_position,
                                 const WorldRotation& from_rotation,
                                 const WorldPosition& to_position,
                                 WorldPosition* relative_direction) {
  DCHECK(relative_direction);
  *relative_direction =
      from_rotation.conjugate() * (to_position - from_position);
}

// Returns the closest relative position in an axis-aligned bounding box to the
// given |relative_position|.
//
// @param position Input position relative to the center of the bounding box.
// @param aabb_dimensions Bounding box dimensions.
// @return aabb bounded position.
inline void GetClosestPositionInAabb(const WorldPosition& relative_position,
                                     const WorldPosition& aabb_dimensions,
                                     WorldPosition* closest_position) {
  DCHECK(closest_position);
  const WorldPosition aabb_offset = 0.5f * aabb_dimensions;
  (*closest_position)[0] =
      std::min(std::max(relative_position[0], -aabb_offset[0]), aabb_offset[0]);
  (*closest_position)[1] =
      std::min(std::max(relative_position[1], -aabb_offset[1]), aabb_offset[1]);
  (*closest_position)[2] =
      std::min(std::max(relative_position[2], -aabb_offset[2]), aabb_offset[2]);
}

// Returns true if given world |position| is in given axis-aligned bounding box.
//
// @param position Position to be tested.
// @param aabb_center Bounding box center.
// @param aabb_dimensions Bounding box dimensions.
// @return True if |position| is within bounding box, false otherwise.
inline bool IsPositionInAabb(const WorldPosition& position,
                             const WorldPosition& aabb_center,
                             const WorldPosition& aabb_dimensions) {
  return std::abs(position[0] - aabb_center[0]) <= 0.5f * aabb_dimensions[0] &&
         std::abs(position[1] - aabb_center[1]) <= 0.5f * aabb_dimensions[1] &&
         std::abs(position[2] - aabb_center[2]) <= 0.5f * aabb_dimensions[2];
}

// Returns true if an integer overflow occurred during the calculation of
// x = a * b.
//
// @param a First multiplicand.
// @param b Second multiplicand.
// @param x Product.
// @return True if integer overflow occurred, false otherwise.
template <typename T>
inline bool DoesIntegerMultiplicationOverflow(T a, T b, T x) {
  // Detects an integer overflow occurs by inverting the multiplication and
  // testing for x / a != b.
  return a == 0 ? false : (x / a != b);
}

// Returns true if an integer overflow occurred during the calculation of
// a + b.
//
// @param a First summand.
// @param b Second summand.
// @return True if integer overflow occurred, false otherwise.
template <typename T>
inline bool DoesIntegerAdditionOverflow(T a, T b) {
  T x = a + b;
  return x < b;
}

// Safely converts an int to a size_t.
//
// @param i Integer input.
// @param x Size_t output.
// @return True if integer overflow occurred, false otherwise.
inline bool DoesIntSafelyConvertToSizeT(int i, size_t* x) {
  if (i < 0) {
    return false;
  }
  *x = static_cast<size_t>(i);
  return true;
}

// Safely converts a size_t to an int.
//
// @param i Size_t input.
// @param x Integer output.
// @return True if integer overflow occurred, false otherwise.
inline bool DoesSizeTSafelyConvertToInt(size_t i, int* x) {
  if (i > static_cast<size_t>(std::numeric_limits<int>::max())) {
    return false;
  }
  *x = static_cast<int>(i);
  return true;
}

// Finds the greatest common divisor between two integer values using the
// Euclidean algorithm. Always returns a positive integer.
//
// @param a First of the two integer values.
// @param b second of the two integer values.
// @return The greatest common divisor of the two integer values.
inline int FindGcd(int a, int b) {
  a = std::abs(a);
  b = std::abs(b);
  int temp_value = 0;
  while (b != 0) {
    temp_value = b;
    b = a % b;
    a = temp_value;
  }
  return a;
}

// Finds the next power of two from an integer. This method works with values
// representable by unsigned 32 bit integers.
//
// @param input Integer value.
// @return The next power of two from |input|.
inline size_t NextPowTwo(size_t input) {
  // Ensure the value fits in a uint32_t.
  DCHECK_LT(static_cast<uint64_t>(input),
            static_cast<uint64_t>(std::numeric_limits<uint32_t>::max()));
  uint32_t number = static_cast<uint32_t>(--input);
  number |= number >> 1;   // Take care of 2 bit numbers.
  number |= number >> 2;   // Take care of 4 bit numbers.
  number |= number >> 4;   // Take care of 8 bit numbers.
  number |= number >> 8;   // Take care of 16 bit numbers.
  number |= number >> 16;  // Take care of 32 bit numbers.
  number++;
  return static_cast<size_t>(number);
}

// Returns the factorial (!) of x. If x < 0, it returns 0.
inline float Factorial(int x) {
  if (x < 0) return 0.0f;
  float result = 1.0f;
  for (; x > 0; --x) result *= static_cast<float>(x);
  return result;
}

// Returns the double factorial (!!) of x.
// For odd x:  1 * 3 * 5 * ... * (x - 2) * x
// For even x: 2 * 4 * 6 * ... * (x - 2) * x
// If x < 0, it returns 0.
inline float DoubleFactorial(int x) {
  if (x < 0) return 0.0f;
  float result = 1.0f;
  for (; x > 0; x -= 2) result *= static_cast<float>(x);
  return result;
}

// This is a *safe* alternative to std::equal function as a workaround in order
// to avoid MSVC compiler warning C4996 for unchecked iterators (see
// https://msdn.microsoft.com/en-us/library/aa985965.aspx).
// Also note that, an STL equivalent of this function was introduced in C++14 to
// be replaced with this implementation (see version (5) in
// http://en.cppreference.com/w/cpp/algorithm/equal).
template <typename Iterator>
inline bool EqualSafe(const Iterator& lhs_begin, const Iterator& lhs_end,
                      const Iterator& rhs_begin, const Iterator& rhs_end) {
  auto lhs_itr = lhs_begin;
  auto rhs_itr = rhs_begin;
  while (lhs_itr != lhs_end && rhs_itr != rhs_end) {
    if (*lhs_itr != *rhs_itr) {
      return false;
    }
    ++lhs_itr;
    ++rhs_itr;
  }
  return lhs_itr == lhs_end && rhs_itr == rhs_end;
}

// Fast reciprocal of square-root. See: https://goo.gl/fqvstz for details.
//
// @param input The number to be inverse rooted.
// @return An approximation of the reciprocal square root of |input|.
inline float FastReciprocalSqrt(float input) {
  const float kThreeHalfs = 1.5f;
  const uint32_t kMagicNumber = 0x5f3759df;

  // Approximate a logarithm by aliasing to an integer.
  uint32_t integer = *reinterpret_cast<uint32_t*>(&input);
  integer = kMagicNumber - (integer >> 1);
  float approximation = *reinterpret_cast<float*>(&integer);
  const float half_input = input * 0.5f;
  // One iteration of Newton's method.
  return approximation *
         (kThreeHalfs - (half_input * approximation * approximation));
}

// Finds the best-fitting line to a given set of 2D points by minimizing the
// sum of the squares of the vertical (along y-axis) offsets. The slope and
// intercept of the fitted line are recorded, as well as the coefficient of
// determination, which gives the quality of the fitting.
// See http://mathworld.wolfram.com/LeastSquaresFitting.html for how to compute
// these values.
//
// @param x_array Array of the x coordinates of the points.
// @param y_array Array of the y coordinates of the points.
// @param slope Output slope of the fitted line.
// @param intercept Output slope of the fitted line.
// @param r_squared Coefficient of determination.
// @return False if the fitting fails.
bool LinearLeastSquareFitting(const std::vector<float>& x_array,
                              const std::vector<float>& y_array, float* slope,
                              float* intercept, float* r_squared);

// Computes |base|^|exp|, where |exp| is a *non-negative* integer, with the
// squared exponentiation (a.k.a double-and-add) method.
// When T is a floating point type, this has the same semantics as pow(), but
// is much faster.
// T can also be any integral type, in which case computations will be
// performed in the value domain of this integral type, and overflow semantics
// will be those of T.
// You can also use any type for which operator*= is defined.
// See :

// This method is reproduced here so vraudio classes don't need to depend on
// //util/math/mathutil.h
//
// @tparam base Input to the exponent function. Any type for which *= is
//     defined.
// @param exp Integer exponent, must be greater than or equal to zero.
// @return |base|^|exp|.
template <typename T>
static inline T IntegerPow(T base, int exp) {
  DCHECK_GE(exp, 0);
  T result = static_cast<T>(1);
  while (true) {
    if (exp & 1) {
      result *= base;
    }
    exp >>= 1;
    if (!exp) break;
    base *= base;
  }
  return result;
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_MISC_MATH_H_
