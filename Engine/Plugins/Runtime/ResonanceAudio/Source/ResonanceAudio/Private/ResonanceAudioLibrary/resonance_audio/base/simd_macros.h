/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_SIMD_MACROS_H_
#define RESONANCE_AUDIO_BASE_SIMD_MACROS_H_

#if !defined(DISABLE_SIMD) && (defined(__x86_64__) || defined(_M_X64) || \
                               defined(i386) || defined(_M_IX86))
// SSE1 is enabled.
#include <xmmintrin.h>
typedef __m128 SimdVector;
#define SIMD_SSE
#define SIMD_LENGTH 4
#define SIMD_MULTIPLY(a, b) _mm_mul_ps(a, b)
#define SIMD_ADD(a, b) _mm_add_ps(a, b)
#define SIMD_SUB(a, b) _mm_sub_ps(a, b)
#define SIMD_MULTIPLY_ADD(a, b, c) _mm_add_ps(_mm_mul_ps(a, b), c)
#define SIMD_SQRT(a) _mm_rcp_ps(_mm_rsqrt_ps(a))
#define SIMD_RECIPROCAL_SQRT(a) _mm_rsqrt_ps(a)
#define SIMD_LOAD_ONE_FLOAT(p) _mm_set1_ps(p)
#elif !defined(DISABLE_SIMD) && \
    (defined(__aarch64__) || (defined(__arm__) && defined(__ARM_NEON__)))
// ARM NEON is enabled.
#include <arm_neon.h>
typedef float32x4_t SimdVector;
#define SIMD_NEON
#define SIMD_LENGTH 4
#define SIMD_MULTIPLY(a, b) vmulq_f32(a, b)
#define SIMD_ADD(a, b) vaddq_f32(a, b)
#define SIMD_SUB(a, b) vsubq_f32(a, b)
#define SIMD_MULTIPLY_ADD(a, b, c) vmlaq_f32(c, a, b)
#define SIMD_SQRT(a) vrecpeq_f32(vrsqrteq_f32(a))
#define SIMD_RECIPROCAL_SQRT(a) vrsqrteq_f32(a)
#define SIMD_LOAD_ONE_FLOAT(p) vld1q_dup_f32(&(p))
#else
// No SIMD optimizations enabled.
#include "base/misc_math.h"
typedef float SimdVector;
#define SIMD_DISABLED
#define SIMD_LENGTH 1
#define SIMD_MULTIPLY(a, b) ((a) * (b))
#define SIMD_ADD(a, b) ((a) + (b))
#define SIMD_SUB(a, b) ((a) - (b))
#define SIMD_MULTIPLY_ADD(a, b, c) ((a) * (b) + (c))
#define SIMD_SQRT(a) (1.0f / FastReciprocalSqrt(a))
#define SIMD_RECIPROCAL_SQRT(a) FastReciprocalSqrt(a)
#define SIMD_LOAD_ONE_FLOAT(p) (p)
#endif

#endif  // RESONANCE_AUDIO_BASE_SIMD_MACROS_H_
