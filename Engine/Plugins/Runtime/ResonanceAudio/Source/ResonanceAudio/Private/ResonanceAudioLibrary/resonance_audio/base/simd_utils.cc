/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "base/simd_utils.h"

#include <algorithm>
#include <limits>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"
#include "base/simd_macros.h"

#if defined(__clang__)
_Pragma("clang diagnostic push")

// suppress clang warnings here.

#else
__pragma(warning(push))
__pragma(warning(disable: 4100))
#endif

namespace vraudio {

namespace {

#ifdef SIMD_NEON
// Deinterleaving operates on 8 int16s at a time.
const size_t kSixteenBitSimdLength = SIMD_LENGTH * 2;
#endif  // SIMD_NEON

// Float format of max and min values storable in an int16_t, for clamping.
const float kInt16Max = static_cast<float>(0x7FFF);
const float kInt16Min = static_cast<float>(-0x7FFF);

// Conversion factors between float and int16_t (both directions).
const float kFloatFromInt16 = 1.0f / kInt16Max;
const float kInt16FromFloat = kInt16Max;

// Expected SIMD alignment in bytes.
const size_t kSimdSizeBytes = 16;

inline size_t GetNumChunks(size_t length) { return length / SIMD_LENGTH; }

inline size_t GetLeftoverSamples(size_t length) { return length % SIMD_LENGTH; }

template <typename T>
inline bool IsAlignedTemplated(const T* pointer) {
  return reinterpret_cast<uintptr_t>(pointer) % kSimdSizeBytes == 0;
}

#ifdef SIMD_DISABLED
// Calculates the approximate complex magnude of z = real + i * imaginary.
inline void ComplexMagnitude(float real, float imaginary, float* output) {
  *output = real * real + imaginary * imaginary;
  // The value of |output| is not being recalculated, simply modified.
  *output = 1.0f / FastReciprocalSqrt(*output);
}
#endif  // defined(SIMD_DISABLED)

}  // namespace

bool IsAligned(const float* pointer) {
  return IsAlignedTemplated<float>(pointer);
}

bool IsAligned(const int16_t* pointer) {
  return IsAlignedTemplated<int16_t>(pointer);
}

size_t FindNextAlignedArrayIndex(size_t length, size_t type_size_bytes,
                                 size_t memory_alignment_bytes) {
  const size_t byte_length = type_size_bytes * length;
  const size_t unaligned_bytes = byte_length % memory_alignment_bytes;
  const size_t bytes_to_next_aligned =
      (unaligned_bytes == 0) ? 0 : memory_alignment_bytes - unaligned_bytes;
  return (byte_length + bytes_to_next_aligned) / type_size_bytes;
}

void AddPointwise(size_t length, const float* input_a, const float* input_b,
                  float* output) {
  DCHECK(input_a);
  DCHECK(input_b);
  DCHECK(output);

  const SimdVector* input_a_vector =
      reinterpret_cast<const SimdVector*>(input_a);
  const SimdVector* input_b_vector =
      reinterpret_cast<const SimdVector*>(input_b);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);
#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool inputs_aligned = IsAligned(input_a) && IsAligned(input_b);
  const bool output_aligned = IsAligned(output);
  if (inputs_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_ADD(input_a_vector[i], input_b_vector[i]);
    }
  } else if (inputs_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp =
          SIMD_ADD(input_a_vector[i], input_b_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_load_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_load_ps(&input_b[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_ADD(input_a_temp, input_b_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_load_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_load_ps(&input_b[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_ADD(input_a_temp, input_b_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    output_vector[i] = SIMD_ADD(input_a_vector[i], input_b_vector[i]);
  }
#endif  // SIMD_SSE

  // Add samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = input_a[i] + input_b[i];
  }
}

void SubtractPointwise(size_t length, const float* input_a,
                       const float* input_b, float* output) {
  DCHECK(input_a);
  DCHECK(input_b);
  DCHECK(output);

  const SimdVector* input_a_vector =
      reinterpret_cast<const SimdVector*>(input_a);
  const SimdVector* input_b_vector =
      reinterpret_cast<const SimdVector*>(input_b);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);

#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool inputs_aligned = IsAligned(input_a) && IsAligned(input_b);
  const bool output_aligned = IsAligned(output);
  if (inputs_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_SUB(input_b_vector[i], input_a_vector[i]);
    }
  } else if (inputs_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp =
          SIMD_SUB(input_b_vector[i], input_a_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_load_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_load_ps(&input_b[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_SUB(input_b_temp, input_a_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_load_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_load_ps(&input_b[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_SUB(input_b_temp, input_a_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    output_vector[i] = SIMD_SUB(input_b_vector[i], input_a_vector[i]);
  }
#endif  // SIMD_SSE

  // Subtract samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = input_b[i] - input_a[i];
  }
}

void MultiplyPointwise(size_t length, const float* input_a,
                       const float* input_b, float* output) {
  DCHECK(input_a);
  DCHECK(input_b);
  DCHECK(output);

  const SimdVector* input_a_vector =
      reinterpret_cast<const SimdVector*>(input_a);
  const SimdVector* input_b_vector =
      reinterpret_cast<const SimdVector*>(input_b);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);

#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool inputs_aligned = IsAligned(input_a) && IsAligned(input_b);
  const bool output_aligned = IsAligned(output);
  if (inputs_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_MULTIPLY(input_a_vector[i], input_b_vector[i]);
    }
  } else if (inputs_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp =
          SIMD_MULTIPLY(input_a_vector[i], input_b_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_loadu_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_loadu_ps(&input_b[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_MULTIPLY(input_a_temp, input_b_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_loadu_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_loadu_ps(&input_b[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_MULTIPLY(input_a_temp, input_b_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    output_vector[i] = SIMD_MULTIPLY(input_a_vector[i], input_b_vector[i]);
  }
#endif  // SIMD_SSE

  // Multiply samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = input_a[i] * input_b[i];
  }
}

void MultiplyAndAccumulatePointwise(size_t length, const float* input_a,
                                    const float* input_b, float* accumulator) {
  DCHECK(input_a);
  DCHECK(input_b);
  DCHECK(accumulator);

  const SimdVector* input_a_vector =
      reinterpret_cast<const SimdVector*>(input_a);
  const SimdVector* input_b_vector =
      reinterpret_cast<const SimdVector*>(input_b);
  SimdVector* accumulator_vector = reinterpret_cast<SimdVector*>(accumulator);

#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool inputs_aligned = IsAligned(input_a) && IsAligned(input_b);
  const bool accumulator_aligned = IsAligned(accumulator);
  if (inputs_aligned && accumulator_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      accumulator_vector[i] = SIMD_MULTIPLY_ADD(
          input_a_vector[i], input_b_vector[i], accumulator_vector[i]);
    }
  } else if (inputs_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      SimdVector accumulator_temp = _mm_loadu_ps(&accumulator[i * SIMD_LENGTH]);
      accumulator_temp = SIMD_MULTIPLY_ADD(input_a_vector[i], input_b_vector[i],
                                           accumulator_temp);
      _mm_storeu_ps(&accumulator[i * SIMD_LENGTH], accumulator_temp);
    }
  } else if (accumulator_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_loadu_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_loadu_ps(&input_b[i * SIMD_LENGTH]);
      accumulator_vector[i] =
          SIMD_MULTIPLY_ADD(input_a_temp, input_b_temp, accumulator_vector[i]);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_a_temp = _mm_loadu_ps(&input_a[i * SIMD_LENGTH]);
      const SimdVector input_b_temp = _mm_loadu_ps(&input_b[i * SIMD_LENGTH]);
      SimdVector accumulator_temp = _mm_loadu_ps(&accumulator[i * SIMD_LENGTH]);
      accumulator_temp =
          SIMD_MULTIPLY_ADD(input_a_temp, input_b_temp, accumulator_temp);
      _mm_storeu_ps(&accumulator[i * SIMD_LENGTH], accumulator_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    accumulator_vector[i] = SIMD_MULTIPLY_ADD(
        input_a_vector[i], input_b_vector[i], accumulator_vector[i]);
  }
#endif  // SIMD_SSE

  // Apply gain and accumulate to samples at the end that were missed by the
  // SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    accumulator[i] += input_a[i] * input_b[i];
  }
}

void ScalarMultiply(size_t length, float gain, const float* input,
                    float* output) {
  DCHECK(input);
  DCHECK(output);

  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);

  const SimdVector gain_vector = SIMD_LOAD_ONE_FLOAT(gain);
#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (input_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_MULTIPLY(gain_vector, input_vector[i]);
    }
  } else if (input_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp =
          SIMD_MULTIPLY(gain_vector, input_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_MULTIPLY(gain_vector, input_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_MULTIPLY(gain_vector, input_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    output_vector[i] = SIMD_MULTIPLY(gain_vector, input_vector[i]);
  }
#endif  // SIMD_SSE

  // Apply gain to samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = input[i] * gain;
  }
}

void ScalarMultiplyAndAccumulate(size_t length, float gain, const float* input,
                                 float* accumulator) {
  DCHECK(input);
  DCHECK(accumulator);

  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  SimdVector* accumulator_vector = reinterpret_cast<SimdVector*>(accumulator);

  const SimdVector gain_vector = SIMD_LOAD_ONE_FLOAT(gain);
#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool input_aligned = IsAligned(input);
  const bool accumulator_aligned = IsAligned(accumulator);
  if (input_aligned && accumulator_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      accumulator_vector[i] = SIMD_MULTIPLY_ADD(gain_vector, input_vector[i],
                                                accumulator_vector[i]);
    }
  } else if (input_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      SimdVector accumulator_temp = _mm_loadu_ps(&accumulator[i * SIMD_LENGTH]);
      accumulator_temp =
          SIMD_MULTIPLY_ADD(gain_vector, input_vector[i], accumulator_temp);
      _mm_storeu_ps(&accumulator[i * SIMD_LENGTH], accumulator_temp);
    }
  } else if (accumulator_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      accumulator_vector[i] =
          SIMD_MULTIPLY_ADD(gain_vector, input_temp, accumulator_vector[i]);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      SimdVector accumulator_temp = _mm_loadu_ps(&accumulator[i * SIMD_LENGTH]);
      accumulator_temp =
          SIMD_MULTIPLY_ADD(gain_vector, input_temp, accumulator_temp);
      _mm_storeu_ps(&accumulator[i * SIMD_LENGTH], accumulator_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    accumulator_vector[i] =
        SIMD_MULTIPLY_ADD(gain_vector, input_vector[i], accumulator_vector[i]);
  }
#endif  // SIMD_SSE

  // Apply gain and accumulate to samples at the end that were missed by the
  // SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    accumulator[i] += input[i] * gain;
  }
}

void ReciprocalSqrt(size_t length, const float* input, float* output) {
  DCHECK(input);
  DCHECK(output);

#if !defined(SIMD_DISABLED)
  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);
#endif  // !defined(SIMD_DISABLED)

#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (input_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_RECIPROCAL_SQRT(input_vector[i]);
    }
  } else if (input_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp = SIMD_RECIPROCAL_SQRT(input_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_RECIPROCAL_SQRT(input_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_RECIPROCAL_SQRT(input_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#elif defined SIMD_NEON
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    output_vector[i] = SIMD_RECIPROCAL_SQRT(input_vector[i]);
  }
#endif  // SIMD_SSE

  // Apply to samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = FastReciprocalSqrt(input[i]);
  }
}

void Sqrt(size_t length, const float* input, float* output) {
  DCHECK(input);
  DCHECK(output);

#if !defined(SIMD_DISABLED)
  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);
#endif  // !defined(SIMD_DISABLED)

#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (input_aligned && output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      output_vector[i] = SIMD_SQRT(input_vector[i]);
    }
  } else if (input_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector output_temp = SIMD_SQRT(input_vector[i]);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      output_vector[i] = SIMD_SQRT(input_temp);
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector input_temp = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
      const SimdVector output_temp = SIMD_SQRT(input_temp);
      _mm_storeu_ps(&output[i * SIMD_LENGTH], output_temp);
    }
  }
#elif defined SIMD_NEON
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    // This should be faster than using a sqrt method : https://goo.gl/XRKwFp
    output_vector[i] = SIMD_SQRT(input_vector[i]);
  }
#endif  // SIMD_SSE

  // Apply to samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = 1.0f / FastReciprocalSqrt(input[i]);
  }
}

void ApproxComplexMagnitude(size_t length, const float* input, float* output) {
  DCHECK(input);
  DCHECK(output);

#if !defined(SIMD_DISABLED)
  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);
  const size_t num_chunks = GetNumChunks(length);
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
#endif  // !defined(SIMD_DISABLED)

#ifdef SIMD_SSE
  if (input_aligned && output_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector squared_1 =
          SIMD_MULTIPLY(input_vector[first_index], input_vector[first_index]);
      const SimdVector squared_2 =
          SIMD_MULTIPLY(input_vector[second_index], input_vector[second_index]);
      const SimdVector unshuffled_1 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(2, 0, 2, 0));
      const SimdVector unshuffled_2 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(3, 1, 3, 1));
      output_vector[out_index] = SIMD_ADD(unshuffled_1, unshuffled_2);
      output_vector[out_index] = SIMD_SQRT(output_vector[out_index]);
    }
  } else if (input_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector squared_1 =
          SIMD_MULTIPLY(input_vector[first_index], input_vector[first_index]);
      const SimdVector squared_2 =
          SIMD_MULTIPLY(input_vector[second_index], input_vector[second_index]);
      const SimdVector unshuffled_1 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(2, 0, 2, 0));
      const SimdVector unshuffled_2 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(3, 1, 3, 1));
      SimdVector output_temp = SIMD_ADD(unshuffled_1, unshuffled_2);
      output_vector[out_index] = SIMD_SQRT(output_temp);
      _mm_storeu_ps(&output[out_index * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector first_temp =
          _mm_loadu_ps(&input[first_index * SIMD_LENGTH]);
      const SimdVector second_temp =
          _mm_loadu_ps(&input[second_index * SIMD_LENGTH]);
      const SimdVector squared_1 = SIMD_MULTIPLY(first_temp, first_temp);
      const SimdVector squared_2 = SIMD_MULTIPLY(second_temp, second_temp);
      const SimdVector unshuffled_1 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(2, 0, 2, 0));
      const SimdVector unshuffled_2 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(3, 1, 3, 1));
      output_vector[out_index] = SIMD_ADD(unshuffled_1, unshuffled_2);
      output_vector[out_index] = SIMD_SQRT(output_vector[out_index]);
    }
  } else {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector first_temp =
          _mm_loadu_ps(&input[first_index * SIMD_LENGTH]);
      const SimdVector second_temp =
          _mm_loadu_ps(&input[second_index * SIMD_LENGTH]);
      const SimdVector squared_1 = SIMD_MULTIPLY(first_temp, first_temp);
      const SimdVector squared_2 = SIMD_MULTIPLY(second_temp, second_temp);
      const SimdVector unshuffled_1 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(2, 0, 2, 0));
      const SimdVector unshuffled_2 =
          _mm_shuffle_ps(squared_1, squared_2, _MM_SHUFFLE(3, 1, 3, 1));
      SimdVector output_temp = SIMD_ADD(unshuffled_1, unshuffled_2);
      output_temp = SIMD_SQRT(output_temp);
      _mm_storeu_ps(&output[out_index * SIMD_LENGTH], output_temp);
    }
  }
#elif defined SIMD_NEON
  if (input_aligned && output_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector squared_1 =
          SIMD_MULTIPLY(input_vector[first_index], input_vector[first_index]);
      const SimdVector squared_2 =
          SIMD_MULTIPLY(input_vector[second_index], input_vector[second_index]);
      const float32x4x2_t unshuffled = vuzpq_f32(squared_1, squared_2);
      output_vector[out_index] = SIMD_ADD(unshuffled.val[0], unshuffled.val[1]);
      output_vector[out_index] = SIMD_SQRT(output_vector[out_index]);
    }
  } else if (input_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector squared_1 =
          SIMD_MULTIPLY(input_vector[first_index], input_vector[first_index]);
      const SimdVector squared_2 =
          SIMD_MULTIPLY(input_vector[second_index], input_vector[second_index]);
      const float32x4x2_t unshuffled = vuzpq_f32(squared_1, squared_2);
      SimdVector output_temp = SIMD_ADD(unshuffled.val[0], unshuffled.val[1]);
      output_temp = SIMD_SQRT(output_temp);
      vst1q_f32(&output[out_index * SIMD_LENGTH], output_temp);
    }
  } else if (output_aligned) {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector first_temp =
          vld1q_f32(&input[first_index * SIMD_LENGTH]);
      const SimdVector second_temp =
          vld1q_f32(&input[second_index * SIMD_LENGTH]);
      const SimdVector squared_1 = SIMD_MULTIPLY(first_temp, first_temp);
      const SimdVector squared_2 = SIMD_MULTIPLY(second_temp, second_temp);
      const float32x4x2_t unshuffled = vuzpq_f32(squared_1, squared_2);
      output_vector[out_index] = SIMD_ADD(unshuffled.val[0], unshuffled.val[1]);
      output_vector[out_index] = SIMD_SQRT(output_vector[out_index]);
    }
  } else {
    for (size_t out_index = 0; out_index < num_chunks; ++out_index) {
      const size_t first_index = out_index * 2;
      const size_t second_index = first_index + 1;
      const SimdVector first_temp =
          vld1q_f32(&input[first_index * SIMD_LENGTH]);
      const SimdVector second_temp =
          vld1q_f32(&input[second_index * SIMD_LENGTH]);
      const SimdVector squared_1 = SIMD_MULTIPLY(first_temp, first_temp);
      const SimdVector squared_2 = SIMD_MULTIPLY(second_temp, second_temp);
      const float32x4x2_t unshuffled = vuzpq_f32(squared_1, squared_2);
      SimdVector output_temp = SIMD_ADD(unshuffled.val[0], unshuffled.val[1]);
      output_temp = SIMD_SQRT(output_temp);
      vst1q_f32(&output[out_index * SIMD_LENGTH], output_temp);
    }
  }
#endif  // SIMD_SSE

  // Apply to samples at the end that were missed by the SIMD chunking.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t real_index = i * 2;
    const size_t imag_index = real_index + 1;
    const float squared_sum = (input[real_index] * input[real_index]) +
                              (input[imag_index] * input[imag_index]);
    output[i] = 1.0f / FastReciprocalSqrt(squared_sum);
  }
}

void ComplexInterleavedFormatFromMagnitudeAndSinCosPhase(
    size_t length, const float* magnitude, const float* cos_phase,
    const float* sin_phase, float* complex_interleaved_format_output) {
  size_t leftover_samples = 0;
#ifdef SIMD_NEON
  if (IsAligned(complex_interleaved_format_output) && IsAligned(cos_phase) &&
      IsAligned(sin_phase) && IsAligned(magnitude)) {
    const SimdVector* cos_vec = reinterpret_cast<const SimdVector*>(cos_phase);
    const SimdVector* sin_vec = reinterpret_cast<const SimdVector*>(sin_phase);
    const SimdVector* magnitude_vec =
        reinterpret_cast<const SimdVector*>(magnitude);

    const size_t num_chunks = GetNumChunks(length);
    float32x4x2_t interleaved_pair;

    SimdVector* interleaved_vec =
        reinterpret_cast<SimdVector*>(complex_interleaved_format_output);
    for (size_t i = 0, j = 0; j < num_chunks; ++i, j += 2) {
      interleaved_pair = vzipq_f32(cos_vec[i], sin_vec[i]);
      interleaved_vec[j] =
          SIMD_MULTIPLY(interleaved_pair.val[0], magnitude_vec[i]);
      interleaved_vec[j + 1] =
          SIMD_MULTIPLY(interleaved_pair.val[1], magnitude_vec[i]);
    }

    leftover_samples = GetLeftoverSamples(length);
  }
#endif  // SIMD_NEON
  DCHECK_EQ(leftover_samples % 2U, 0U);
  for (size_t i = leftover_samples, j = leftover_samples / 2; i < length;
       i += 2, ++j) {
    const size_t imaginary_offset = i + 1;
    complex_interleaved_format_output[i] = magnitude[j] * cos_phase[j];
    complex_interleaved_format_output[imaginary_offset] =
        magnitude[j] * sin_phase[j];
  }
}

void StereoFromMonoSimd(size_t length, const float* mono, float* left,
                        float* right) {
  ScalarMultiply(length, kInverseSqrtTwo, mono, left);
  std::copy_n(left, length, right);
}

void MonoFromStereoSimd(size_t length, const float* left, const float* right,
                        float* mono) {
  DCHECK(left);
  DCHECK(right);
  DCHECK(mono);

  const SimdVector* left_vector = reinterpret_cast<const SimdVector*>(left);
  const SimdVector* right_vector = reinterpret_cast<const SimdVector*>(right);
  SimdVector* mono_vector = reinterpret_cast<SimdVector*>(mono);

  const SimdVector inv_root_two_vec = SIMD_LOAD_ONE_FLOAT(kInverseSqrtTwo);
#ifdef SIMD_SSE
  const size_t num_chunks = GetNumChunks(length);
  const bool inputs_aligned = IsAligned(left) && IsAligned(right);
  const bool mono_aligned = IsAligned(mono);
  if (inputs_aligned && mono_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      mono_vector[i] = SIMD_MULTIPLY(inv_root_two_vec,
                                     SIMD_ADD(left_vector[i], right_vector[i]));
    }
  } else if (inputs_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector mono_temp = SIMD_MULTIPLY(
          inv_root_two_vec, SIMD_ADD(left_vector[i], right_vector[i]));
      _mm_storeu_ps(&mono[i * SIMD_LENGTH], mono_temp);
    }
  } else if (mono_aligned) {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector left_temp = _mm_loadu_ps(&left[i * SIMD_LENGTH]);
      const SimdVector right_temp = _mm_loadu_ps(&right[i * SIMD_LENGTH]);
      mono_vector[i] =
          SIMD_MULTIPLY(inv_root_two_vec, SIMD_ADD(left_temp, right_temp));
    }
  } else {
    for (size_t i = 0; i < num_chunks; ++i) {
      const SimdVector left_temp = _mm_loadu_ps(&left[i * SIMD_LENGTH]);
      const SimdVector right_temp = _mm_loadu_ps(&right[i * SIMD_LENGTH]);
      const SimdVector mono_temp =
          SIMD_MULTIPLY(inv_root_two_vec, SIMD_ADD(left_temp, right_temp));
      _mm_storeu_ps(&mono[i * SIMD_LENGTH], mono_temp);
    }
  }
#else
  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    mono_vector[i] = SIMD_MULTIPLY(inv_root_two_vec,
                                   SIMD_ADD(left_vector[i], right_vector[i]));
  }
#endif  // SIMD_SSE
  const size_t leftover_samples = GetLeftoverSamples(length);
  // Downmix samples at the end that were missed by the SIMD chunking.
  DCHECK_GE(length, leftover_samples);
  for (size_t i = length - leftover_samples; i < length; ++i) {
    mono[i] = kInverseSqrtTwo * (left[i] + right[i]);
  }
}

#ifdef SIMD_NEON

void Int16FromFloat(size_t length, const float* input, int16_t* output) {
  DCHECK(input);
  DCHECK(output);

  //  if (input_aligned || output_aligned) {
  const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
  int16x4_t* output_vector = reinterpret_cast<int16x4_t*>(output);

  // A temporary 32 bit integer vector is needed as we only have intrinsics to
  // convert from 32 bit floats to 32 bit ints. Then truncate to 16 bit ints.
  int32x4_t temporary_wide_vector;
  SimdVector temporary_float_vector;

  const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kInt16FromFloat);

  for (size_t i = 0; i < GetNumChunks(length); ++i) {
    temporary_float_vector = SIMD_MULTIPLY(scaling_vector, input_vector[i]);
    temporary_wide_vector = vcvtq_s32_f32(temporary_float_vector);
    output_vector[i] = vqmovn_s32(temporary_wide_vector);
  }

  // The remainder.
  const size_t leftover_samples = GetLeftoverSamples(length);
  DCHECK_GE(length, leftover_samples);
  float temp_float;
  for (size_t i = length - leftover_samples; i < length; ++i) {
    temp_float = input[i] * kInt16FromFloat;
    temp_float = std::min(kInt16Max, std::max(kInt16Min, temp_float));
    output[i] = static_cast<int16_t>(temp_float);
  }
}

void FloatFromInt16(size_t length, const int16_t* input, float* output) {
  DCHECK(input);
  DCHECK(output);

  size_t leftover_samples = length;
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (input_aligned || output_aligned) {
    const int16x4_t* input_vector = reinterpret_cast<const int16x4_t*>(input);
    SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);

    int16x4_t temporary_narrow_vector;
    SimdVector temporary_float_vector;
    int32x4_t temporary_wide_vector;
    const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kFloatFromInt16);

    if (input_aligned && output_aligned) {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_wide_vector = vmovl_s16(input_vector[i]);
        output_vector[i] = vcvtq_f32_s32(temporary_wide_vector);
        output_vector[i] = SIMD_MULTIPLY(scaling_vector, output_vector[i]);
      }
    } else if (input_aligned) {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_wide_vector = vmovl_s16(input_vector[i]);
        temporary_float_vector = vcvtq_f32_s32(temporary_wide_vector);
        temporary_float_vector =
            SIMD_MULTIPLY(scaling_vector, temporary_float_vector);
        vst1q_f32(&output[i * SIMD_LENGTH], temporary_float_vector);
      }
    } else {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_narrow_vector = vld1_s16(&input[i * SIMD_LENGTH]);
        temporary_wide_vector = vmovl_s16(temporary_narrow_vector);
        output_vector[i] = vcvtq_f32_s32(temporary_wide_vector);
        output_vector[i] = SIMD_MULTIPLY(scaling_vector, output_vector[i]);
      }
    }
    leftover_samples = GetLeftoverSamples(length);
  }

  // The remainder.
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = static_cast<float>(input[i]) * kFloatFromInt16;
  }
}

#elif (defined SIMD_SSE && !defined(_MSC_VER))

void Int16FromFloat(size_t length, const float* input, int16_t* output) {
  DCHECK(input);
  DCHECK(output);

  size_t leftover_samples = length;
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (output_aligned) {
    const SimdVector* input_vector = reinterpret_cast<const SimdVector*>(input);
    __m64* output_vector = reinterpret_cast<__m64*>(output);

    const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kInt16FromFloat);
    const SimdVector min_vector = SIMD_LOAD_ONE_FLOAT(kInt16Min);
    const SimdVector max_vector = SIMD_LOAD_ONE_FLOAT(kInt16Max);

    SimdVector temporary_float_vector;

    if (input_aligned) {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_float_vector = SIMD_MULTIPLY(scaling_vector, input_vector[i]);
        temporary_float_vector = _mm_max_ps(temporary_float_vector, min_vector);
        temporary_float_vector = _mm_min_ps(temporary_float_vector, max_vector);
        output_vector[i] = _mm_cvtps_pi16(temporary_float_vector);
      }
    } else {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_float_vector = _mm_loadu_ps(&input[i * SIMD_LENGTH]);
        temporary_float_vector =
            SIMD_MULTIPLY(scaling_vector, temporary_float_vector);
        temporary_float_vector = _mm_max_ps(temporary_float_vector, min_vector);
        temporary_float_vector = _mm_min_ps(temporary_float_vector, max_vector);
        output_vector[i] = _mm_cvtps_pi16(temporary_float_vector);
      }
    }
    // There is no easy way to simply store the 16 bit ints so we dont have an
    // |input_aligned| only case.
    leftover_samples = GetLeftoverSamples(length);
  }

  // The remainder.
  float temp_float;
  for (size_t i = length - GetLeftoverSamples(length); i < length; ++i) {
    temp_float = input[i] * kInt16FromFloat;
    temp_float = std::min(kInt16Max, std::max(kInt16Min, temp_float));
    output[i] = static_cast<int16_t>(temp_float);
  }
}

void FloatFromInt16(size_t length, const int16_t* input, float* output) {
  DCHECK(input);
  DCHECK(output);

  size_t leftover_samples = length;
  const bool input_aligned = IsAligned(input);
  const bool output_aligned = IsAligned(output);
  if (input_aligned) {
    SimdVector* output_vector = reinterpret_cast<SimdVector*>(output);
    const __m64* input_vector = reinterpret_cast<const __m64*>(input);

    const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kFloatFromInt16);

    if (output_aligned) {
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        output_vector[i] = _mm_cvtpi16_ps(input_vector[i]);
        output_vector[i] = SIMD_MULTIPLY(scaling_vector, output_vector[i]);
      }
    } else {
      SimdVector temporary_float_vector;
      for (size_t i = 0; i < GetNumChunks(length); ++i) {
        temporary_float_vector = _mm_cvtpi16_ps(input_vector[i]);
        temporary_float_vector =
            SIMD_MULTIPLY(scaling_vector, temporary_float_vector);
        _mm_storeu_ps(&output[i * SIMD_LENGTH], temporary_float_vector);
      }
    }
    // There is no easy way to simply load the 16 bit ints so we dont have an
    // |output_aligned| only case.
    leftover_samples = GetLeftoverSamples(length);
  }

  // The remainder.
  for (size_t i = length - leftover_samples; i < length; ++i) {
    output[i] = static_cast<float>(input[i]) * kFloatFromInt16;
  }
}

#else  // SIMD disabled or Windows build.

void Int16FromFloat(size_t length, const float* input, int16_t* output) {
  DCHECK(input);
  DCHECK(output);

  float temp_float;
  for (size_t i = 0; i < length; ++i) {
    temp_float = input[i] * kInt16FromFloat;
    temp_float = std::min(kInt16Max, std::max(kInt16Min, temp_float));
    output[i] = static_cast<int16_t>(temp_float);
  }
}

void FloatFromInt16(size_t length, const int16_t* input, float* output) {
  DCHECK(input);
  DCHECK(output);

  for (size_t i = 0; i < length; ++i) {
    output[i] = static_cast<float>(input[i]) * kFloatFromInt16;
  }
}

#endif  // SIMD_NEON

void InterleaveStereo(size_t length, const int16_t* channel_0,
                      const int16_t* channel_1, int16_t* interleaved_buffer) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const int16x8_t* channel_0_vec =
        reinterpret_cast<const int16x8_t*>(channel_0);
    const int16x8_t* channel_1_vec =
        reinterpret_cast<const int16x8_t*>(channel_1);

    const size_t num_chunks = length / kSixteenBitSimdLength;
    int16x8x2_t interleaved_pair;

    int16x8_t* interleaved_vec =
        reinterpret_cast<int16x8_t*>(interleaved_buffer);
    for (size_t i = 0, j = 0; i < num_chunks; ++i, j += 2) {
      interleaved_pair = vzipq_s16(channel_0_vec[i], channel_1_vec[i]);
      interleaved_vec[j] = interleaved_pair.val[0];
      interleaved_vec[j + 1] = interleaved_pair.val[1];
    }

    leftover_samples = length % kSixteenBitSimdLength;
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    interleaved_buffer[interleaved_index] = channel_0[i];
    interleaved_buffer[interleaved_index + 1] = channel_1[i];
  }
}

void InterleaveStereo(size_t length, const float* channel_0,
                      const float* channel_1, float* interleaved_buffer) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const SimdVector* channel_0_vec =
        reinterpret_cast<const SimdVector*>(channel_0);
    const SimdVector* channel_1_vec =
        reinterpret_cast<const SimdVector*>(channel_1);

    const size_t num_chunks = GetNumChunks(length);
    float32x4x2_t interleaved_pair;

    SimdVector* interleaved_vec =
        reinterpret_cast<SimdVector*>(interleaved_buffer);
    for (size_t i = 0, j = 0; i < num_chunks; ++i, j += 2) {
      interleaved_pair = vzipq_f32(channel_0_vec[i], channel_1_vec[i]);
      interleaved_vec[j] = interleaved_pair.val[0];
      interleaved_vec[j + 1] = interleaved_pair.val[1];
    }

    leftover_samples = GetLeftoverSamples(length);
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    interleaved_buffer[interleaved_index] = channel_0[i];
    interleaved_buffer[interleaved_index + 1] = channel_1[i];
  }
}

void InterleaveStereo(size_t length, const float* channel_0,
                      const float* channel_1, int16_t* interleaved_buffer) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const SimdVector* channel_0_vec =
        reinterpret_cast<const SimdVector*>(channel_0);
    const SimdVector* channel_1_vec =
        reinterpret_cast<const SimdVector*>(channel_1);

    const size_t num_chunks = GetNumChunks(length);
    float32x4x2_t interleaved_pair;
    int32x4_t temporary_wide_vector;

    const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kInt16FromFloat);
    const SimdVector min_vector = SIMD_LOAD_ONE_FLOAT(kInt16Min);
    const SimdVector max_vector = SIMD_LOAD_ONE_FLOAT(kInt16Max);

    int16x4_t* interleaved_vec =
        reinterpret_cast<int16x4_t*>(interleaved_buffer);
    for (size_t i = 0; i < num_chunks; ++i) {
      const size_t interleaved_index = kNumStereoChannels * i;
      interleaved_pair = vzipq_f32(channel_0_vec[i], channel_1_vec[i]);
      interleaved_pair.val[0] =
          SIMD_MULTIPLY(scaling_vector, interleaved_pair.val[0]);
      interleaved_pair.val[0] = vmaxq_f32(interleaved_pair.val[0], min_vector);
      interleaved_pair.val[0] = vminq_f32(interleaved_pair.val[0], max_vector);
      temporary_wide_vector = vcvtq_s32_f32(interleaved_pair.val[0]);
      interleaved_vec[interleaved_index] = vqmovn_s32(temporary_wide_vector);
      interleaved_pair.val[1] =
          SIMD_MULTIPLY(scaling_vector, interleaved_pair.val[1]);
      interleaved_pair.val[1] = vmaxq_f32(interleaved_pair.val[1], min_vector);
      interleaved_pair.val[1] = vminq_f32(interleaved_pair.val[1], max_vector);
      temporary_wide_vector = vcvtq_s32_f32(interleaved_pair.val[1]);
      interleaved_vec[interleaved_index + 1] =
          vqmovn_s32(temporary_wide_vector);
    }

    leftover_samples = GetLeftoverSamples(length);
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    interleaved_buffer[interleaved_index] = static_cast<int16_t>(std::max(
        kInt16Min, std::min(kInt16Max, kInt16FromFloat * channel_0[i])));
    interleaved_buffer[interleaved_index + 1] = static_cast<int16_t>(std::max(
        kInt16Min, std::min(kInt16Max, kInt16FromFloat * channel_1[i])));
  }
}

void DeinterleaveStereo(size_t length, const int16_t* interleaved_buffer,
                        int16_t* channel_0, int16_t* channel_1) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const size_t num_chunks = length / kSixteenBitSimdLength;
    leftover_samples = length % kSixteenBitSimdLength;
    int16x8_t* channel_0_vec = reinterpret_cast<int16x8_t*>(channel_0);
    int16x8_t* channel_1_vec = reinterpret_cast<int16x8_t*>(channel_1);
    int16x8x2_t deinterleaved_pair;
    const int16x8_t* interleaved_vec =
        reinterpret_cast<const int16x8_t*>(interleaved_buffer);
    for (size_t chunk = 0; chunk < num_chunks; ++chunk) {
      const size_t interleaved_index = chunk * kNumStereoChannels;
      deinterleaved_pair = vuzpq_s16(interleaved_vec[interleaved_index],
                                     interleaved_vec[interleaved_index + 1]);
      channel_0_vec[chunk] = deinterleaved_pair.val[0];
      channel_1_vec[chunk] = deinterleaved_pair.val[1];
    }
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    channel_0[i] = interleaved_buffer[interleaved_index];
    channel_1[i] = interleaved_buffer[interleaved_index + 1];
  }
}

void DeinterleaveStereo(size_t length, const float* interleaved_buffer,
                        float* channel_0, float* channel_1) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const size_t num_chunks = GetNumChunks(length);
    leftover_samples = GetLeftoverSamples(length);
    SimdVector* channel_0_vec = reinterpret_cast<SimdVector*>(channel_0);
    SimdVector* channel_1_vec = reinterpret_cast<SimdVector*>(channel_1);
    float32x4x2_t deinterleaved_pair;

    const SimdVector* interleaved_vec =
        reinterpret_cast<const SimdVector*>(interleaved_buffer);
    for (size_t chunk = 0; chunk < num_chunks; ++chunk) {
      const size_t interleaved_index = chunk * kNumStereoChannels;
      deinterleaved_pair = vuzpq_f32(interleaved_vec[interleaved_index],
                                     interleaved_vec[interleaved_index + 1]);
      channel_0_vec[chunk] = deinterleaved_pair.val[0];
      channel_1_vec[chunk] = deinterleaved_pair.val[1];
    }
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    channel_0[i] = interleaved_buffer[interleaved_index];
    channel_1[i] = interleaved_buffer[interleaved_index + 1];
  }
}

void DeinterleaveStereo(size_t length, const int16_t* interleaved_buffer,
                        float* channel_0, float* channel_1) {
  DCHECK(interleaved_buffer);
  DCHECK(channel_0);
  DCHECK(channel_1);

  size_t leftover_samples = length;
#ifdef SIMD_NEON
  if (IsAligned(interleaved_buffer) && IsAligned(channel_0) &&
      IsAligned(channel_1)) {
    const size_t num_chunks = GetNumChunks(length);
    leftover_samples = GetLeftoverSamples(length);
    SimdVector* channel_0_vec = reinterpret_cast<SimdVector*>(channel_0);
    SimdVector* channel_1_vec = reinterpret_cast<SimdVector*>(channel_1);
    int16x4x2_t deinterleaved_pair;
    int32x4_t temporary_wide;
    const SimdVector scaling_vector = SIMD_LOAD_ONE_FLOAT(kFloatFromInt16);

    const int16x4_t* interleaved_vec =
        reinterpret_cast<const int16x4_t*>(interleaved_buffer);
    for (size_t chunk = 0; chunk < num_chunks; ++chunk) {
      const size_t interleaved_index = chunk * kNumStereoChannels;
      deinterleaved_pair = vuzp_s16(interleaved_vec[interleaved_index],
                                    interleaved_vec[interleaved_index + 1]);
      temporary_wide = vmovl_s16(deinterleaved_pair.val[0]);
      channel_0_vec[chunk] = vcvtq_f32_s32(temporary_wide);
      channel_0_vec[chunk] =
          SIMD_MULTIPLY(scaling_vector, channel_0_vec[chunk]);
      temporary_wide = vmovl_s16(deinterleaved_pair.val[1]);
      channel_1_vec[chunk] = vcvtq_f32_s32(temporary_wide);
      channel_1_vec[chunk] =
          SIMD_MULTIPLY(scaling_vector, channel_1_vec[chunk]);
    }
  }
#endif  // SIMD_NEON
  for (size_t i = length - leftover_samples; i < length; ++i) {
    const size_t interleaved_index = kNumStereoChannels * i;
    channel_0[i] = static_cast<float>(interleaved_buffer[interleaved_index]) *
                   kFloatFromInt16;
    channel_1[i] =
        static_cast<float>(interleaved_buffer[interleaved_index + 1]) *
        kFloatFromInt16;
  }
}

void InterleaveQuad(size_t length, const int16_t* channel_0,
                    const int16_t* channel_1, const int16_t* channel_2,
                    const int16_t* channel_3, int16_t* workspace,
                    int16_t* interleaved_buffer) {
#ifdef SIMD_NEON
  DCHECK(IsAligned(workspace));
  const size_t double_length = length * 2;
  int16_t* workspace_half_point =
      workspace + FindNextAlignedArrayIndex(double_length, sizeof(int16_t),
                                            kMemoryAlignmentBytes);
  InterleaveStereo(length, channel_0, channel_2, workspace);
  InterleaveStereo(length, channel_1, channel_3, workspace_half_point);
  InterleaveStereo(double_length, workspace, workspace_half_point,
                   interleaved_buffer);
#else
  for (size_t i = 0; i < length; ++i) {
    const size_t interleaved_index = kNumFirstOrderAmbisonicChannels * i;
    interleaved_buffer[interleaved_index] = channel_0[i];
    interleaved_buffer[interleaved_index + 1] = channel_1[i];
    interleaved_buffer[interleaved_index + 2] = channel_2[i];
    interleaved_buffer[interleaved_index + 3] = channel_3[i];
  }
#endif  // SIMD_NEON
}

void InterleaveQuad(size_t length, const float* channel_0,
                    const float* channel_1, const float* channel_2,
                    const float* channel_3, float* workspace,
                    float* interleaved_buffer) {
#ifdef SIMD_NEON
  DCHECK(IsAligned(workspace));
  const size_t double_length = length * 2;
  float* workspace_half_point =
      workspace + FindNextAlignedArrayIndex(double_length, sizeof(float),
                                            kMemoryAlignmentBytes);
  DCHECK(IsAligned(workspace_half_point));
  InterleaveStereo(length, channel_0, channel_2, workspace);
  InterleaveStereo(length, channel_1, channel_3, workspace_half_point);
  InterleaveStereo(double_length, workspace, workspace_half_point,
                   interleaved_buffer);
#else
  for (size_t i = 0; i < length; ++i) {
    const size_t interleaved_index = kNumFirstOrderAmbisonicChannels * i;
    interleaved_buffer[interleaved_index] = channel_0[i];
    interleaved_buffer[interleaved_index + 1] = channel_1[i];
    interleaved_buffer[interleaved_index + 2] = channel_2[i];
    interleaved_buffer[interleaved_index + 3] = channel_3[i];
  }
#endif  // SIMD_NEON
}

void DeinterleaveQuad(size_t length, const int16_t* interleaved_buffer,
                      int16_t* workspace, int16_t* channel_0,
                      int16_t* channel_1, int16_t* channel_2,
                      int16_t* channel_3) {
#ifdef SIMD_NEON
  DCHECK(IsAligned(workspace));
  const size_t double_length = length * 2;
  int16_t* workspace_half_point =
      workspace + FindNextAlignedArrayIndex(double_length, sizeof(int16_t),
                                            kMemoryAlignmentBytes);
  DCHECK(IsAligned(workspace_half_point));
  DeinterleaveStereo(double_length, interleaved_buffer, workspace,
                     workspace_half_point);
  DeinterleaveStereo(length, workspace, channel_0, channel_2);
  DeinterleaveStereo(length, workspace_half_point, channel_1, channel_3);
#else
  for (size_t i = 0; i < length; ++i) {
    const size_t interleaved_index = kNumFirstOrderAmbisonicChannels * i;
    channel_0[i] = interleaved_buffer[interleaved_index];
    channel_1[i] = interleaved_buffer[interleaved_index + 1];
    channel_2[i] = interleaved_buffer[interleaved_index + 2];
    channel_3[i] = interleaved_buffer[interleaved_index + 3];
  }
#endif  // SIMD_NEON
}

void DeinterleaveQuad(size_t length, const float* interleaved_buffer,
                      float* workspace, float* channel_0, float* channel_1,
                      float* channel_2, float* channel_3) {
#ifdef SIMD_NEON
  DCHECK(IsAligned(workspace));
  const size_t double_length = length * 2;
  float* workspace_half_point =
      workspace + FindNextAlignedArrayIndex(double_length, sizeof(float),
                                            kMemoryAlignmentBytes);
  DCHECK(IsAligned(workspace_half_point));
  DeinterleaveStereo(double_length, interleaved_buffer, workspace,
                     workspace_half_point);
  DeinterleaveStereo(length, workspace, channel_0, channel_2);
  DeinterleaveStereo(length, workspace_half_point, channel_1, channel_3);
#else
  for (size_t i = 0; i < length; ++i) {
    const size_t interleaved_index = kNumFirstOrderAmbisonicChannels * i;
    channel_0[i] = interleaved_buffer[interleaved_index];
    channel_1[i] = interleaved_buffer[interleaved_index + 1];
    channel_2[i] = interleaved_buffer[interleaved_index + 2];
    channel_3[i] = interleaved_buffer[interleaved_index + 3];
  }
#endif  // SIMD_NEON
}

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#else
__pragma(warning(pop))
#endif