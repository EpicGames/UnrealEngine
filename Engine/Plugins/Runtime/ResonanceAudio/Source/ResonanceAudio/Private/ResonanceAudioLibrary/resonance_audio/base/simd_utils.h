/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_SIMD_UTILS_H_
#define RESONANCE_AUDIO_BASE_SIMD_UTILS_H_

#include <cstddef>
#include <cstdint>

namespace vraudio {

// Checks if the pointer provided is correctly aligned for SIMD.
//
// @param pointer Pointer to check.
// @return True if the pointer is correctly aligned.
bool IsAligned(const float* pointer);
bool IsAligned(const int16_t* pointer);

// Rounds a number of frames up to the next aligned memory address
// based on |memory_alignment_bytes|. This allows for aligning offset pointers
// into a single chunk of allocated memory.
//
// @param length Number of samples before the desired offset pointer.
// @param type_size_bytes Size of the type of each entry in the array.
// @param memory_alignment_bytes Number of bytes to which an address is aligned.
// @return Number of samples into the memory chunk to ensure aligned memory.
size_t FindNextAlignedArrayIndex(size_t length, size_t type_size_bytes,
                                 size_t memory_alignment_bytes);

// Adds a float array |input_a| to another float array |input_b| and stores the
// result in |output|.
//
// @param length Number of floats.
// @param input_a Pointer to the first float in input_a array.
// @param input_b Pointer to the first float in input_b array.
// @param output Pointer to the first float in output array.
void AddPointwise(size_t length, const float* input_a, const float* input_b,
                  float* output);

// Subtracts a float array |input|, pointwise from another float array |output|.
//
// @param length Number of floats.
// @param input Pointer to the first float in input_a array.
// @param output Pointer to the first float in input_b array.
// @param output Pointer to the first float in output array.
void SubtractPointwise(size_t length, const float* input_a,
                       const float* input_b, float* output);

// Pointwise multiplies a float array |input_a| with another float array
// |input_b| and stores the result in |output|.
//
// @param length Number of floats.
// @param input Pointer to the first float in input_a array.
// @param input Pointer to the first float in input_b array.
// @param output Pointer to the first float in output array.
void MultiplyPointwise(size_t length, const float* input_a,
                       const float* input_b, float* output);

// Pointwise multiplies a float array |input_a| with another float array
// |input_b| and adds the result onto |accumulator|.
//
// @param length Number of floats.
// @param input_a Pointer to the first float in input_a array.
// @param input_b Pointer to the first float in input_b array.
// @param accumulator Pointer to the first float in accumulator array.
void MultiplyAndAccumulatePointwise(size_t length, const float* input_a,
                                    const float* input_b, float* accumulator);

// Multiplies a float array |input| by a scalar |gain| over |length| samples.
//
// @param length Number of floats.
// @param gain Scalar value with which to multiply the input.
// @param input Pointer to the first float in input array.
// @param output Pointer to the first float in output array.
void ScalarMultiply(size_t length, float gain, const float* input,
                    float* output);

// Multiplies a float array |input| by a scalar |gain| over |length| samples and
// adds the result onto |accumulator|.
//
// @param length Number of floats.
// @param gain Scalar value with which to multiply the input.
// @param input Pointer to the first float in input array.
// @param output Pointer to the first float in accumulator array.
void ScalarMultiplyAndAccumulate(size_t length, float gain, const float* input,
                                 float* accumulator);

// Calculates an approximmate reciprocal square root.
//
// @param length Number of floats.
// @param input Pointer to the first float in input array.
// @param output Pointer to the first float in output array.
void ReciprocalSqrt(size_t length, const float* input, float* output);

// Calculates an approximate square root.
//
// @param length Number of floats.
// @param input Pointer to the first float in input array.
// @param output Pointer to the first float in output array.
void Sqrt(size_t length, const float* input, float* output);

// Calculates the approximate magnitudes of interleaved complex numbers.
//
// @param length Number of complex numbers in the input array,
//     (i.e. half its length).
// @param input Pointer to the first float in input array. Length: 2 * |length|.
// @param output Pointer to the first float in output array, Length: |length|.
void ApproxComplexMagnitude(size_t length, const float* input, float* output);

// Calculates the complex values in interleaved format (real, imaginary), from a
// vector of magnitudes and of sines and cosines of phase.
//
// @param length Number of total entries (real & imaginary) in the input array.
// @param magnitude Pointer to the first float in the magnitude array, Length:
//     |length| /  2
// @param cos_phase Pointer to the first float in the cosine phase array,
//      Length: |length| /  2
// @param sin_phase Pointer to the first float in the sine phase array, Length:
//     |length| /  2
// @param complex_interleaved_format_output Pointer to the first float in the
//     output array. Length: |length|.
void ComplexInterleavedFormatFromMagnitudeAndSinCosPhase(
    size_t length, const float* magnitude, const float* cos_phase,
    const float* sin_phase, float* complex_interleaved_format_output);

// Generates an identical left and right pair of stereo channels from a mono
// input channel, where each channel is the mono channel times 1/sqrt(2).
//
// @param length Number of floats.
// @param mono Pointer to the first float in an input mono array.
// @param left Pointer to the first float in the left output array.
// @param right Pointer to the first float in the right output array.
void StereoFromMonoSimd(size_t length, const float* mono, float* left,
                        float* right);

// Generates a mono downmix from a pair of stereo channels, where the output is
// equal to the sum of the two inputs times 1/sqrt(2).
//
// @param length Number of floats.
// @param left Pointer to the first float in the left input array.
// @param right Pointer to the first float in the right input array.
// @param mono Pointer to the first float in an output mono array.
void MonoFromStereoSimd(size_t length, const float* left, const float* right,
                        float* mono);

// Converts an array of 32 bit float input to clamped 16 bit int output.
//
// @param length Number of floats in the input array and int16_ts in the output.
// @param input Float array.
// @param output Int array.
void Int16FromFloat(size_t length, const float* input, int16_t* output);

// Converts an array of 16 bit int input to 32 bit float output.
//
// @param length Number of int16_ts in the input array and floats in the output.
// @param input Int array.
// @param output Float array.
void FloatFromInt16(size_t length, const int16_t* input, float* output);

// Interleaves a pair of mono buffers of int_16 data into a stereo buffer.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param channel_0 Input buffer of mono data for the first channel.
// @param channel_1 Input buffer of mono data for the second channel.
// @param interleaved_buffer Output buffer of stereo interleaved data.
void InterleaveStereo(size_t length, const int16_t* channel_0,
                      const int16_t* channel_1, int16_t* interleaved_buffer);

// Interleaves a pair of mono buffers of float data into a stereo buffer.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param channel_0 Input buffer of mono data for the first channel.
// @param channel_1 Input buffer of mono data for the second channel.
// @param interleaved_buffer Output buffer of stereo interleaved data.
void InterleaveStereo(size_t length, const float* channel_0,
                      const float* channel_1, float* interleaved_buffer);

// Interleaves a pair of mono buffers of float data into a stereo buffer of
// int16_t data.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param channel_0 Input buffer of mono data for the first channel (float).
// @param channel_1 Input buffer of mono data for the second channel (float).
// @param interleaved_buffer Output buffer of stereo interleaved data (int16_t).
void InterleaveStereo(size_t length, const float* channel_0,
                      const float* channel_1, int16_t* interleaved_buffer);

// Deinterleaves a stereo buffer of int16_t data into a pair of mono buffers.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param interleaved_buffer Input buffer of stereo interleaved data.
// @param channel_0 Output buffer of mono data for the first channel.
// @param channel_1 Output buffer of mono data for the second channel.
void DeinterleaveStereo(size_t length, const int16_t* interleaved_buffer,
                        int16_t* channel_0, int16_t* channel_1);

// Deinterleaves a stereo buffer of float data into a pair of mono buffers.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param interleaved_buffer Input buffer of stereo interleaved data.
// @param channel_0 Output buffer of mono data for the first channel.
// @param channel_1 Output buffer of mono data for the second channel.
void DeinterleaveStereo(size_t length, const float* interleaved_buffer,
                        float* channel_0, float* channel_1);

// Deinterleaves a stereo buffer of int16_t data into a pair of mono float
// buffers, performing the int16 to floating point conversion.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be twice this size.
// @param interleaved_buffer Input buffer of stereo interleaved data (int16_t).
// @param channel_0 Output buffer of mono data for the first channel (float).
// @param channel_1 Output buffer of mono data for the second channel (float).
void DeinterleaveStereo(size_t length, const int16_t* interleaved_buffer,
                        float* channel_0, float* channel_1);

// Interleaves four mono buffers of int16_t data into a quad buffer.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be four times this size and the workspace must be five times this size.
// @param channel_0 Input buffer of mono data for the first channel.
// @param channel_1 Input buffer of mono data for the second channel.
// @param channel_2 Input buffer of mono data for the third channel.
// @param channel_3 Input buffer of mono data for the fourth channel.
// @param workspace Aligned buffer of 5 * |length| samples in length.
// @param interleaved_buffer Output buffer of quad interleaved data.
void InterleaveQuad(size_t length, const int16_t* channel_0,
                    const int16_t* channel_1, const int16_t* channel_2,
                    const int16_t* channel_3, int16_t* workspace,
                    int16_t* interleaved_buffer);

// Interleaves four mono buffers of float data into a quad buffer.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be four times this size and the workspace must be five times this size.
// @param channel_0 Input buffer of mono data for the first channel.
// @param channel_1 Input buffer of mono data for the second channel.
// @param channel_2 Input buffer of mono data for the third channel.
// @param channel_3 Input buffer of mono data for the fourth channel.
// @param workspace Aligned buffer of 5 * |length| samples in length.
// @param interleaved_buffer Output buffer of quad interleaved data.
void InterleaveQuad(size_t length, const float* channel_0,
                    const float* channel_1, const float* channel_2,
                    const float* channel_3, float* workspace,
                    float* interleaved_buffer);

// Deinterleaves a quad buffer of int16_t data into four mono buffers.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be four times this size and the workspace must be five times this size.
// @param interleaved_buffer Input buffer of quad interleaved data.
// @param workspace Aligned buffer of 5 * |length| samples in length.
// @param channel_0 Output buffer of mono data for the first channel.
// @param channel_1 Output buffer of mono data for the second channel.
// @param channel_2 Output buffer of mono data for the third channel.
// @param channel_3 Output buffer of mono data for the fourth channel.
void DeinterleaveQuad(size_t length, const int16_t* interleaved_buffer,
                      int16_t* workspace, int16_t* channel_0,
                      int16_t* channel_1, int16_t* channel_2,
                      int16_t* channel_3);

// Deinterleaves a quad buffer of float data into four mono buffers.
//
// @param length Number of frames per mono channel. The interleaved buffer must
//     be four times this size and the workspace must be five times this size.
// @param interleaved_buffer Input buffer of quad interleaved data.
// @param workspace Aligned buffer of 5 * |length| samples in length.
// @param channel_0 Output buffer of mono data for the first channel.
// @param channel_1 Output buffer of mono data for the second channel.
// @param channel_2 Output buffer of mono data for the third channel.
// @param channel_3 Output buffer of mono data for the fourth channel.
void DeinterleaveQuad(size_t length, const float* interleaved_buffer,
                      float* workspace, float* channel_0, float* channel_1,
                      float* channel_2, float* channel_3);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_SIMD_UTILS_H_
