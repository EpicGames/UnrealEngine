/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_SOURCE_PARAMETERS_H_
#define RESONANCE_AUDIO_BASE_SOURCE_PARAMETERS_H_

#include "api/resonance_audio_api.h"

#include "base/constants_and_types.h"
#include "base/object_transform.h"

namespace vraudio {

// Gain attenuation types for audio sources.
enum AttenuationType {
  kInput = 0,
  kDirect,
  kReflections,
  kReverb,
  kNumAttenuationTypes
};

// Parameters describing an audio source.
struct SourceParameters {
  // Object transform associated with this buffer.
  ObjectTransform object_transform;

  // Angular spread in degrees. Range [0, 360].
  float spread_deg = 0.0f;

  // Source gain factor.
  float gain = 1.0f;

  // Source gain attenuation factors to be calculated per each buffer.
  float attenuations[kNumAttenuationTypes];

  // Distance attenuation. Value 1 represents no attenuation should be applied,
  // value 0 will fully attenuate the volume. Range [0, 1].
  float distance_attenuation = 1.0f;

  // Distance attenuation rolloff model to use.
  DistanceRolloffModel distance_rolloff_model =
      DistanceRolloffModel::kLogarithmic;

  // Minimum distance at which to apply distance attenuation.
  float minimum_distance = 0.0f;

  // Maximum distance at which to apply distance attenuation.
  float maximum_distance = 500.0f;

  // Alpha weighting of source's directivity pattern. This sets the balance
  // between the dipole and omnidirectional directivity patterns which combine
  // to produce the single directivity output value. Range [0, 1], where 0 is
  // fully omnidirectional and 1 is fully dipole.
  float directivity_alpha = 0.0f;

  // Source directivity order. Increasing this value increases the directivity
  // towards the front of the source. Range [1, inf).
  float directivity_order = 1.0f;

  // Alpha weighting of listener's directivity pattern. This sets the balance
  // between the dipole and omnidirectional pickup patterns which combine to
  // produce the single output value. Range [0, 1], where 0 is fully
  // omnidirectional and 1 is fully dipole.
  float listener_directivity_alpha = 0.0f;

  // Listener directivity order. Increasing this value increases the directivity
  // towards the front of the listener. Range [1, inf).
  float listener_directivity_order = 1.0f;

  // Occlusion intensity. Value 0 represents no occlusion, values greater than 1
  // represent multiple occlusions. The intensity of each occlusion is scaled
  // in range [0, 1].
  float occlusion_intensity = 0.0f;

  // Near field effect gain. Range [0, 9].
  float near_field_gain = 0.0f;

  // Source gain factor for the room effects.
  float room_effects_gain = 1.0f;

  // Whether the source uses binaural rendering or stereo panning.
  bool enable_hrtf = true;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_SOURCE_PARAMETERS_H_
