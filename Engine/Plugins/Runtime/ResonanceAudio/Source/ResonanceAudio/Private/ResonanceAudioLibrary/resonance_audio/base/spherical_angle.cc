/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "base/spherical_angle.h"

#include <cmath>

#include "base/constants_and_types.h"

namespace vraudio {

SphericalAngle::SphericalAngle(float azimuth, float elevation)
    : azimuth_(azimuth), elevation_(elevation) {}

SphericalAngle::SphericalAngle() : SphericalAngle(0.0f, 0.0f) {}

SphericalAngle::SphericalAngle(const SphericalAngle& other)
    : azimuth_(other.azimuth_), elevation_(other.elevation_) {}

SphericalAngle& SphericalAngle::operator=(const SphericalAngle other) {
  if (&other == this) {
    return *this;
  }
  this->azimuth_ = other.azimuth_;
  this->elevation_ = other.elevation_;
  return *this;
}

SphericalAngle SphericalAngle::FromWorldPosition(
    const WorldPosition& world_position) {
  return SphericalAngle(
      std::atan2(-world_position[0], -world_position[2]),
      std::atan2(world_position[1],
                 std::sqrt(world_position[0] * world_position[0] +
                           world_position[2] * world_position[2])));
}

SphericalAngle SphericalAngle::FromDegrees(float azimuth_degrees,
                                           float elevation_degrees) {
  return SphericalAngle(azimuth_degrees * kRadiansFromDegrees,
                        elevation_degrees * kRadiansFromDegrees);
}

SphericalAngle SphericalAngle::FlipAzimuth() const {
  return SphericalAngle(-azimuth_, elevation_);
}

WorldPosition SphericalAngle::GetWorldPositionOnUnitSphere() const {
  return WorldPosition(-std::cos(elevation_) * std::sin(azimuth_),
                       std::sin(elevation_),
                       -std::cos(elevation_) * std::cos(azimuth_));
}

SphericalAngle SphericalAngle::Rotate(const WorldRotation& rotation) const {
  const WorldPosition original_world_position = GetWorldPositionOnUnitSphere();
  const WorldPosition rotated_world_position =
      rotation * original_world_position;
  return FromWorldPosition(rotated_world_position);
}

bool SphericalAngle::operator==(const SphericalAngle& other) const {
  return (azimuth_ == other.azimuth_) && (elevation_ == other.elevation_);
}

}  // namespace vraudio
