/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_SPHERICAL_ANGLE_H_
#define RESONANCE_AUDIO_BASE_SPHERICAL_ANGLE_H_

#include "base/misc_math.h"

namespace vraudio {

// Represents angular position on a sphere in terms of azimuth and elevation.
class SphericalAngle {
 public:
  // Constructs a spherical angle with the given azimuth and elevation.
  SphericalAngle(float azimuth, float elevation);

  // Constructs a default spherical angle (azimuth = 0, elevation = 0).
  SphericalAngle();

  // Constructs a spherical angle from a given one.
  SphericalAngle(const SphericalAngle& other);

  SphericalAngle& operator=(const SphericalAngle other);

  // Returns a spherical angle representation of given |world_position| (World
  // Space).
  //
  // @param world_position 3D position in world space.
  // @return Spherical angle that represents the |world_position|.
  static SphericalAngle FromWorldPosition(const WorldPosition& world_position);

  // Returns a spherical angle from azimuth and elevation in degrees.
  static SphericalAngle FromDegrees(float azimuth_degrees,
                                    float elevation_degrees);

  // Returns another spherical angle with the same elevation but the azimuth
  // sign flipped.
  //
  // @return Horizontally flipped version of the spherical angle.
  SphericalAngle FlipAzimuth() const;

  // Returns the |WorldPosition| coordinates (World Space) on the unit sphere
  // corresponding to this spherical angle. The transformation is
  // defined as such:
  // x = -cos(elevation) * sin(azimuth)
  // y = sin(elevation)
  // z = -cos(elevation) * cos(azimuth)
  //
  // @return 3D position in world space.
  WorldPosition GetWorldPositionOnUnitSphere() const;

  // Returns the rotated version of the spherical angle using given
  // |WorldRotation|.
  //
  // @param rotation Rotation to be applied to the spherical angle.
  // @return Rotated version of the spherical angle.
  SphericalAngle Rotate(const WorldRotation& rotation) const;

  void set_azimuth(float azimuth) { azimuth_ = azimuth; }
  void set_elevation(float elevation) { elevation_ = elevation; }

  float azimuth() const { return azimuth_; }
  float elevation() const { return elevation_; }

  bool operator==(const SphericalAngle& other) const;

 private:
  float azimuth_;
  float elevation_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_BASE_SPHERICAL_ANGLE_H_
