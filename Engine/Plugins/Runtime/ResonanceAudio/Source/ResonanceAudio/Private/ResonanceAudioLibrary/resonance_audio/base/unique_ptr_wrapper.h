/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_BASE_UNIQUE_PTR_WRAPPER_H_
#define RESONANCE_AUDIO_BASE_UNIQUE_PTR_WRAPPER_H_

#include <memory>

// Wrapper around std::unique_ptr to enable the binding of unique_ptr buffers to
// std:function and/or lambda function.
template <typename T>
struct UniquePtrWrapper {
  UniquePtrWrapper(const UniquePtrWrapper& other) : ptr(std::move(other.ptr)) {}
  UniquePtrWrapper(UniquePtrWrapper&& other) : ptr(std::move(other.ptr)) {}
  explicit UniquePtrWrapper(std::unique_ptr<T> buffer)
      : ptr(std::move(buffer)) {}
  mutable std::unique_ptr<T> ptr;
};

#endif  // RESONANCE_AUDIO_BASE_UNIQUE_PTR_WRAPPER_H_
