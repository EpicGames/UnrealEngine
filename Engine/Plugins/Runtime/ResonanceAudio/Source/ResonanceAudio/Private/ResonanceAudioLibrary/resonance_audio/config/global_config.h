/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_CONFIG_GLOBAL_CONFIG_H_
#define RESONANCE_AUDIO_CONFIG_GLOBAL_CONFIG_H_

#include "graph/graph_manager_config.h"

namespace vraudio {

inline GraphManagerConfig GlobalConfig() {
  GraphManagerConfig config;
  config.configuration_name = "Global Config";

  config.max_ambisonic_order = 3;
  config.sh_hrir_filenames = {{1, "WAV/Subject_002/SH/sh_hrir_order_1.wav"},
                              {2, "WAV/Subject_002/SH/sh_hrir_order_2.wav"},
                              {3, "WAV/Subject_002/SH/sh_hrir_order_3.wav"}};
  return config;
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_CONFIG_GLOBAL_CONFIG_H_
