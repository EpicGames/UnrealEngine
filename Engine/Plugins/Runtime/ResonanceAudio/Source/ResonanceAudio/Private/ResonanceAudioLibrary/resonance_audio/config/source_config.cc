/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "config/source_config.h"

namespace vraudio {

SourceGraphConfig StereoPanningConfig() {
  SourceGraphConfig config;
  config.configuration_name = "Stereo Panning";

  config.ambisonic_order = 1;
  config.enable_hrtf = false;
  config.enable_direct_rendering = true;

  return config;
}

SourceGraphConfig BinauralLowQualityConfig() {
  SourceGraphConfig config;
  config.configuration_name = "Binaural Low Quality";

  config.ambisonic_order = 1;
  config.enable_hrtf = true;
  config.enable_direct_rendering = true;

  return config;
}

SourceGraphConfig BinauralMediumQualityConfig() {
  SourceGraphConfig config;
  config.configuration_name = "Binaural Medium Quality";

  config.ambisonic_order = 2;
  config.enable_hrtf = true;
  config.enable_direct_rendering = true;

  return config;
}

SourceGraphConfig BinauralHighQualityConfig() {
  SourceGraphConfig config;
  config.configuration_name = "Binaural High Quality";

  config.ambisonic_order = 3;
  config.enable_hrtf = true;
  config.enable_direct_rendering = true;

  return config;
}

SourceGraphConfig RoomEffectsOnlyConfig() {
  SourceGraphConfig config;
  config.configuration_name = "Room Effects Only";

  config.ambisonic_order = 1;
  config.enable_hrtf = false;
  config.enable_direct_rendering = false;

  return config;
}

}  // namespace vraudio
