/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_CONFIG_SOURCE_CONFIG_H_
#define RESONANCE_AUDIO_CONFIG_SOURCE_CONFIG_H_

#include "graph/source_graph_config.h"

namespace vraudio {

SourceGraphConfig StereoPanningConfig();
SourceGraphConfig BinauralLowQualityConfig();
SourceGraphConfig BinauralMediumQualityConfig();
SourceGraphConfig BinauralHighQualityConfig();
SourceGraphConfig RoomEffectsOnlyConfig();

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_CONFIG_SOURCE_CONFIG_H_
