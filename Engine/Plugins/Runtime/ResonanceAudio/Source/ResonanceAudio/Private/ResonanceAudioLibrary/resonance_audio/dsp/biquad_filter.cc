/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/biquad_filter.h"

#include "base/constants_and_types.h"

namespace vraudio {

namespace {

// This value has been chosen empirically after a series of experiments
// performed by kellyi@ and alperg@. It was found that crossfading over 256
// samples yielded no audible glitching artefacts and an acceptable amount of
// delay.

const size_t kIdealSamplesToIterate = 256U;

}  // namespace

BiquadFilter::BiquadFilter(const BiquadCoefficients& coefficients,
                           size_t frames_per_buffer)
    : biquad_delay_line_({{0.0f, 0.0f}}),
      interpolate_flag_(false),
      interpolate_counter_(0),
      old_delay_line_({{0.0f, 0.0f}}),
      samples_to_iterate_over_(frames_per_buffer > kIdealSamplesToIterate
                                   ? kIdealSamplesToIterate
                                   : frames_per_buffer),
      fade_scale_(1.0f / static_cast<float>(samples_to_iterate_over_)),
      old_coefficients_() {
  DCHECK_GT(frames_per_buffer, 0U);
  CHECK_GT(coefficients_.a[0], kEpsilonFloat);
  SetCoefficients(coefficients);
}

void BiquadFilter::SetCoefficients(const BiquadCoefficients& coefficients) {
  coefficients_ = coefficients;
  // Normalize the coefficients for use in the |FilterSample()| function.
  coefficients_.a[1] /= coefficients_.a[0];
  coefficients_.a[2] /= coefficients_.a[0];
  coefficients_.b[0] /= coefficients_.a[0];
  coefficients_.b[1] /= coefficients_.a[0];
  coefficients_.b[2] /= coefficients_.a[0];
}

void BiquadFilter::Filter(const AudioBuffer::Channel& input_channel,
                          AudioBuffer::Channel* output_channel) {
  DCHECK(output_channel);
  DCHECK_EQ(input_channel.size(), output_channel->size());

  if (interpolate_flag_) {
    for (size_t frame = 0; frame < input_channel.size(); ++frame) {
      // Biquad coefficients are updated here.
      UpdateInterpolate();
      (*output_channel)[frame] = InterpolateFilterSample(input_channel[frame]);
    }
  } else {
    for (size_t frame = 0; frame < input_channel.size(); ++frame) {
      (*output_channel)[frame] = FilterSample(
          input_channel[frame], &biquad_delay_line_, coefficients_);
    }
  }
}

void BiquadFilter::InterpolateToCoefficients(
    const BiquadCoefficients& coefficients) {
  interpolate_flag_ = true;
  // Reset the counter so we perform the update over samples_to_iterate_over_
  // samples.
  interpolate_counter_ = 0;
  // Store the old coefficients, update the new ones and transfer data between
  // delay lines.
  old_coefficients_ = coefficients_;
  coefficients_ = coefficients;
  old_delay_line_ = biquad_delay_line_;
}

void BiquadFilter::Clear() {
  biquad_delay_line_ = {{0.0f, 0.0f}};
  interpolate_flag_ = false;
  interpolate_counter_ = 0;
  old_delay_line_ = {{0.0f, 0.0f}};
}

float BiquadFilter::FilterSample(float input, std::array<float, 2>* delay_line,
                                 const BiquadCoefficients& coefficients) {
  // Using A Direct Form II Implementation Difference equation:
  // Source: Digital Signal Processing Principles Algorithms and Applications
  //    Fourth Edition. John G. Prolakis and Dimitris G. Manolakis - Chapter 9
  //  w[n] =         x[n] - (a1/a0)*w[n-1] - (a2/a0)*w[n-2]
  //  y(n) = (b0/a0)*w[n] + (b1/a0)*w[n-1] + (b2/a0)*w[n-2]
  // where x[n] is input, w[n] is storage and y[n] is output.
  // The division by a0 has been performed in Biquad::SetCoefficients.

  // define a temporary storage value w.
  const float w = input - (*delay_line)[0] * coefficients.a[1] -
                  (*delay_line)[1] * coefficients.a[2];

  // Do second half of the calculation to generate output.
  const float output = w * coefficients.b[0] +
                       (*delay_line)[0] * coefficients.b[1] +
                       (*delay_line)[1] * coefficients.b[2];

  // Update delay line.
  (*delay_line)[1] = (*delay_line)[0];
  (*delay_line)[0] = w;

  return output;
}

void BiquadFilter::UpdateInterpolate() {
  if (++interpolate_counter_ > samples_to_iterate_over_) {
    interpolate_flag_ = false;
  }
}

float BiquadFilter::InterpolateFilterSample(float input) {
  const float new_filter_output =
      FilterSample(input, &biquad_delay_line_, coefficients_);

  if (!interpolate_flag_) {
    return new_filter_output;
  } else {
    // Process the "old" filter values.
    const float old_filter_output =
        FilterSample(input, &old_delay_line_, old_coefficients_);
    // A linear crossfade between old_filter_output and new_filter_output,
    // stepsize is fade_scale_.
    const float weight = fade_scale_ * static_cast<float>(interpolate_counter_);
    const float sample_diff = new_filter_output - old_filter_output;
    return weight * sample_diff + old_filter_output;
  }
}

}  // namespace vraudio
