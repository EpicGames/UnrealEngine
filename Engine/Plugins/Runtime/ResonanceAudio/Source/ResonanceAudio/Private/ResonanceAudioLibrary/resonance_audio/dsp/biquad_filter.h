/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_BIQUAD_FILTER_H_
#define RESONANCE_AUDIO_DSP_BIQUAD_FILTER_H_



#include "base/audio_buffer.h"
#include "base/logging.h"

#include "ResonanceEigenHelper.h"
#include <array>

namespace vraudio {

// Set of transfer function coefficients.
struct BiquadCoefficients {
  // Constructor takes as its arguments 6 floats representing the transfer
  // function coefficients of a biquad filter.
  BiquadCoefficients(float a0, float a1, float a2, float b0, float b1, float b2)
      : a({{a0, a1, a2}}), b({{b0, b1, b2}}) {}

  // Default constructor that sets a0 and b0 == 1 so that if the coefficients
  // are used, the filter is stable and has no effect on input. i.e. input
  // appears to just pass through.
  BiquadCoefficients() : BiquadCoefficients(1.0, 0.0, 0.0, 1.0, 0.0, 0.0) {}

  // The denominator quadratic coefficients.
  std::array<float, 3> a;
  // The numerator quadratic coefficients.
  std::array<float, 3> b;
};

// Class representing a biquad filter as its transfer function coefficients.
// This class also performs filtering.
class BiquadFilter {
 public:
  // Constructs a BiquadFilter using a BiquadCoefficients struct.
  //
  // @param coefficients A BiquadCoefficients to set the internal
  //     |coefficients_|.
  // @param frames_per_buffer The number of frames in each data buffer to be
  //     processed. This is used to set the number of samples to iterate over
  //     during a change of filter state.
  BiquadFilter(const BiquadCoefficients& coefficients,
               size_t frames_per_buffer);

  // Filter method for use with AudioBuffer::Channel.
  //
  // @param input_channel An AudioBuffer::Channel of input.
  // @param output_channel A pointer to an AudioBuffer::Channel for output.
  void Filter(const AudioBuffer::Channel& input_channel,
              AudioBuffer::Channel* output_channel);

  // Sets the fiter's coefficents to those passed, with all
  // coefficients bar a0 scaled by 1/a0.
  //
  // @param coefficients A set of BiquadCoefficients.
  void SetCoefficients(const BiquadCoefficients& coefficients);

  // Sets the target coefficients to be interpolated.
  //
  // @param coefficents The BiquadCoefficients we wish to interpolate to over
  //     samples_to_iterate_over_ samples of the next input buffer.
  void InterpolateToCoefficients(const BiquadCoefficients& coefficients);

  // Clears the internal state of the filter, except for coefficients.
  void Clear();

 private:
  friend class BiquadFilterInterpolateTest;

  // Filters a single sample of input.
  //
  // @param input A single input sample from a Planar or Interleaved buffer.
  // @param delay_line The delay_line to use (important for interpolation).
  // @param coefficients The biquad coeffients for use in filtering.
  // @return An output value to be placed in a Planar or Interleaved buffer.
  float FilterSample(float input, std::array<float, 2>* delay_line,
                            const BiquadCoefficients& coefficients);

  // If InterpolateToState() has been called to assign new filter coefficients,
  // this function will be called samples_to_iterate_over_ times within the next
  // call of the Filter() function to slowly transition to the new coefficients
  // which were passed to InterpolateToState() previously.
  void UpdateInterpolate();

  // Filters a single sample of input while transitioning between coefficients.
  // Performs a linear crossfade over samples_to_iterate_over_ samples.
  //
  // @param input A single input sample from a Planar or Interleaved buffer.
  // @return Output value to be placed in an audio buffer.
  float InterpolateFilterSample(float input);

  // Stores the memory state of the biquad
  std::array<float, 2> biquad_delay_line_;

  // Flag that denotes whether or not we are transitioning to another set of
  // coefficients via interpolation.
  bool interpolate_flag_;

  // Counter that denotes how far we are into transitioning to another set of
  // coefficients via interpolation.
  size_t interpolate_counter_;

  // Stores the memory state of the biquad.
  std::array<float, 2> old_delay_line_;

  // Number of samples over which to apply the filter coefficients.
  size_t samples_to_iterate_over_;

  // Value used to crossfade between filter outputs.
  float fade_scale_;

  // A set of coefficient which are updated as we interpolate between filter
  // coefficients.
  BiquadCoefficients old_coefficients_;

  // Represents and maintains the state of the biquad filter in terms of its
  // transfer function coefficients.
  BiquadCoefficients coefficients_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_BIQUAD_FILTER_H_
