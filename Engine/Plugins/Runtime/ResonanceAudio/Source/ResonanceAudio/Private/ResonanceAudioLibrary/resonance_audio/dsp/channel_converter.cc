/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/channel_converter.h"

#include <cmath>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_utils.h"

namespace vraudio {

void ConvertStereoFromMono(const AudioBuffer& input, AudioBuffer* output) {
  DCHECK(output);
  DCHECK_EQ(input.num_channels(), kNumMonoChannels);
  DCHECK_EQ(output->num_channels(), kNumStereoChannels);
  DCHECK_EQ(input.num_frames(), output->num_frames());
  StereoFromMonoSimd(input.num_frames(), &input[0][0], &(*output)[0][0],
                     &(*output)[1][0]);
}

void ConvertMonoFromStereo(const AudioBuffer& input, AudioBuffer* output) {
  DCHECK(output);
  DCHECK_EQ(input.num_channels(), kNumStereoChannels);
  DCHECK_EQ(output->num_channels(), kNumMonoChannels);
  DCHECK_EQ(input.num_frames(), output->num_frames());
  MonoFromStereoSimd(input.num_frames(), &input[0][0], &input[1][0],
                     &(*output)[0][0]);
}

}  // namespace vraudio
