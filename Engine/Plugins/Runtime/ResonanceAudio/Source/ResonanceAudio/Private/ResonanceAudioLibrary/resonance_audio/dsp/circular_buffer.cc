/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/circular_buffer.h"

#include <algorithm>

#include "base/constants_and_types.h"

namespace vraudio {

CircularBuffer::CircularBuffer(size_t buffer_length, size_t num_input_frames,
                               size_t num_output_frames)
    : num_input_frames_(num_input_frames),
      num_output_frames_(num_output_frames),
      buffer_(kNumMonoChannels, buffer_length),
      write_cursor_(0),
      read_cursor_(0),
      num_valid_frames_(0) {
  CHECK_GE(buffer_length, num_input_frames + num_output_frames);
}

bool CircularBuffer::InsertBuffer(const AudioBuffer::Channel& input) {
  DCHECK_EQ(input.size(), num_input_frames_);

  if (num_valid_frames_ + num_input_frames_ > buffer_.num_frames()) {
    return false;
  }
  // Remaining space after the write cursor.
  const size_t forward_write_space = read_cursor_ <= write_cursor_
                                         ? buffer_.num_frames() - write_cursor_
                                         : read_cursor_ - write_cursor_;

  // Copy the input into the buffer.
  AudioBuffer::Channel* buffer_channel = &buffer_[0];
  if (forward_write_space >= num_input_frames_) {
    DCHECK_LE(buffer_channel->begin() + write_cursor_ + num_input_frames_,
              buffer_channel->end());
    std::copy(input.begin(), input.end(),
              buffer_channel->begin() + write_cursor_);
  } else {
    DCHECK_LE(buffer_channel->begin() + write_cursor_ + forward_write_space,
              buffer_channel->end());
    DCHECK_LT(input.begin() + forward_write_space, input.end());
    std::copy(input.begin(), input.begin() + forward_write_space,
              buffer_channel->begin() + write_cursor_);
    DCHECK_LE(buffer_channel->begin() + forward_write_space,
              buffer_channel->end());
    std::copy(input.begin() + forward_write_space, input.end(),
              buffer_channel->begin());
  }

  write_cursor_ = (write_cursor_ + num_input_frames_) % buffer_.num_frames();
  num_valid_frames_ += num_input_frames_;
  return true;
}

bool CircularBuffer::RetrieveBuffer(AudioBuffer::Channel* output) {
  return RetrieveBufferWithOffset(/*offset=*/0, output);
}

bool CircularBuffer::RetrieveBufferWithOffset(size_t offset,
                                              AudioBuffer::Channel* output) {
  DCHECK_LE(output->begin() + num_output_frames_ + offset, output->end());

  if (num_valid_frames_ < num_output_frames_) {
    return false;
  }
  // Remaining space after the read cursor.
  const size_t forward_read_space = read_cursor_ < write_cursor_
                                        ? write_cursor_ - read_cursor_
                                        : buffer_.num_frames() - read_cursor_;

  // Copy the buffer values into the output.
  AudioBuffer::Channel* buffer_channel = &buffer_[0];
  if (forward_read_space >= num_output_frames_) {
    DCHECK_LE(buffer_channel->begin() + read_cursor_ + num_output_frames_,
              buffer_channel->end());
    std::copy(buffer_channel->begin() + read_cursor_,
              buffer_channel->begin() + read_cursor_ + num_output_frames_,
              output->begin() + offset);
  } else {
    DCHECK_LE(buffer_channel->begin() + read_cursor_ + forward_read_space,
              buffer_channel->end());
    DCHECK_LE(output->begin() + forward_read_space + offset, output->end());
    std::copy(buffer_channel->begin() + read_cursor_,
              buffer_channel->begin() + read_cursor_ + forward_read_space,
              output->begin() + offset);
    DCHECK_GE(buffer_channel->begin() + num_output_frames_ - forward_read_space,
              buffer_channel->begin());
    DCHECK_LE(output->begin() + offset + num_output_frames_, output->end());
    std::copy(buffer_channel->begin(),
              buffer_channel->begin() + num_output_frames_ - forward_read_space,
              output->begin() + offset + forward_read_space);
  }
  read_cursor_ = (read_cursor_ + num_output_frames_) % buffer_.num_frames();
  num_valid_frames_ -= num_output_frames_;
  return true;
}

void CircularBuffer::Clear() {
  read_cursor_ = 0;
  write_cursor_ = 0;
  num_valid_frames_ = 0;
}

}  // namespace vraudio
