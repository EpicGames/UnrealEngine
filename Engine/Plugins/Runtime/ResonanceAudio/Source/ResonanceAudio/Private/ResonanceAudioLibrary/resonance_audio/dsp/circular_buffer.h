/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_CIRCULAR_BUFFER_H_
#define RESONANCE_AUDIO_DSP_CIRCULAR_BUFFER_H_

#include "base/audio_buffer.h"

namespace vraudio {
// Class that implements a simple mono circular buffer, accepting input and
// output of different length.
class CircularBuffer {
 public:
  // Constructs a circular buffer.
  //
  // @param buffer_length The length of the Circular buffer. This value must be
  //     at least |num_input_frames| + |num_output_frames| so that we can always
  //     either add or remove data.
  // @param num_input_frames Length of the input buffers in frames.
  // @param num_output_frames Length of the output buffers in frames.
  CircularBuffer(size_t buffer_length, size_t num_input_frames,
                 size_t num_output_frames);

  // Inserts a buffer of mono input into the |CircularBuffer| if space permits.
  //
  // @param input A channel of input data |num_input_frames| in length.
  // @return True if there was space in the buffer and the input was
  //     successfully inserted, false otherwise.
  bool InsertBuffer(const AudioBuffer::Channel& input);

  // Retrieves a buffer of output from the |CircularBuffer| if it contains
  // sufficient data.
  //
  // @param output A channel to hold |num_output_frames| of output data. The
  //     channel may be greater in length than |num_output_frames|, in this
  //     case only the first |num_output_frames| will be overwritten.
  // @return True if there was sufficient data in the buffer and the output was
  //     successfully retrieved.
  bool RetrieveBuffer(AudioBuffer::Channel* output);

  // Retrieves a buffer of output from the |CircularBuffer| to an offset
  // location in an output channel, provided it contains sufficient data.
  //
  // @param offset Number of samples of offset into the |output| channel.
  // @param output A channel to hold |num_output_frames| of output data. The
  //     channel may be greater in length than |num_output_frames| + |offset|,
  //     in this case only the first |num_output_frames| after |offset| will be
  //     overwritten.
  // @return True if there was sufficient data in the buffer and the output was
  //     successfully retrieved.
  bool RetrieveBufferWithOffset(size_t offset, AudioBuffer::Channel* output);

  // Returns the number of samples of data currently in the |CircularBuffer|.
  //
  // @return The number of samples of data currently in the buffer.
  size_t GetOccupancy() const { return num_valid_frames_; }

  // Resets the |CircularBuffer|.
  void Clear();

 private:
  // Number of input frames to be inserted into the buffer.
  const size_t num_input_frames_;

  // Number of output frames to be retrieved from the buffer.
  const size_t num_output_frames_;

  // Mono audio buffer to hold the data.
  AudioBuffer buffer_;

  // Position at which we are writing into the buffer.
  size_t write_cursor_;

  // position at which we are reading from the buffer.
  size_t read_cursor_;

  // Number of frames of data currently stored within the buffer.
  size_t num_valid_frames_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_CIRCULAR_BUFFER_H_
