/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/delay_filter.h"

#include <cmath>

#include "base/constants_and_types.h"
#include "base/misc_math.h"


namespace vraudio {
// The following explains the behaviour of the delay line when a change to the
// delay length is made. Initially |delay| is 2, |frames_per_buffer_|
// is 4, and |delay_line_| is 8 in length.
//
//      delay = 2
//
// input0 [1, 2, 3, 4]  output0 [0, 0, 1, 2]
//
//             0  1  2  3  4  5  6  7
// delay line [1, 2, 3, 4, 0, 0, 0, 0]
//                         |     |
//                         w     r
// ----------------------------------------------------------
//
//      delay = 2
//
// input1 [5, 6, 7, 8]  output1 [3, 4, 5, 6]
//
//             0  1  2  3  4  5  6  7
// delay line [1, 2, 3, 4, 5, 6, 7, 8]
//             |     |
//             w     r
// ----------------------------------------------------------
//
//      delay = 1
//
//             0  1  2  3  4  5  6  7
// delay line [1, 2, 3, 4, 5, 6, 7, 8]
//             |        |
//             w        r
// ----------------------------------------------------------
//
//      delay = 1
//
// input2 [9, 10, 11, 12] output2 [8, 9, 10, 11]
//
//             0  1  2  3   4   5   6  7
// delay line [9, 10, 11, 12, 5, 6, 7, 8]
//                            |        |
//                            w        r
// ----------------------------------------------------------

DelayFilter::DelayFilter(size_t max_delay_length, size_t frames_per_buffer)
    : frames_per_buffer_(frames_per_buffer),
      delay_line_(nullptr),
      write_cursor_(0) {
  DCHECK_GT(frames_per_buffer_, 0U);
  SetMaximumDelay(max_delay_length);
}

void DelayFilter::SetMaximumDelay(size_t max_delay_length) {
  max_delay_length_ = max_delay_length;
  const size_t new_buffer_size = frames_per_buffer_ + max_delay_length_;

  if (delay_line_ == nullptr) {
    delay_line_.reset(new AudioBuffer(kNumMonoChannels, new_buffer_size));
    delay_line_->Clear();
    return;
  }
  AudioBuffer::Channel* delay_channel = &(*delay_line_)[0];
  // If |delay_line_| is not large enough, resize.
  const size_t current_buffer_size = delay_line_->num_frames();
  if (max_delay_length_ > current_buffer_size - frames_per_buffer_) {
    // Allocate |new_delay_line| and populate it with the current |delay_line_|
    // data, so that replacing the buffer does not affect the already stored
    // samples.
    auto new_delay_line = std::unique_ptr<AudioBuffer>(
        new AudioBuffer(kNumMonoChannels, new_buffer_size));
    new_delay_line->Clear();
    std::copy(delay_channel->begin() + write_cursor_, delay_channel->end(),
              (*new_delay_line)[0].begin());
    if (write_cursor_ > 0) {
      // Positive |write_cursor_| means that we still have remaining samples to
      // be moved to |new_delay_line_| all of which reside left of the cursor.
      std::copy(
          delay_channel->begin(), delay_channel->begin() + write_cursor_,
          (*new_delay_line)[0].begin() + current_buffer_size - write_cursor_);
      write_cursor_ = current_buffer_size;
    }
    delay_line_ = std::move(new_delay_line);
  }
}

void DelayFilter::InsertData(const AudioBuffer::Channel& input) {

  DCHECK_EQ(input.size(), frames_per_buffer_);

  const size_t delay_buffer_size = delay_line_->num_frames();

  // Record the remaining space in the |delay_line_| after the write cursor.
  const size_t remaining_size_write = delay_buffer_size - write_cursor_;
  AudioBuffer::Channel* delay_channel = &(*delay_line_)[0];

  // Copy the input into the delay line.
  if (remaining_size_write >= frames_per_buffer_) {
    DCHECK_LE(delay_channel->begin() + write_cursor_ + frames_per_buffer_,
              delay_channel->end());
    std::copy(input.begin(), input.end(),
              delay_channel->begin() + write_cursor_);
  } else {
    DCHECK_LE(delay_channel->begin() + write_cursor_ + remaining_size_write,
              delay_channel->end());
    DCHECK_LE(input.begin() + remaining_size_write, input.end());
    std::copy(input.begin(), input.begin() + remaining_size_write,
              delay_channel->begin() + write_cursor_);
    DCHECK_LE(delay_channel->begin() + remaining_size_write,
              delay_channel->end());
    std::copy(input.begin() + remaining_size_write, input.end(),
              delay_channel->begin());
  }

  write_cursor_ = (write_cursor_ + frames_per_buffer_) % delay_buffer_size;
}

void DelayFilter::GetDelayedData(size_t delay_samples,
                                 AudioBuffer::Channel* buffer) {

  DCHECK(buffer);
  DCHECK_GE(delay_samples, 0U);
  DCHECK_LE(delay_samples, max_delay_length_);

  const size_t delay_buffer_size = delay_line_->num_frames();
  // Position in the delay line to begin reading from.
  DCHECK_GE(write_cursor_ + delay_buffer_size,
            delay_samples + frames_per_buffer_);
  const size_t read_cursor =
      (write_cursor_ + delay_buffer_size - delay_samples - frames_per_buffer_) %
      delay_buffer_size;
  // Record the remaining space in the |delay_line_| after the read cursor.
  const size_t remaining_size_read = delay_buffer_size - read_cursor;
  AudioBuffer::Channel* delay_channel = &(*delay_line_)[0];

  // Extract a portion of the delay line into the buffer.
  if (remaining_size_read >= frames_per_buffer_) {
    DCHECK_LE(buffer->begin() + frames_per_buffer_, buffer->end());
    DCHECK_LE(delay_channel->begin() + read_cursor + frames_per_buffer_,
              delay_channel->end());
    std::copy(delay_channel->begin() + read_cursor,
              delay_channel->begin() + read_cursor + frames_per_buffer_,
              buffer->begin());
  } else {
    DCHECK_LE(buffer->begin() + delay_channel->size() - read_cursor,
              buffer->end());
    std::copy(delay_channel->begin() + read_cursor, delay_channel->end(),
              buffer->begin());
    DCHECK_LE(buffer->begin() + frames_per_buffer_, buffer->end());
    DCHECK_LE(delay_channel->begin() + frames_per_buffer_ - remaining_size_read,
              delay_channel->end());
    std::copy(delay_channel->begin(),
              delay_channel->begin() + frames_per_buffer_ - remaining_size_read,
              buffer->begin() + remaining_size_read);
  }
}

}  // namespace vraudio
