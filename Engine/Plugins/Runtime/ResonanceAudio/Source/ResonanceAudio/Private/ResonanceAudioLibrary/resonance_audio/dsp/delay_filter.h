/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_DELAY_FILTER_H_
#define RESONANCE_AUDIO_DSP_DELAY_FILTER_H_

#include <algorithm>
#include <memory>

#include "base/audio_buffer.h"

namespace vraudio {

// Single channel delay line class. Delays input buffer data by a non-negative
// integer number of samples.
//
// This implementation is not thread safe. The ClearBuffer() and InsertData()
// functions should not be called by a seperate thread during GetDelayedData().
class DelayFilter {
 public:
  // Constructs a DelayFilter.
  //
  // @param max_delay_length Maximum number of samples the input should be
  //     delayed by.
  // @param frames_per_buffer Number of frames in each processed buffer.
  DelayFilter(size_t max_delay_length, size_t frames_per_buffer);

  // Sets the maximum delay length. It will allocate more space in the
  // |delay_line_| if the new |max_delay_length| is more than doubled.
  //
  // @param max_delay_length New maximum delay in samples.
  void SetMaximumDelay(size_t max_delay_length);

  // Returns the current length of the |delay_line_|.
  size_t GetDelayBufferLength() const { return delay_line_->num_frames(); }

  // Returns the current maximum delay applicable to input buffers.
  size_t GetMaximumDelayLength() const { return max_delay_length_; }

  // Sets all of the |delay_line_| samples to zero.
  void ClearBuffer() { delay_line_->Clear(); }

  // Copies an |AudioBuffer::Channel| of data to the delay line.
  //
  // @param input Input data.
  void InsertData(const AudioBuffer::Channel& input);

  // Fills an |AudioBuffer::Channel| with data delayed by a specified amount
  // less than or equal to the delay line's set |max_delay_length_|.
  //
  // @param delay_samples Requested delay to the data extraced from the delay
  //     line. Must be less than or equal to |max_delay_length_|.
  // @param buffer Pointer to the output data, i.e., delayed input data.
  void GetDelayedData(size_t delay_samples, AudioBuffer::Channel* buffer);

 private:
  // Maximum length of the delay to be applied (in samples).
  size_t max_delay_length_;

  // Number of frames in each AudioBuffer input/output.
  size_t frames_per_buffer_;

  // The delay line holding all of the delayed samples.
  std::unique_ptr<AudioBuffer> delay_line_;

  // Position in the delay line to begin writing to.
  size_t write_cursor_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_DELAY_FILTER_H_
