/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/distance_attenuation.h"

#include <algorithm>
#include <cmath>

#include "base/constants_and_types.h"

namespace vraudio {

float ComputeLogarithmicDistanceAttenuation(
    const WorldPosition& listener_position,
    const WorldPosition& source_position, float min_distance,
    float max_distance) {
  const float distance = (listener_position - source_position).norm();
  if (distance > max_distance) {
    return 0.0f;
  }
  // Logarithmic attenuation.
  const float min_distance_allowed =
      std::max(min_distance, kNearFieldThreshold);
  if (distance > min_distance_allowed) {
    const float attenuation_interval = max_distance - min_distance_allowed;
    if (attenuation_interval > kEpsilonFloat) {
      // Compute the distance attenuation value by the logarithmic curve
      // "1 / (d + 1)" with an offset of |min_distance_allowed|.
      const float relative_distance = distance - min_distance_allowed;
      const float attenuation = 1.0f / (relative_distance + 1.0f);
      // Shift the curve downwards by the attenuation value at |max_distance|,
      // and scale the value by the inverse of it in order to keep the curve's
      // peak value 1 at |min_distance_allowed|.
      const float attenuation_max = 1.0f / (1.0f + attenuation_interval);
      return (attenuation - attenuation_max) / (1.0f - attenuation_max);
    }
  }
  return 1.0f;
}

float ComputeLinearDistanceAttenuation(const WorldPosition& listener_position,
                                       const WorldPosition& source_position,
                                       float min_distance, float max_distance) {
  const float distance = (listener_position - source_position).norm();
  if (distance > max_distance) {
    return 0.0f;
  }
  // Linear attenuation.
  const float min_distance_allowed =
      std::max(min_distance, kNearFieldThreshold);
  if (distance > min_distance_allowed) {
    const float attenuation_interval = max_distance - min_distance_allowed;
    if (attenuation_interval > kEpsilonFloat) {
      return (max_distance - distance) / attenuation_interval;
    }
  }
  return 1.0f;
}

float ComputeNearFieldEffectGain(const WorldPosition& listener_position,
                                 const WorldPosition& source_position) {
  const float distance = (listener_position - source_position).norm();
  if (distance < kNearFieldThreshold) {
    return (1.0f / std::max(distance, kMinNearFieldDistance)) - 1.0f;
  }
  return 0.0f;
}

void UpdateAttenuationParameters(float primary_gain, float reflections_gain,
                                 float reverb_gain,
                                 const WorldPosition& listener_position,
                                 SourceParameters* parameters) {
  // Compute distance attenuation.
  const WorldPosition& source_position = parameters->object_transform.position;
  const auto rolloff_model = parameters->distance_rolloff_model;
  const float min_distance = parameters->minimum_distance;
  const float max_distance = parameters->maximum_distance;

  float distance_attenuation = 0.0f;
  switch (rolloff_model) {
    case DistanceRolloffModel::kLogarithmic:
      distance_attenuation = ComputeLogarithmicDistanceAttenuation(
          listener_position, source_position, min_distance, max_distance);
      break;
    case DistanceRolloffModel::kLinear:
      distance_attenuation = ComputeLinearDistanceAttenuation(
          listener_position, source_position, min_distance, max_distance);
      break;
    case DistanceRolloffModel::kNone:
    default:
      // Distance attenuation is already set by the user.
      distance_attenuation = parameters->distance_attenuation;
      break;
  }
  // Update gain attenuations.
  const float input_gain = primary_gain * parameters->gain;
  const float direct_attenuation = input_gain * distance_attenuation;
  const float room_effects_attenuation = parameters->room_effects_gain;

  parameters->attenuations[AttenuationType::kInput] = input_gain;
  parameters->attenuations[AttenuationType::kDirect] = direct_attenuation;
  parameters->attenuations[AttenuationType::kReflections] =
      room_effects_attenuation * direct_attenuation * reflections_gain;
  parameters->attenuations[AttenuationType::kReverb] =
      room_effects_attenuation * input_gain * reverb_gain;
}

}  // namespace vraudio
