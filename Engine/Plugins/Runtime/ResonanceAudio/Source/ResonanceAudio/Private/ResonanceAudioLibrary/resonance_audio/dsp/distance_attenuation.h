/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_DISTANCE_ATTENUATION_H_
#define RESONANCE_AUDIO_DSP_DISTANCE_ATTENUATION_H_

#include "base/misc_math.h"
#include "base/source_parameters.h"

namespace vraudio {

// Returns the distance attenuation for |source_position| with respect to
// |listener_position|. The amplitude will decrease by approximately 6 dB every
// time the distance is doubled, i.e., the sound pressure "p" (amplitude)
// approximately falls inversely proportional to the distance "1/r".
//
// @param listener_position World position of the listener.
// @param source_position World position of the source.
// @param min_distance The minimum distance at which distance attenuation is
//     applied.
// @param max_distance The maximum distance at which the direct sound has a gain
//     of 0.0.
// @return Attenuation (gain) value in range [0.0f, 1.0f].
float ComputeLogarithmicDistanceAttenuation(
    const WorldPosition& listener_position,
    const WorldPosition& source_position, float min_distance,
    float max_distance);

// Returns the distance attenuation for |source_position| with respect to
// |listener_position|. The amplitude will decrease linearly between
// |min_distance| and |max_distance| from 1.0 to 0.0.
//
// @param listener_position World position of the listener.
// @param source_position World position of the source.
// @param min_distance The minimum distance at which distance attenuation is
//     applied.
// @param max_distance The maximum distance at which the direct sound has a gain
//     of 0.0.
// @return Attenuation (gain) value in range [0.0f, 1.0f].
float ComputeLinearDistanceAttenuation(const WorldPosition& listener_position,
                                       const WorldPosition& source_position,
                                       float min_distance, float max_distance);

// Calculates the gain to be applied to the near field compensating stereo mix.
// This function will return 0.0f for all sources further away than one meter
// and will return a value between 0.0 and 9.0 for sources as they approach
// the listener's head location.
//
// @param listener_position World position of the listener.
// @param source_position World position of the source.
// @return Gain value in range [0.0f, 9.0f].
float ComputeNearFieldEffectGain(const WorldPosition& listener_position,
                                 const WorldPosition& source_position);

// Calculates and updates gain attenuations of the given source |parameters|.
//
// @param master_gain Global gain adjustment in amplitude.
// @param reflections_gain Reflections gain in amplitude.
// @param reverb_gain Reverb gain in amplitude.
// @param listener_position World position of the listener.
// @param parameters Source parameters to apply the gain attenuations into.
void UpdateAttenuationParameters(float master_gain, float reflections_gain,
                                 float reverb_gain,
                                 const WorldPosition& listener_position,
                                 SourceParameters* parameters);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_DISTANCE_ATTENUATION_H_
