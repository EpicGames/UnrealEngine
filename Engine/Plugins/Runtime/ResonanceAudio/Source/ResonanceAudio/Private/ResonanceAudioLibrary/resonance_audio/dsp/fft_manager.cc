/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/fft_manager.h"

#include <algorithm>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"
#include "base/simd_utils.h"


// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

namespace vraudio {

namespace {

// for |fft_size_|s less than 2^14, the stack is used, on the reccomendation of
// the author of the pffft library.
const size_t kPffftMaxStackSize = 16384;

}  // namespace

// The pffft implementation requires a minimum fft size of 32 samples.
const size_t FftManager::kMinFftSize = 32;

FftManager::FftManager(size_t frames_per_buffer)
    : fft_size_(std::max(NextPowTwo(frames_per_buffer) * 2, kMinFftSize)),
      frames_per_buffer_(frames_per_buffer),
      inverse_fft_scale_(1.0f / static_cast<float>(fft_size_)),
      temp_zeropad_buffer_(kNumMonoChannels, fft_size_),
      temp_freq_buffer_(kNumMonoChannels, fft_size_) {
  DCHECK_GT(frames_per_buffer, 0U);
  DCHECK_GE(fft_size_, kMinFftSize);
  DCHECK(!(fft_size_ & (fft_size_ - 1)));  // Ensure its a power of two.
  // Suggested pffft initialization.
  if (fft_size_ > kPffftMaxStackSize) {
    // Allocate memory for work space factors etc, Size reccomended by pffft.
    const size_t num_bytes = 2 * fft_size_ * sizeof(float);
    pffft_workspace_ =
        reinterpret_cast<float*>(pffft_aligned_malloc(num_bytes));
  }

  fft_ = pffft_new_setup(static_cast<int>(fft_size_), PFFFT_REAL);

  temp_zeropad_buffer_.Clear();
}

FftManager::~FftManager() {
  pffft_destroy_setup(fft_);
  if (pffft_workspace_ != nullptr) {
    pffft_aligned_free(pffft_workspace_);
  }
}

void FftManager::FreqFromTimeDomain(const AudioBuffer::Channel& time_channel,
                                    AudioBuffer::Channel* freq_channel) {


  DCHECK(freq_channel);
  DCHECK_EQ(freq_channel->size(), fft_size_);
  DCHECK_LE(time_channel.size(), fft_size_);

  // Perform forward FFT transform.
  if (time_channel.size() == fft_size_) {
    pffft_transform(fft_, time_channel.begin(), freq_channel->begin(),
                    pffft_workspace_, PFFFT_FORWARD);
  } else {
    std::copy_n(time_channel.begin(), frames_per_buffer_,
                temp_zeropad_buffer_[0].begin());
    pffft_transform(fft_, temp_zeropad_buffer_[0].begin(),
                    freq_channel->begin(), pffft_workspace_, PFFFT_FORWARD);
  }
}

void FftManager::TimeFromFreqDomain(const AudioBuffer::Channel& freq_channel,
                                    AudioBuffer::Channel* time_channel) {


  DCHECK(time_channel);
  DCHECK_EQ(freq_channel.size(), fft_size_);

  // Perform reverse FFT transform.
  const size_t time_channel_size = time_channel->size();
  if (time_channel_size == fft_size_) {
    pffft_transform(fft_, freq_channel.begin(), time_channel->begin(),
                    pffft_workspace_, PFFFT_BACKWARD);
  } else {
    DCHECK_EQ(time_channel_size, frames_per_buffer_);
    auto& temp_channel = temp_freq_buffer_[0];
    pffft_transform(fft_, freq_channel.begin(), temp_channel.begin(),
                    pffft_workspace_, PFFFT_BACKWARD);
    std::copy_n(temp_channel.begin(), frames_per_buffer_,
                time_channel->begin());
  }
}

void FftManager::ApplyReverseFftScaling(AudioBuffer::Channel* time_channel) {
  DCHECK(time_channel->size() == frames_per_buffer_ ||
         time_channel->size() == fft_size_);
  // Normalization must be performed here as we normally do this as part of the
  // convolution.
  ScalarMultiply(time_channel->size(), inverse_fft_scale_,
                 time_channel->begin(), time_channel->begin());
}

void FftManager::GetCanonicalFormatFreqBuffer(const AudioBuffer::Channel& input,
                                              AudioBuffer::Channel* output) {
  DCHECK_EQ(input.size(), fft_size_);
  DCHECK_EQ(output->size(), fft_size_);
  pffft_zreorder(fft_, input.begin(), output->begin(), PFFFT_FORWARD);
}

void FftManager::GetPffftFormatFreqBuffer(const AudioBuffer::Channel& input,
                                          AudioBuffer::Channel* output) {
  DCHECK_EQ(input.size(), fft_size_);
  DCHECK_EQ(output->size(), fft_size_);
  pffft_zreorder(fft_, input.begin(), output->begin(), PFFFT_BACKWARD);
}

void FftManager::MagnitudeFromCanonicalFreqBuffer(
    const AudioBuffer::Channel& freq_channel,
    AudioBuffer::Channel* magnitude_channel) {
  DCHECK(magnitude_channel);
  DCHECK_EQ(freq_channel.size(), fft_size_);
  DCHECK_EQ(magnitude_channel->size(), frames_per_buffer_ + 1);

  (*magnitude_channel)[0] = std::abs(freq_channel[0]);
  ApproxComplexMagnitude(frames_per_buffer_ - 1, freq_channel.begin() + 2,
                         magnitude_channel->begin() + 1);
  (*magnitude_channel)[frames_per_buffer_] = std::abs(freq_channel[1]);
}

void FftManager::CanonicalFreqBufferFromMagnitudeAndPhase(
    const AudioBuffer::Channel& magnitude_channel,
    const AudioBuffer::Channel& phase_channel,
    AudioBuffer::Channel* canonical_freq_channel) {
  DCHECK(canonical_freq_channel);
  DCHECK_EQ(magnitude_channel.size(), frames_per_buffer_ + 1);
  DCHECK_EQ(phase_channel.size(), frames_per_buffer_ + 1);
  DCHECK_EQ(canonical_freq_channel->size(), fft_size_);

  (*canonical_freq_channel)[0] = magnitude_channel[0];
  (*canonical_freq_channel)[1] = -magnitude_channel[frames_per_buffer_];
  for (size_t i = 1, j = 2; i < frames_per_buffer_; ++i, j += 2) {
    (*canonical_freq_channel)[j] =
        magnitude_channel[i] * std::cos(phase_channel[i]);
    (*canonical_freq_channel)[j + 1] =
        magnitude_channel[i] * std::sin(phase_channel[i]);
  }
}

void FftManager::CanonicalFreqBufferFromMagnitudeAndSinCosPhase(
    size_t phase_offset, const AudioBuffer::Channel& magnitude_channel,
    const AudioBuffer::Channel& sin_phase_channel,
    const AudioBuffer::Channel& cos_phase_channel,
    AudioBuffer::Channel* canonical_freq_channel) {
  static const size_t kSimdLength = 4;
  DCHECK(canonical_freq_channel);
  DCHECK_EQ(magnitude_channel.size(), frames_per_buffer_ + 1);
  DCHECK_GE(sin_phase_channel.size() + phase_offset, frames_per_buffer_ + 1);
  DCHECK_GE(cos_phase_channel.size() + phase_offset, frames_per_buffer_ + 1);
  DCHECK_EQ(canonical_freq_channel->size(), fft_size_);

  (*canonical_freq_channel)[0] = magnitude_channel[0];
  (*canonical_freq_channel)[1] = -magnitude_channel[frames_per_buffer_];
  // Continue on till we can gaurantee alignment in our audio buffer.
  for (size_t i = 1, j = 2; i <= kSimdLength; ++i, j += 2) {
    (*canonical_freq_channel)[j] =
        magnitude_channel[i] * cos_phase_channel[i + phase_offset];
    (*canonical_freq_channel)[j + 1] =
        magnitude_channel[i] * sin_phase_channel[i + phase_offset];
  }
  ComplexInterleavedFormatFromMagnitudeAndSinCosPhase(
      2 * (frames_per_buffer_ - kSimdLength), &magnitude_channel[kSimdLength],
      &cos_phase_channel[kSimdLength + phase_offset],
      &sin_phase_channel[kSimdLength + phase_offset],
      &(*canonical_freq_channel)[2 * kSimdLength]);
}

void FftManager::FreqDomainConvolution(const AudioBuffer::Channel& input_a,
                                       const AudioBuffer::Channel& input_b,
                                       AudioBuffer::Channel* scaled_output) {
  DCHECK_EQ(input_a.size(), fft_size_);
  DCHECK_EQ(input_b.size(), fft_size_);
  DCHECK_EQ(scaled_output->size(), fft_size_);
  pffft_zconvolve_accumulate(fft_, input_a.begin(), input_b.begin(),
                             scaled_output->begin(), inverse_fft_scale_);
}

}  // namespace vraudio
