/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_FFT_MANAGER_H_
#define RESONANCE_AUDIO_DSP_FFT_MANAGER_H_

#include "pffft.h"
#include "base/audio_buffer.h"

namespace vraudio {

// This class wraps the pffft library and enables reall FFT transformations to
// be performed on aligned float buffers of data. The class also manages all
// necessary data buffers and zero padding. This class is not thread safe.
class FftManager {
 public:
  // Minimum required FFT size.
  static const size_t kMinFftSize;

  // Constructs a FftManager insatnce. One instance of this class can be shared.
  // This class is not thread safe.
  //
  // @param frames_per_buffer System's number of frames per buffer.
  explicit FftManager(size_t frames_per_buffer);

  // Destroys a FftManager instance freeing associated aligned memory.
  ~FftManager();

  // Transforms a single channel of time domain input data into a frequency
  // domain representation.
  //
  // @param time_channel Time domain input. If the length is less than
  //     |fft_size_|, the input is zeropadded. The max length is |fft_size_|.
  // @param freq_channel Frequency domain output, |fft_size| samples long.
  void FreqFromTimeDomain(const AudioBuffer::Channel& time_channel,
                          AudioBuffer::Channel* freq_channel);

  // Transforms a single channel of frequency domain input data into a time
  // domain representation. Note: The input must be in pffft format see:
  // goo.gl/LYbgX7. This method can output to a buffer of either
  // |frames_per_buffer_| or |fft_size_| in length. This feature ensures an
  // additional copy is not needed where this method is to be used with an
  // overlap add.
  //
  // @param freq_channel Frequency domain input, |fft_size| samples long.
  // @param time_channel Time domain output, |frames_per_buffer_| samples long
  //     OR |fft_size_| samples long.
  void TimeFromFreqDomain(const AudioBuffer::Channel& freq_channel,
                          AudioBuffer::Channel* time_channel);

  // Applies a 1/|fft_size_| scaling to time domain output. NOTE this need not
  // be applied where a convolution is taking place as the scaling will be
  // included therein.
  //
  // @param time_channel Time domain data to be scaled.
  void ApplyReverseFftScaling(AudioBuffer::Channel* time_channel);

  // Transforms a pffft frequency domain format buffer into canonical format
  // with alternating real and imaginary values with increasing frequency. The
  // first two entries of |output| are the real part of the DC and Nyquist
  // frequencies (imaginary part is zero). The alternating real and imaginary
  // parts start from the third entry in |output|. For more info on the pffft
  // format see: goo.gl/LYbgX7
  //
  // @param input Frequency domain input channel, |fft_size| samples long.
  // @param output Frequency domain output channel,|fft_size| samples long.
  void GetCanonicalFormatFreqBuffer(const AudioBuffer::Channel& input,
                                    AudioBuffer::Channel* output);

  // Transforms a canonical frequency domain format buffer into pffft format.
  // For more info on the pffft format see: goo.gl/LYbgX7
  //
  // @param input Frequency domain input channel, |fft_size| samples long.
  // @param output Frequency domain output channel, |fft_size| samples long.
  void GetPffftFormatFreqBuffer(const AudioBuffer::Channel& input,
                                AudioBuffer::Channel* output);

  // Genarates a buffer containing the single sided magnitude spectrum of a
  // frequency domain buffer. The input must be in Canonical format. The output
  // will have DC frequency as it's first entry and the Nyquist as it's last.
  //
  // @param freq_channel Canonical format frequency domain buffer,
  //     |fft_size_| samples long.
  // @param magnitude_channel Magnitude of the |freq_channel|.
  //     |frames_per_buffer_| + 1 samples long.
  void MagnitudeFromCanonicalFreqBuffer(
      const AudioBuffer::Channel& freq_channel,
      AudioBuffer::Channel* magnitude_channel);

  // Combines single sided magnitude and phase spectra into a canonical format
  // frequency domain buffer. The inputs must have DC frequency as their first
  // entry and the Nyquist as their last.
  //
  // @param magnitude_channel Magnitude of the |frequency_buffer|.
  //     |frames_per_buffer_| + 1 samples long.
  // @param phase_channel Phase of the |frequency_buffer|.
  //     |frames_per_buffer_| + 1 samples long.
  // @param canonical_freq_channel Canonical format frequency domain buffer,
  //     |fft_size_| samples long.
  void CanonicalFreqBufferFromMagnitudeAndPhase(
      const AudioBuffer::Channel& magnitude_channel,
      const AudioBuffer::Channel& phase_channel,
      AudioBuffer::Channel* canonical_freq_channel);

  // Combines single sided magnitude spectrum and the cosine and sine of a phase
  // spectrum into a canonical format frequency domain buffer. The inputs must
  // have DC frequency as their first entry and the Nyquist as their last.
  // The phase spectra channels can be offset by |phase_offset|. This feature
  // is specifically for use as an optimization in the |SpectralReverb|.
  //
  // @param phase_offset An offset into the channels of the phase buffer.
  // @param magnitude_channel Magnitude of the |frequency_buffer|.
  //     |frames_per_buffer_| + 1 samples long.
  // @param sin_phase_channel Sine of the phase of the |frequency_buffer|.
  //     |frames_per_buffer_| + 1 samples long.
  // @param cos_phase_channel Cosine of the phase of the |frequency_buffer|.
  //     |frames_per_buffer_| + 1 samples long.
  // @param canonical_freq_channel Canonical format frequency domain buffer,
  //     |fft_size_| samples long.
  void CanonicalFreqBufferFromMagnitudeAndSinCosPhase(
      size_t phase_offset, const AudioBuffer::Channel& magnitude_channel,
      const AudioBuffer::Channel& sin_phase_channel,
      const AudioBuffer::Channel& cos_phase_channel,
      AudioBuffer::Channel* canonical_freq_channel);

  // Performs a pointwise complex multiplication of two frequency domain buffers
  // and applies tha inverse scaling factor of 1/|fft_size_|. This operation is
  // equivalent to a time domain circular convolution.
  //
  // @param input_a Frequency domain input channel, |fft_size| samples long.
  // @param input_b Frequency domain input channel, |fft_size| samples long.
  // @param scaled_output Frequency domain output channel, |fft_size| samples
  //     long.
  void FreqDomainConvolution(const AudioBuffer::Channel& input_a,
                             const AudioBuffer::Channel& input_b,
                             AudioBuffer::Channel* scaled_output);

  // Returns the number of points in the FFT.
  size_t GetFftSize() const { return fft_size_; }

 private:
  // FFT size in samples.
  const size_t fft_size_;

  // Number of frames in each buffer of input data.
  const size_t frames_per_buffer_;

  // Inverse scale to be applied to buffers transformed from frequency to time
  // domain.
  const float inverse_fft_scale_;

  // Temporary time domain buffer to store zeropadded input.
  AudioBuffer temp_zeropad_buffer_;

  // Temporary freq domain buffer to store.
  AudioBuffer temp_freq_buffer_;

  // pffft states.
  PFFFT_Setup* fft_;

  // Workspace for pffft. This pointer should be set to null for |fft_size_|
  // less than 2^14. In which case the stack is used. This is the recommendation
  // by the author of the pffft library.
  float* pffft_workspace_ = nullptr;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_FFT_MANAGER_H_
