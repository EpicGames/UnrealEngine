/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/filter_coefficient_generators.h"

#include <algorithm>
#include <cmath>

#include "base/constants_and_types.h"

#ifndef M_PI
#define DEFINED_M_PI
#define M_PI 3.14159265358979323846
#endif

namespace vraudio {

namespace {

// Hard coded 4th order fit calculated from:

const int kPolynomialOrder = 4;
const float kPolynomialCoefficients[] = {
    2.52730826376129e-06f, 0.00018737263228963f, 0.00618822707412605f,
    0.113947188715447f, 0.999048314740445f};

// Threshold frequency for human hearing 20Hz.
const float kTwentyHz = 20.0f;

// ln(2)/2 for use in bandpass filter.
const float kLn2over2 = 0.346573590279973f;

}  // namespace

BiquadCoefficients ComputeBandPassBiquadCoefficients(int sample_rate,
                                                     float center_frequency,
                                                     int bandwidth) {
  DCHECK_GT(sample_rate, 0);
  DCHECK_GE(center_frequency, 0.0f);
  DCHECK_GT(bandwidth, 0);
  // Check if an invalid |center_frequency|, greater than or equal to the
  // Nyquist rate is passed as input.
  CHECK_LT(center_frequency, 0.5f * static_cast<float>(sample_rate));

  // Frequency of interest (in radians).
  const float w0 = kTwoPi * center_frequency / static_cast<float>(sample_rate);
  // Intermediate storage.
  const float cos_w0 = std::cos(w0);
  const float sin_w0 = std::sin(w0);
  const float alpha =
      sin_w0 * sinhf(kLn2over2 * static_cast<float>(bandwidth) * w0 / sin_w0);

  // The BiquadFilterState constructor is passed (a0, a1, a2, b0, b1, b2).
  return BiquadCoefficients(1.0f + alpha, -2.0f * cos_w0, 1.0f - alpha, alpha,
                            0.0f, -alpha);
}

void ComputeDualBandBiquadCoefficients(
    int sample_rate, float crossover_frequency,
    BiquadCoefficients* low_pass_coefficients,
    BiquadCoefficients* high_pass_coefficients) {
  DCHECK_GT(sample_rate, 0);
  DCHECK_GE(crossover_frequency, 0.0f);
  DCHECK_LE(crossover_frequency, static_cast<float>(sample_rate) / 2.0f);
  DCHECK(low_pass_coefficients);
  DCHECK(high_pass_coefficients);

  const float k = std::tan(static_cast<float>(M_PI) * crossover_frequency /
                           static_cast<float>(sample_rate));
  const float k_squared = k * k;
  const float denominator = k_squared + 2.0f * k + 1.0f;

  // |denominator| must not be near 0. Since |k| is always guaranteed to be in
  // the range  0 < k < pi/2, the |denominator| should always be >=1. This is a
  // sanity check only.
  DCHECK_GT(denominator, kEpsilonDouble);

  // Computes numerator coefficients of the low-pass |low_pass_coefficients|
  // bi-quad.
  low_pass_coefficients->a[0] = 1.0f;
  low_pass_coefficients->a[1] = 2.0f * (k_squared - 1.0f) / denominator;
  low_pass_coefficients->a[2] = (k_squared - 2.0f * k + 1.0f) / denominator;

  // Numerator coefficients of the high-pass |high_pass_coefficients| bi-quad
  // are the same.
  BiquadCoefficients high_pass;
  std::copy(low_pass_coefficients->a.begin(), low_pass_coefficients->a.end(),
            high_pass_coefficients->a.begin());

  // Computes denominator coefficients of the low-pass |low_pass_coefficients|
  // bi-quad.
  low_pass_coefficients->b[0] = k_squared / denominator;
  low_pass_coefficients->b[1] = 2.0f * low_pass_coefficients->b[0];
  low_pass_coefficients->b[2] = low_pass_coefficients->b[0];

  // Computes denominator coefficients of the high-pass |high_pass_coefficients|
  // bi-quad.
  high_pass_coefficients->b[0] = 1.0f / denominator;
  high_pass_coefficients->b[1] = -2.0f * high_pass_coefficients->b[0];
  high_pass_coefficients->b[2] = high_pass_coefficients->b[0];
}

BiquadCoefficients ComputeLowPassBiquadCoefficients(
    int sample_rate, float specification_frequency, float attenuation) {
  DCHECK_GT(sample_rate, 0);
  DCHECK_GE(specification_frequency, 0.0f);
  DCHECK_LE(specification_frequency, static_cast<float>(sample_rate) / 2.0f);
  DCHECK_LT(attenuation, 0.0f);

  // Frequency of interest (in radians).
  const float w0 =
      kTwoPi * specification_frequency / static_cast<float>(sample_rate);

  // Q is the Q-factor. For more information see "Digital Signal Processing" -
  // J. G. Prolakis, D. G. Manolakis - Published by Pearson.
  float Q = 0.0f;

  // Variable to handle the growth in power as one extra mult per iteration
  // across the polynomial coefficients.
  float attenuation_to_a_power = 1.0f;

  // Add in each term in reverse order.
  for (int order = kPolynomialOrder; order >= 0; --order) {
    Q += kPolynomialCoefficients[order] * attenuation_to_a_power;
    attenuation_to_a_power *= attenuation;
  }

  // Intermediate storage of commonly used values.
  const float alpha = std::sin(w0) / (2.0f * Q);
  const float cos_w0 = std::cos(w0);

  // Filter coefficients.
  float a0 = 1.0f + alpha;
  float a1 = -2.0f * cos_w0;
  float a2 = 1.0f - alpha;
  // Coefficients b0 and b2 will have the same value in this case.
  float b0_b2 = (1.0f - cos_w0) / 2.0f;
  float b1 = 1.0f - cos_w0;

  return BiquadCoefficients(a0, a1, a2, b0_b2, b1, b0_b2);
}

float ComputeLowPassMonoPoleCoefficient(float cuttoff_frequency,
                                        int sample_rate) {
  float coefficient = 0.0f;
  // Check that the cuttoff_frequency provided is not too low (below human
  // threshold, also danger of filter instability).
  if (cuttoff_frequency > kTwentyHz) {
    const float inverse_time_constant = kTwoPi * cuttoff_frequency;
    const float sample_rate_float = static_cast<float>(sample_rate);
    coefficient =
        sample_rate_float / (inverse_time_constant + sample_rate_float);
  }
  return coefficient;
}

}  // namespace vraudio

#ifdef DEFINED_M_PI
#undef M_PI
#endif
