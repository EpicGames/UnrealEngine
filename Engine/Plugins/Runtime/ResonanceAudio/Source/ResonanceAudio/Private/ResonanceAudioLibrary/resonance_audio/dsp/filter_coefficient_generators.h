/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_FILTER_COEFFICIENT_GENERATORS_H_
#define RESONANCE_AUDIO_DSP_FILTER_COEFFICIENT_GENERATORS_H_

#include "dsp/biquad_filter.h"

// Functions for the generation of filter coefficients for various common tasks.
// Currently supports the following filter types:
// Low pass first-order filter.
// Low pass biquad filter.
// Band pass biquad filter.
// Dual band, matched phase biquad shelf filters.
namespace vraudio {

// Computes corresponding biquad coefficients for band pass filter with respect
// to the centre frequency and the bandwidth between -3 dB frequencies.
//
//           b0 + b1*z^-1 + b2*z^-2
//   H(z) = ------------------------
//           a0 + a1*z^-1 + a2*z^-2
//
// where:
//       w0 = 2*pi*center_frequency/sample_rate
//
//       alpha = sin(w0)*sinh( ln(2)/2 * bandwidth * w0/sin(w0) )
//
//    b0 =   alpha
//    b1 =   0
//    b2 =  -alpha
//    a0 =   1 + alpha
//    a1 =  -2*cos(w0)
//    a2 =   1 - alpha
//
// @param sample_rate Sampling rate in Hz.
// @param centre_frequency Centre frequency of passband.
// @param bandwidth Bandwidth in octaves between -3 dB frequencies.
// @return Output structure of band-pass BiquadCoefficients.
BiquadCoefficients ComputeBandPassBiquadCoefficients(int sample_rate,
                                                     float centre_frequency,
                                                     int bandwidth);

// Computes two sets of transfer function coefficients to be used with a
// pair of generic bi-quad filters. The coefficients are used to implement a
// phase-matched low-pass |low_pass_state| and high-pass |high_pass_state|
// filter pair with a cross-over frequency defined as |crossover_frequency|.
//
// Implementation of the matched bi-quad filter pair as described in:
// http://www.ai.sri.com/ajh/ambisonics/BLaH3.pdf
//
//           b0 + b1*z^-1 + b2*z^-2
//   H(z) = ------------------------
//           a0 + a1*z^-1 + a2*z^-2
//
// where:
//
// a0 = 1
//
//        2(k^2 − 1)
// a1 = --------------
//       k^2 + 2k + 1
//
//       k^2 - 2k + 1
// a2 = --------------
//       k^2 + 2k + 1
//
// low-pass:                   high-pass:
//
//           k^2                          1
// b0 = -------------          b0 = --------------
//       k^2 + 2k + 1                k^2 + 2k + 1
//
// b1 = 2b0                    b1 = -2b0
//
// b2 = b0                     b2 = b0
//
// and
//
//          pi * crossover_frequency
// k = tan --------------------------
//              sample_frequency
//
// @param sample_rate Sampling rate in [Hz]
// @param crossover_frequency Cross-over frequency in [Hz]
// @param low_pass_coefficients Output structure of low-pass bi-quad
// coefficients
// @param high_pass_coefficients Output structure of high-pass bi-quad
// coefficients.
void ComputeDualBandBiquadCoefficients(
    int sample_rate, float crossover_frequency,
    BiquadCoefficients* low_pass_coefficients,
    BiquadCoefficients* high_pass_coefficients);

// Computes biquad coefficients for low pass filter with respect to the
// specification frequency and the attenuation value at that frequency.
//
//           b0 + b1*z^-1 + b2*z^-2
//   H(z) = ------------------------
//           a0 + a1*z^-1 + a2*z^-2
//
// where:
//        Q = 2.5273e-06*attenuation^4 + 0.00018737*attenuation^3 +
//        0.0061882*attenuation^2 + 0.11395*attenuation + 0.99905
//
//        w0 = 2*pi*specification_frequency/sample_rate;
//
//        alpha = sin(w0)/(2*Q);
//
//     a0 =   1 + alpha;
//     a1 =  -2*cos(w0);
//     a2 =   1 - alpha;
//     b0 =  (1 - cos(w0))/2;
//     b1 =   1 - cos(w0);
//     b2 =  (1 - cos(w0)/2;
//
// These coefficients were generated after a bilinerar transform of an
// analogue filter  H(s) = 1 / (s^2 + s/Q + 1),  from:
// www.analog.com/library/analogdialogue/archives/43-09/edch%208%20filter.pdf.
//
// Please note Q has been calculated by fitting a 4th order polynomial to the
// average of Q vs attenuation curves at different f0's from 10kHz to 19kHz.
// Please note that at differing frequencies these graphs had identical shape
// and differed only by an overall offset of at most 0.1 (attenuation) dB from
// the mean. This script can be found at

//
// @param sample_rate Sampling rate in Hz.
// @param specification_frequency Frequency at which attenuation applies in Hz.
// @param attenuation Attenuation at specification_frequency in negative dB.
// @return low_pass Output structure of low-pass BiquadCoefficients.
BiquadCoefficients ComputeLowPassBiquadCoefficients(
    int sample_rate, float specification_frequency, float attenuation);

// Generates a coefficient for the |MonoPoleFilterClass| based on a 3dB
// bandwidth.
//
// The Laplace transfer function of a first order low pass system is:
//
//         1
//    ------------     where tau is the RC time constant of the system.
//    1 + tau * s
//
// For a discrete moving average filter with input x[n] and output y[n],
// the difference equation is:
//                               y[n] =  a * y[n - 1] + (1 - a) * x[n]
//         tau
//   a = ----------   where T is the sample period.
//        tau + T
//                   since the -3dB bandwith of a first order system can be
//           1       related to its time constant by
// f3 = --------------
//      2 * pi * tau       we can obtain 'a' from cuttoff_frequency and
//                          sample rate.
//
// @param cuttoff_frequency The -3dB frequency in Hz of the low pass lobe.
// @param sample_rate System sampling rate.
// @return A |MonoPoleFilterClass| coefficient for the specified bandwidth.
float ComputeLowPassMonoPoleCoefficient(float cuttoff_frequency,
                                        int sample_rate);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_FILTER_COEFFICIENT_GENERATORS_H_
