/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "dsp/fir_filter.h"

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_macros.h"

namespace vraudio {

FirFilter::FirFilter(const AudioBuffer::Channel& filter_coefficients,
                     size_t frames_per_buffer) {
  DCHECK_GE(filter_coefficients.size(), 1U);
  // Create the kernel buffer in repeated entry representation from standard
  // form FIR representation.
  const size_t coefficients_length = filter_coefficients.size();
  filter_length_ = coefficients_length +
                   ((coefficients_length % SIMD_LENGTH == 0)
                        ? 0
                        : SIMD_LENGTH - (coefficients_length % SIMD_LENGTH));
  num_filter_chunks_ = filter_length_ / SIMD_LENGTH;
  DCHECK_EQ(filter_length_ % SIMD_LENGTH, 0);
  coefficients_ = AudioBuffer(kNumMonoChannels, filter_length_ * SIMD_LENGTH);
  AudioBuffer::Channel* coefficients = &(coefficients_[0]);
  // Store the coefficients so that each individual coefficient is repeated
  // SIMD_LENGTH times.
  for (size_t coeff = 0; coeff < coefficients_length; ++coeff) {
    auto coefficient_iter = coefficients->begin() + (coeff * SIMD_LENGTH);
    std::fill(coefficient_iter, coefficient_iter + SIMD_LENGTH,
              filter_coefficients[coeff]);
  }
  std::fill(coefficients->begin() + (coefficients_length * SIMD_LENGTH),
            coefficients->end(), 0.0f);
  // Allocate an aligned buffer with |filter_length_| extra samples to
  // store previous input samples.
  state_ = AudioBuffer(kNumMonoChannels, frames_per_buffer + filter_length_);
  state_.Clear();
}

void FirFilter::Process(const AudioBuffer::Channel& input,
                        AudioBuffer::Channel* output) {
  DCHECK(output);
  DCHECK_EQ(input.size(), state_.num_frames() - filter_length_);
  // In this case we know that SIMD_LENGTH == 1, therefore we don't need to take
  // account of it.
  const size_t input_length = input.size();
  std::copy_n(state_[0].end() - filter_length_, filter_length_,
              state_[0].begin());
  std::copy_n(input.begin(), input_length, state_[0].begin() + filter_length_);
#if defined(SIMD_DISABLED)
  AudioBuffer::Channel* coefficients = &(coefficients_[0]);
  AudioBuffer::Channel* data = &(state_[0]);
  for (size_t frame = 0; frame < input_length; ++frame) {
    for (size_t coeff = 0; coeff < filter_length_; ++coeff) {
      (*output)[frame] +=
          (*coefficients)[coeff] * (*data)[filter_length_ - coeff];
    }
  }
#else
  const SimdVector* cptr_input =
      reinterpret_cast<const SimdVector*>(&(state_[0][0]));
  const SimdVector* cptr_filter =
      reinterpret_cast<const SimdVector*>(&(coefficients_[0][0]));
  SimdVector* ptr_output = reinterpret_cast<SimdVector*>(&((*output)[0]));

  const size_t num_input_chunks = input_length / SIMD_LENGTH;
  DCHECK_EQ(input_length % SIMD_LENGTH, 0);

  // A pair of |SimdVector|s one for holding the input for each operation, the
  // other to store data that needed to be copied as it straddled a four float
  // boundary.
  SimdVector input_use;

#if defined(SIMD_SSE)
  SimdVector input_split;
#endif

  for (size_t input_position = num_filter_chunks_;
       input_position < num_input_chunks + num_filter_chunks_;
       ++input_position) {
    // Replace these with the indexed pointers to save on copies.
    SimdVector* output_now = &ptr_output[input_position - num_filter_chunks_];
    DCHECK_GE(input_position, num_filter_chunks_);

    for (size_t chunk = 0; chunk < num_filter_chunks_; ++chunk) {
      const size_t filter_index_offset = chunk * SIMD_LENGTH;
      const SimdVector* input_a = &cptr_input[input_position - (chunk + 1)];
      const SimdVector* input_b = &cptr_input[input_position - chunk];
#if defined(SIMD_SSE)
      // Select four floats from input_a and input_b, based on the mask. Here we
      // take the latter two entries from input_b followed by the first two
      // entries of input_a.
      input_split = _mm_shuffle_ps(*input_a, *input_b, _MM_SHUFFLE(1, 0, 3, 2));

      // All from input_b.
      *output_now = SIMD_MULTIPLY_ADD(
          *input_b, cptr_filter[filter_index_offset], *output_now);
      // One from input_a and the rest (three) from input_b.
      input_use =
          _mm_shuffle_ps(input_split, *input_b, _MM_SHUFFLE(2, 1, 2, 1));
      *output_now = SIMD_MULTIPLY_ADD(
          input_use, cptr_filter[1 + filter_index_offset], *output_now);
      // Two from input_a and two from input_b.
      *output_now = SIMD_MULTIPLY_ADD(
          input_split, cptr_filter[2 + filter_index_offset], *output_now);
      // Three from input_a and one from input_b.
      input_use =
          _mm_shuffle_ps(*input_a, input_split, _MM_SHUFFLE(2, 1, 2, 1));
      *output_now = SIMD_MULTIPLY_ADD(
          input_use, cptr_filter[3 + filter_index_offset], *output_now);
#elif defined(SIMD_NEON)
      // All from input_b.
      *output_now = SIMD_MULTIPLY_ADD(
          *input_b, cptr_filter[filter_index_offset], *output_now);
      // One from input_a and the rest (three) from input_b.
      input_use = vextq_f32(*input_a, *input_b, 3);
      *output_now = SIMD_MULTIPLY_ADD(
          input_use, cptr_filter[1 + filter_index_offset], *output_now);
      // Two from input_a and two from input_b.
      input_use = vextq_f32(*input_a, *input_b, 2);
      *output_now = SIMD_MULTIPLY_ADD(
          input_use, cptr_filter[2 + filter_index_offset], *output_now);
      // Three from input_a and one from input_b.
      input_use = vextq_f32(*input_a, *input_b, 1);
      *output_now = SIMD_MULTIPLY_ADD(
          input_use, cptr_filter[3 + filter_index_offset], *output_now);
#endif  // SIMD_SSE/SIMD_NEON
    }
  }
#endif  // SIMD_DISABLED
}

}  // namespace vraudio
