/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_FIR_FILTER_H_
#define RESONANCE_AUDIO_DSP_FIR_FILTER_H_

#include "base/audio_buffer.h"

namespace vraudio {

class FirFilter {
 public:
  // Constructs a |FirFilter| from a mono |AudioBuffer| of filter coefficients.
  //
  // @param filter_coefficients FIR filter coefficients.
  // @param frames_per_buffer Number of frames of data in each audio buffer.
  FirFilter(const AudioBuffer::Channel& filter_coefficients,
            size_t frames_per_buffer);

  // Filters an array of input with a finite impulse response (FIR) filter in
  // the time domain. All pointers and lengths passed must be SIMD compatible.
  //
  // @param input Pointer to an aray of input.
  // @param output Pointer to a cleared block of memory of the input length.
  void Process(const AudioBuffer::Channel& input, AudioBuffer::Channel* output);

  // Returns the length of the filter kernel after zeropadding.
  //
  // @return Length of the FIR filter in frames.
  size_t filter_length() const { return filter_length_; }

 private:
  // Length of the filter kernel in frames after zeropadding.
  size_t filter_length_;

  // Number of filter chunks equivalent to the filter length after zeropadding
  // divided by the SIMD_LENGTH.
  size_t num_filter_chunks_;

  // Coefficients of the filter stored in a repeated format.
  AudioBuffer coefficients_;

  // Aligned buffer of previous and current input.
  AudioBuffer state_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_FIR_FILTER_H_
