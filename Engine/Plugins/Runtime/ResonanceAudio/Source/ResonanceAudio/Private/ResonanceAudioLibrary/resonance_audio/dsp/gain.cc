/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/gain.h"

#include "base/constants_and_types.h"
#include "base/simd_macros.h"
#include "base/simd_utils.h"

namespace vraudio {

float LinearGainRamp(size_t ramp_length, float start_gain, float end_gain,
                     const AudioBuffer::Channel& input_samples,
                     AudioBuffer::Channel* output_samples,
                     bool accumulate_output) {
  DCHECK(output_samples);
  DCHECK_EQ(input_samples.size(), output_samples->size());
  DCHECK_GT(ramp_length, 0U);

  const size_t process_length = std::min(ramp_length, input_samples.size());
  const float gain_increment_per_sample =
      (end_gain - start_gain) / static_cast<float>(ramp_length);

  float current_gain = start_gain;
  if (accumulate_output) {
    for (size_t frame = 0; frame < process_length; ++frame) {
      (*output_samples)[frame] += current_gain * input_samples[frame];
      current_gain += gain_increment_per_sample;
    }
  } else {
    for (size_t frame = 0; frame < process_length; ++frame) {
      (*output_samples)[frame] = current_gain * input_samples[frame];
      current_gain += gain_increment_per_sample;
    }
  }

  return current_gain;
}

void ConstantGain(size_t offset_index, float gain,
                  const AudioBuffer::Channel& input_samples,
                  AudioBuffer::Channel* output_samples,
                  bool accumulate_output) {
  DCHECK(output_samples);
  const size_t input_size = input_samples.size();
  DCHECK_EQ(input_size, output_samples->size());
  DCHECK_LT(offset_index, input_size);

  // Apply gain to samples at the beginning, prior to SIMD_LENGTH alignment.
  const size_t unaligned_samples = SIMD_LENGTH - (offset_index % SIMD_LENGTH);
  const size_t offset_index_simd =
      std::min(input_size, offset_index + unaligned_samples);
  if (accumulate_output) {
    for (size_t i = offset_index; i < offset_index_simd; ++i) {
      (*output_samples)[i] += input_samples[i] * gain;
    }
  } else {
    for (size_t i = offset_index; i < offset_index_simd; ++i) {
      (*output_samples)[i] = input_samples[i] * gain;
    }
  }

  if (offset_index_simd == input_size) {
    // Return if there are no remaining operations to carry out.
    return;
  }

  const size_t aligned_length = input_size - offset_index_simd;
  const float* aligned_input = &(input_samples[offset_index_simd]);
  float* aligned_output = &(*output_samples)[offset_index_simd];

  // Apply gain via SIMD operations.
  if (accumulate_output) {
    ScalarMultiplyAndAccumulate(aligned_length, gain, aligned_input,
                                aligned_output);
  } else {
    ScalarMultiply(aligned_length, gain, aligned_input, aligned_output);
  }
}

bool IsGainNearZero(float gain) {
  return std::abs(gain) < kNegative60dbInAmplitude;
}

bool IsGainNearUnity(float gain) {
  return std::abs(1.0f - gain) < kNegative60dbInAmplitude;
}

}  // namespace vraudio
