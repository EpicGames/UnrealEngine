/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_GAIN_H_
#define RESONANCE_AUDIO_DSP_GAIN_H_

#include <cstddef>
#include "base/audio_buffer.h"

namespace vraudio {

// Implements a linearly-interpolated application of gain to a buffer channel.
//
// @param ramp_length Length of interpolation ramp in samples. Must be > 0.
// @param start_gain Starting gain value for ramp.
// @param end_gain Finishing gain value for ramp.
// @param input_samples Channel buffer to which interpolated gain is applied.
// @param output_samples Channel buffer to contain scaled output.
// @param accumulate_output True if the processed input should be mixed into the
//     output. Otherwise, the output will be replaced by the processed input.
// @return Next gain value to be applied to the buffer channel.
float LinearGainRamp(size_t ramp_length, float start_gain, float end_gain,
                     const AudioBuffer::Channel& input_samples,
                     AudioBuffer::Channel* output_samples,
                     bool accumulate_output);

// Applies a gain value to a vector of buffer samples starting at some offset.
//
// @param offset_index Starting index for gain application in buffer.
// @param gain Gain value applied to samples.
// @param input_samples Channel buffer to which gain is applied.
// @param output_samples Channel buffer to contain scaled output.
// @param accumulate_output True if the processed input should be mixed into the
//     output. Otherwise, the output will be replaced by the processed input.
void ConstantGain(size_t offset_index, float gain,
                  const AudioBuffer::Channel& input_samples,
                  AudioBuffer::Channel* output_samples, bool accumulate_output);

// Checks if the gain factor is close enough to zero (less than -60 decibels).
//
// @param gain Gain value to be tested.
// @return true if the current gain factors are near zero, false otherwise.
bool IsGainNearZero(float gain);

// Checks if the gain state is close enough to Unity (less than -60 decibels
// below or above).
//
// @param gain Gain value to be tested.
// @return true if the current gain factors are near unity, false otherwise.
bool IsGainNearUnity(float gain);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_GAIN_H_
