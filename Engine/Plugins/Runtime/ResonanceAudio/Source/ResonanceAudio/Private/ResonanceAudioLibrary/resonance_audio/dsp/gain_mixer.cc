/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/gain_mixer.h"

#include <cmath>

#include "base/logging.h"

namespace vraudio {

GainMixer::GainMixer(size_t num_channels, size_t frames_per_buffer)
    : num_channels_(num_channels),
      output_(num_channels_, frames_per_buffer),
      is_empty_(false) {
  DCHECK_NE(num_channels_, 0U);
  Reset();
}

void GainMixer::AddInput(const AudioBuffer& input,
                         const std::vector<float>& gains) {
  DCHECK_EQ(gains.size(), num_channels_);
  DCHECK_EQ(input.num_channels(), num_channels_);
  DCHECK_EQ(input.num_frames(), output_.num_frames());

  auto* gain_processors = GetOrCreateProcessors(input.source_id());
  // Accumulate the input buffers into the output buffer.
  for (size_t i = 0; i < num_channels_; ++i) {
    if (input[i].IsEnabled()) {
      (*gain_processors)[i].ApplyGain(gains[i], input[i], &output_[i],
                                      true /* accumulate_output */);
    } else {
      // Make sure the gain processor is initialized.
      (*gain_processors)[i].Reset(gains[i]);
    }
  }
  is_empty_ = false;
}

void GainMixer::AddInputChannel(const AudioBuffer::Channel& input,
                                SourceId source_id,
                                const std::vector<float>& gains) {
  DCHECK_EQ(gains.size(), num_channels_);
  DCHECK_EQ(input.size(), output_.num_frames());

  auto* gain_processors = GetOrCreateProcessors(source_id);
  // Accumulate the input buffers into the output buffer.
  for (size_t i = 0; i < num_channels_; ++i) {
    if (input.IsEnabled()) {
      (*gain_processors)[i].ApplyGain(gains[i], input, &output_[i],
                                      true /* accumulate_output */);
    } else {
      // Make sure the gain processor is initialized.
      (*gain_processors)[i].Reset(gains[i]);
    }
  }
  is_empty_ = false;
}

const AudioBuffer* GainMixer::GetOutput() const {
  if (is_empty_) {
    return nullptr;
  }
  return &output_;
}

void GainMixer::Reset() {
  if (!is_empty_) {
    // Delete the processors for sources which no longer exist.
    for (auto it = source_gain_processors_.begin();
         it != source_gain_processors_.end();
         /* no increment */) {
      if (it->second.processors_active) {
        it->second.processors_active = false;
        ++it;
      } else {
        source_gain_processors_.erase(it++);
      }
    }
    // Reset the output buffer.
    output_.Clear();
  }
  is_empty_ = true;
}

GainMixer::GainProcessors::GainProcessors(size_t num_channels)
    : processors_active(true), processors(num_channels) {}

std::vector<GainProcessor>* GainMixer::GetOrCreateProcessors(
    SourceId source_id) {
  // Attempt to find a |ScaleAndAccumulateProcessor| for the given |source_id|,
  // if none can be found add one. In either case mark that the processor has
  // been used so that it is not later deleted.
  if (source_gain_processors_.find(source_id) ==
      source_gain_processors_.end()) {
    source_gain_processors_.insert({source_id, GainProcessors(num_channels_)});
  }
  source_gain_processors_.at(source_id).processors_active = true;
  return &(source_gain_processors_.at(source_id).processors);
}

}  // namespace vraudio
