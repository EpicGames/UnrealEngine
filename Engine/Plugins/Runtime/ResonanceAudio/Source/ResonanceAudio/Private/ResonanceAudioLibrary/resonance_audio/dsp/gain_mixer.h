/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_GAIN_MIXER_H_
#define RESONANCE_AUDIO_DSP_GAIN_MIXER_H_

#include <memory>
#include <unordered_map>
#include <vector>

#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "dsp/gain_processor.h"

namespace vraudio {

class GainMixer {
 public:
  GainMixer(size_t num_channels, size_t frames_per_buffer);

  // Adds a separately scaled version of each channel of the input buffer to the
  // output buffer's corresponding channels.
  //
  // @param input Input buffer to be added.
  // @param gains Gains to be applied to the buffers channels. Must be equal in
  //     length to the number of channels in |input|.
  void AddInput(const AudioBuffer& input, const std::vector<float>& gains);

  // Adds a single input channel to each of the output buffer's channels, with
  // a separate gain applied to the input channel per output channel.
  //
  // @param input Input channel to be added.
  // @param source_id Identifier corresponding to the input.
  // @param gains Gains to be applied to the buffers channels. Must be equal in
  //     length to the number of channels in the output buffer.
  void AddInputChannel(const AudioBuffer::Channel& input, SourceId source_id,
                       const std::vector<float>& gains);

  // Returns a pointer to the accumulator.
  //
  // @return Pointer to the processed (mixed) output buffer, or nullptr if no
  //     input has been added to the accumulator.
  const AudioBuffer* GetOutput() const;

  // Resets the state of the accumulator.
  void Reset();

 private:
  // Comprises one |GainProcessor| per channel of a source and a boolean to
  // denote whether that source is active.
  struct GainProcessors {
    explicit GainProcessors(size_t num_channels);

    // Bool to signify if a given source is still passing data to a processor.
    bool processors_active;

    // Scale and accumulation processors, one per channel for each source.
    std::vector<GainProcessor> processors;
  };

  // Returns the |GainProcessor|s associated with a |source_id| (or creates
  // one if needed) and sets the corresponding |processors_active| flag to true.
  //
  // @param source_id Identifier for a given input.
  // @return The corresponding |ScalingAccumulators|.
  std::vector<GainProcessor>* GetOrCreateProcessors(SourceId source_id);

  // Number of channels.
  const size_t num_channels_;

  // Output buffer (accumulator).
  AudioBuffer output_;

  // Denotes whether the accumulator has processed any inputs or not.
  bool is_empty_;

  // Scale and accumulation processors, one per channel for each source.
  std::unordered_map<SourceId, GainProcessors> source_gain_processors_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_GAIN_MIXER_H_
