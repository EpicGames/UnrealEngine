/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/gain_processor.h"

#include <algorithm>
#include <cmath>

#include "base/constants_and_types.h"

#include "dsp/gain.h"

namespace vraudio {

// Delegate default constructor.
GainProcessor::GainProcessor() : current_gain_(0.0f), is_initialized_(false) {}

GainProcessor::GainProcessor(float initial_gain)
    : current_gain_(initial_gain), is_initialized_(true) {}

void GainProcessor::ApplyGain(float target_gain,
                              const AudioBuffer::Channel& input,
                              AudioBuffer::Channel* output,
                              bool accumulate_output) {

  DCHECK(output);

  if (!is_initialized_) {
    Reset(target_gain);
  }

  // Check buffer length.
  const size_t input_length = input.size();
  DCHECK_GT(input_length, 0U);
  DCHECK_EQ(input_length, output->size());

  // Index for where to stop interpolating.
  size_t ramp_length =
      static_cast<size_t>(std::abs(target_gain - current_gain_) *
                          static_cast<float>(kUnitRampLength));

  // Check if there is a new gain value.
  if (ramp_length > 0) {
    // Apply gain ramp to buffer.
    current_gain_ = LinearGainRamp(ramp_length, current_gain_, target_gain,
                                   input, output, accumulate_output);
  } else {
    // No ramping needed.
    current_gain_ = target_gain;
  }

  // Apply constant gain to the rest of the buffer.
  if (ramp_length < input_length) {
    if (IsGainNearZero(current_gain_)) {
      // Skip processing if the gain is zero.
      if (!accumulate_output) {
        // Directly fill the remaining output with zeros.
        std::fill(output->begin() + ramp_length, output->end(), 0.0f);
      }
      return;
    } else if (IsGainNearUnity(current_gain_) && !accumulate_output) {
      // Skip processing if the gain is unity.
      if (&input != output) {
        // Directly copy the remaining input samples into output.
        std::copy(input.begin() + ramp_length, input.end(),
                  output->begin() + ramp_length);
      }
      return;
    }
    ConstantGain(ramp_length, current_gain_, input, output, accumulate_output);
  }
}

float GainProcessor::GetGain() const { return current_gain_; }

void GainProcessor::Reset(float gain) {
  current_gain_ = gain;
  is_initialized_ = true;
}

}  // namespace vraudio
