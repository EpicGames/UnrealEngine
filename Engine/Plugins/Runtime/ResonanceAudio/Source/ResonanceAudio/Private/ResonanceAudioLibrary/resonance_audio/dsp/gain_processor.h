/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_GAIN_PROCESSOR_H_
#define RESONANCE_AUDIO_DSP_GAIN_PROCESSOR_H_

#include <cmath>
#include <cstddef>
#include <vector>

#include "base/audio_buffer.h"

namespace vraudio {

// Processor class which applies a gain to a vector of samples from an audio
// buffer. A short linear ramp is applied to the gain to reduce audible
// artifacts in the output.
class GainProcessor {
 public:
  // Default constructor keeps the gain state uninitialized. The first call to
  // |ApplyGain| sets the internal gain state.
  GainProcessor();

  // Constructs |GainProcessor| with some initial gain value.
  //
  // @param initial_gain Gain value used as starting point for first processing
  //   period's gain ramping.
  explicit GainProcessor(float initial_gain);

  // Applies gain supplied to the input samples.
  //
  // @param target_gain Target gain value.
  // @param input Samples to which gain will be applied.
  // @param output Samples to which gain has been applied.
  // @param accumulate_output True if the processed input should be mixed into
  //     the output. Otherwise, the output will be replaced by the processed
  //     input.
  void ApplyGain(float target_gain, const AudioBuffer::Channel& input,
                 AudioBuffer::Channel* output, bool accumulate_output);

  // Returns the |current_gain_| value.
  //
  // @return Current gain applied by the |GainProcessor|.
  float GetGain() const;

  // Resets the gain processor to a new gain factor.
  //
  // @param gain Gain value.
  void Reset(float gain);

 private:
  // Latest gain value to be applied to buffer values.
  float current_gain_;

  // Flag to indiciate if an initial gain has been assigned.
  bool is_initialized_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_GAIN_PROCESSOR_H_
