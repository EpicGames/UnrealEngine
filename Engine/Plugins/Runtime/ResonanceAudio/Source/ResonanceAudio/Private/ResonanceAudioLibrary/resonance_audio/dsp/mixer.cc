/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/mixer.h"

#include "base/logging.h"

namespace vraudio {

Mixer::Mixer(size_t target_num_channels, size_t frames_per_buffer)
    : output_(target_num_channels, frames_per_buffer), is_empty_(false) {
  Reset();
}

void Mixer::AddInput(const AudioBuffer& input) {
  DCHECK_EQ(input.num_frames(), output_.num_frames());

  // Accumulate the input buffers into the output buffer.
  const size_t num_channels =
      std::min(input.num_channels(), output_.num_channels());
  for (size_t n = 0; n < num_channels; ++n) {
    if (input[n].IsEnabled()) {
      output_[n] += input[n];
    }
  }
  is_empty_ = false;
}

const AudioBuffer* Mixer::GetOutput() const {
  if (is_empty_) {
    return nullptr;
  }
  return &output_;
}

void Mixer::Reset() {
  if (!is_empty_) {
    output_.Clear();
  }
  is_empty_ = true;
}

}  // namespace vraudio
