/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_MIXER_H_
#define RESONANCE_AUDIO_DSP_MIXER_H_

#include <memory>

#include "base/audio_buffer.h"

namespace vraudio {

// Accepts multiple input buffers and outputs a downmix to a single output
// buffer. All input buffers must have the same number of frames per buffer, the
// output will have the target number of channels regardless of the input number
// of channels.
class Mixer {
 public:
  // Constructor.
  //
  // @param target_num_channels Target number of channels in accumulator buffer.
  // @param frames_per_buffer Number of frames in accumulator buffer.
  Mixer(size_t target_num_channels, size_t frames_per_buffer);

  // Adds an input buffer to the mixer, updates the output buffer accordingly.
  //
  // @param input Input buffer to be added.
  void AddInput(const AudioBuffer& input);

  // Returns a pointer to the accumulator.
  //
  // @return Pointer to the processed (mixed) output buffer, or nullptr if no
  //     input has been added to the accumulator.
  const AudioBuffer* GetOutput() const;

  // Resets the state of the accumulator.
  void Reset();

 private:
  // Output buffer (accumulator).
  AudioBuffer output_;

  // Denotes whether the accumulator has processed any inputs or not.
  bool is_empty_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_MIXER_H_
