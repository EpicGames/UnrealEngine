/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/mono_pole_filter.h"

#include <algorithm>

#include "base/constants_and_types.h"


namespace vraudio {

MonoPoleFilter::MonoPoleFilter(float coefficient) : previous_output_(0.0f) {
  SetCoefficient(coefficient);
}

void MonoPoleFilter::SetCoefficient(float coefficient) {
  coefficient_ = std::max(std::min(coefficient, 1.0f), 0.0f);
}

bool MonoPoleFilter::Filter(const AudioBuffer::Channel& input,
                            AudioBuffer::Channel* output) {

  DCHECK(output);
  const size_t num_frames = input.size();
  DCHECK_EQ(num_frames, output->size());

  // Do not perform processing if the coefficient is zero to avoid wasteful
  // "all pass" cases.
  if (coefficient_ < kEpsilonFloat) {
    previous_output_ = input[num_frames - 1];
    return false;
  }

  // The difference equation implemented here is as follows:
  // y[n] = a * (y[n-1] - x[n]) + x[n]
  // where y[n] is the output and x[n] is the input vector.
  for (size_t frame = 0; frame < num_frames; ++frame) {
    (*output)[frame] =
        coefficient_ * (previous_output_ - input[frame]) + input[frame];
    previous_output_ = (*output)[frame];
  }
  return true;
}

}  // namespace vraudio
