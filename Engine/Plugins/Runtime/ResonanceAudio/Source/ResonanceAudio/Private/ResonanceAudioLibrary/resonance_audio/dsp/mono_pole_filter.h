/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_MONO_POLE_FILTER_H_
#define RESONANCE_AUDIO_DSP_MONO_POLE_FILTER_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Class representing a mono pole filter. This class also performs filtering.
class MonoPoleFilter {
 public:
  // Constructs a |MonoPoleFilter| given a single coefficient.
  //
  // @param coefficient A single coefficient between 0.0f and 1.0f.
  explicit MonoPoleFilter(float coefficient);

  // Filter method for use with AudioBuffer::Channel.
  //
  // @param input |AudioBuffer::Channel| of input to be processed.
  // @param output Pointer to output |AudioBuffer::Channel|.
  // @return Returns false if the filter has an allpass configuration. This
  //     helps to avoid copies whenever the output is expected to be identical
  //     to the input.
  bool Filter(const AudioBuffer::Channel& input, AudioBuffer::Channel* output);

  // Sets the filter's coefficent.
  //
  // @param coefficient A mono pole filter coefficient.
  void SetCoefficient(float coefficient);

 private:
  // The previous frame computed by the filter.
  float previous_output_;

  // Represents and maintains the state of the filter in terms of its
  // transfer function coefficient.
  float coefficient_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_MONO_POLE_FILTER_H_
