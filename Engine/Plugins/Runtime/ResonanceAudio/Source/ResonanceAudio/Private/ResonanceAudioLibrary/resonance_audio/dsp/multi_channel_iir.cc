/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/multi_channel_iir.h"

#include <algorithm>
#include <limits>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_macros.h"


namespace vraudio {

std::unique_ptr<MultiChannelIir> MultiChannelIir::Create(
    size_t num_channels, size_t frames_per_buffer,
    const std::vector<std::vector<float>>& numerators,
    const std::vector<std::vector<float>>& denominators) {
  CHECK_EQ(denominators.size(), numerators.size());
  CHECK_EQ(denominators.size(), num_channels);
  CHECK_EQ(num_channels % SIMD_LENGTH, 0U);
  CHECK_GT(num_channels, 0U);
  for (size_t channel = 0; channel < num_channels; ++channel) {
    CHECK_GT(denominators[channel].size(), 0U);
    CHECK_GT(std::abs(denominators[channel][0]),
             std::numeric_limits<float>::epsilon());
    CHECK_EQ(denominators[channel].size(), numerators[channel].size());
  }

  std::unique_ptr<MultiChannelIir> multi_channel_iir(new MultiChannelIir(
      num_channels, frames_per_buffer, numerators[0].size()));
  CHECK(multi_channel_iir);

  for (size_t channel = 0; channel < num_channels; ++channel) {
    size_t interleaved_index = channel;
    for (size_t i = 0; i < numerators[channel].size(); ++i) {
      // Make the denominator polynomials monic, (divide all coefficients by the
      // the first denominator coefficients). Furthermore negate all
      // coefficients beyond the first (see equation in Process method).
      // Copy the coefficients into the |numerator_| and |denominator_| buffers.
      multi_channel_iir->numerator_[0][interleaved_index] =
          numerators[channel][i] / denominators[channel][0];
      multi_channel_iir->denominator_[0][interleaved_index] =
          denominators[channel][i] /
          ((i > 0) ? -denominators[channel][0] : denominators[channel][0]);
      interleaved_index += num_channels;
    }
  }

  multi_channel_iir->delay_line_.Clear();
  return multi_channel_iir;
}

void MultiChannelIir::Process(AudioBuffer::Channel* interleaved_buffer) {

  DCHECK(interleaved_buffer);
  DCHECK_EQ(interleaved_buffer->size(), num_channels_ * frames_per_buffer_);
  const SimdVector* simd_numerator =
      reinterpret_cast<SimdVector*>(&(numerator_[0][0]));
  const SimdVector* simd_denominator =
      reinterpret_cast<SimdVector*>(&(denominator_[0][0]));
  SimdVector* simd_delay_line =
      reinterpret_cast<SimdVector*>(&(delay_line_[0][0]));
  SimdVector* simd_buffer =
      reinterpret_cast<SimdVector*>(&((*interleaved_buffer)[0]));

  const size_t num_channel_chunks = num_channels_ / SIMD_LENGTH;
  const size_t num_buffer_chunks = interleaved_buffer->size() / SIMD_LENGTH;
  const size_t delay_length_in_chunks = std::max(static_cast<size_t>(1), num_channel_chunks * num_coefficients_);

  for (size_t current_frame = 0, individual_frame = 0;
       current_frame < num_buffer_chunks;
       current_frame += num_channel_chunks, individual_frame += num_channels_) {
    DCHECK_LT(individual_frame, interleaved_buffer->size());
    // Copy the current sample into the delay line at the very start.
    // {x[n], w[n-1], w[n-2], . . .} where each x, w represents all channels.
    std::copy_n(interleaved_buffer->begin() + individual_frame, num_channels_,
                delay_line_[0].begin() + (delay_line_front_ * SIMD_LENGTH));
    // Using A Direct Form II implementation difference equation:
    // Source: Digital Signal Processing Principles Algorithms & Applications
    //    Fourth Edition. John G. Prolakis and Dimitris G. Manolakis - Chap 9
    //  w[n] =         x[n] - (a1/a0)*w[n-1] - (a2/a0)*w[n-2] - . . .
    //  y(n) = (b0/a0)*w[n] + (b1/a0)*w[n-1] + (b2/a0)*w[n-2] + . . .
    // where x[n] is input, w[n] is storage and y[n] is output.
    // The division by a0 has been performed in the constructor along with the
    // negation of the denominator coefficients beyond the first. Note also
    // that each term here refers to a set of channels.
    for (size_t channel_chunk = 0; channel_chunk < num_channel_chunks;
         ++channel_chunk) {
      // First zero out the relevant section of the buffer before accumulation.
      // Zero constant used for loading zeros into a neon simd array, as the
      // |vld1q_dup_f32| neon intrinsic requires an lvalue parameter.
      const float kZerof = 0.0f;
      simd_buffer[current_frame + channel_chunk] = SIMD_LOAD_ONE_FLOAT(kZerof);
      for (size_t coeff_offset = num_channel_chunks;
           coeff_offset < delay_length_in_chunks;
           coeff_offset += num_channel_chunks) {
        // Denominator part.
        const size_t multiplication_index = channel_chunk + coeff_offset;
        const size_t delay_multiplication_index =
            (multiplication_index + delay_line_front_) % delay_length_in_chunks;
        const size_t delay_write_index = channel_chunk + delay_line_front_;
        simd_delay_line[delay_write_index] =
            SIMD_MULTIPLY_ADD(simd_denominator[multiplication_index],
                              simd_delay_line[delay_multiplication_index],
                              simd_delay_line[delay_write_index]);
      }
      for (size_t coeff_offset = 0; coeff_offset < delay_length_in_chunks;
           coeff_offset += num_channel_chunks) {
        // Numerator part.
        const size_t multiplication_index = channel_chunk + coeff_offset;
        const size_t write_index = current_frame + channel_chunk;
        const size_t delay_multiplication_index =
            (multiplication_index + delay_line_front_) % delay_length_in_chunks;
        simd_buffer[write_index] =
            SIMD_MULTIPLY_ADD(simd_numerator[multiplication_index],
                              simd_delay_line[delay_multiplication_index],
                              simd_buffer[write_index]);
      }
    }
    // Update the index to the wrapped around 'front' of the delay line.
    delay_line_front_ = ((static_cast<int>(delay_line_front_) -
                          num_channel_chunks + delay_length_in_chunks) %
                         delay_length_in_chunks);
  }
}

MultiChannelIir::MultiChannelIir(size_t num_channels, size_t frames_per_buffer,
                                 size_t num_coefficients)
    : num_channels_(num_channels),
#if !DISABLE_CHECKS
      frames_per_buffer_(frames_per_buffer),
#endif
      num_coefficients_(num_coefficients),
      delay_line_front_(0),
      numerator_(kNumMonoChannels, num_coefficients_ * num_channels_),
      denominator_(kNumMonoChannels, num_coefficients_ * num_channels_),
      delay_line_(kNumMonoChannels, num_coefficients_ * num_channels_) {}

}  // namespace vraudio
