/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_MULTI_CHANNEL_IIR_H_
#define RESONANCE_AUDIO_DSP_MULTI_CHANNEL_IIR_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"

namespace vraudio {

// Class that performs IIR filtering on interleaved data. This class can be used
// to implement any order of IIR filter on multichannel data where the number of
// channels is a multiple of the SIMD vector length.
class MultiChannelIir {
 public:
  // Returns a |MultiChannelIir| given valid parameters.
  //
  // @param num_channels Number of channels in each input buffer. The number of
  //     channels must be divisible by |SIMD_LENGTH|.
  // @param frames_per_buffer Number of frames in each input buffer.
  // @param numerators Numerator coefficients, one set per channel.
  // @param denominators Denominator coefficients, should be equal in length to
  //     the |numerator| vector, one set per channel.
  // @return A |MultiChannelIir| instance.
  static std::unique_ptr<MultiChannelIir> Create(
      size_t num_channels, size_t frames_per_buffer,
      const std::vector<std::vector<float>>& numerators,
      const std::vector<std::vector<float>>& denominators);

  // Processes an interleaved buffer of input data with the given IIR filter.
  //
  // @param interleaved_buffer A single channel of data containing input in
  //     interleaved format, this will contain output data in interleaved format
  //     on return.
  void Process(AudioBuffer::Channel* interleaved_buffer);

 private:
  // Constructs a |MultiChannelIir|.
  //
  // @param num_channels Number of channels in each input buffer. The number of
  //     channels must be divisible by |SIMD_LENGTH|.
  // @param frames_per_buffer Number of frames in each input buffer.
  // @param num_coefficients Number of coefficients in the numerator, which
  //     equals the number in the denominator.
  MultiChannelIir(size_t num_channels, size_t frames_per_buffer,
                  size_t num_coefficients);

  // Number of channels in each input buffer.
  const size_t num_channels_;

#if !DISABLE_CHECKS
  // Number of frames in each input buffer.
  const size_t frames_per_buffer_;
#endif

  // Number of coefficients in the numerator and denominator polynomials.
  const size_t num_coefficients_;

  // Current front of the delay line which is circularly indexed.
  size_t delay_line_front_;

  // Stores numerator coefficients in repeated fashion.
  AudioBuffer numerator_;

  // Stores denominator coefficients in repeated fashion.
  AudioBuffer denominator_;

  // Holds previous data computed from the numerator section.
  AudioBuffer delay_line_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_MULTI_CHANNEL_IIR_H_
