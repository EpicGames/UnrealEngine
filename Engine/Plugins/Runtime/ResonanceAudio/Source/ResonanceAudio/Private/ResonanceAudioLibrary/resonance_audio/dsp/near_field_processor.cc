/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/near_field_processor.h"

#include "base/constants_and_types.h"
#include "dsp/filter_coefficient_generators.h"
#include "dsp/gain.h"

namespace vraudio {

namespace {

// Cross-over frequency of the band-splitting filter.
const float kCrossOverFrequencyHz = 1000.0f;

// +6dB bass boost factor converted to linear scale.
const float kBassBoost = 2.0f;

// Average group delay of the HRTF filters in seconds. Please see
// [internal ref]
const float kMeanHrtfGroupDelaySeconds = 0.00066667f;

// Average group delay of the shelf-filter in samples.
const size_t kMeanShelfFilterGroupDelaySamples = 1;

}  // namespace

NearFieldProcessor::NearFieldProcessor(int sample_rate,
                                       size_t frames_per_buffer)
    : frames_per_buffer_(frames_per_buffer),
      delay_compensation_(static_cast<size_t>(kMeanHrtfGroupDelaySeconds *
                                              static_cast<float>(sample_rate)) -
                          kMeanShelfFilterGroupDelaySamples),
      lo_pass_filter_(BiquadCoefficients(), frames_per_buffer_),
      hi_pass_filter_(BiquadCoefficients(), frames_per_buffer_),
      low_passed_buffer_(kNumMonoChannels, frames_per_buffer_),
      delay_filter_(delay_compensation_, frames_per_buffer_) {
  DCHECK_GT(sample_rate, 0);
  DCHECK_GT(frames_per_buffer, 0);
  DCHECK_LT(kCrossOverFrequencyHz, 0.5f * static_cast<float>(sample_rate));

  // Generate biquad coefficients and construct low- and high-pass filter
  // states.
  BiquadCoefficients lo_pass_coefficients;
  BiquadCoefficients hi_pass_coefficients;
  ComputeDualBandBiquadCoefficients(sample_rate, kCrossOverFrequencyHz,
                                    &lo_pass_coefficients,
                                    &hi_pass_coefficients);

  // Create two biquad filters initialized with the above filter coefficients.
  lo_pass_filter_.SetCoefficients(lo_pass_coefficients);
  hi_pass_filter_.SetCoefficients(hi_pass_coefficients);
}

void NearFieldProcessor::Process(const AudioBuffer::Channel& input,
                                 AudioBuffer::Channel* output,
                                 bool enable_hrtf) {

  DCHECK(output);
  DCHECK_EQ(input.size(), frames_per_buffer_);
  DCHECK_EQ(output->size(), frames_per_buffer_);

  // Low-pass filter the input and put it in the temporary low-passed buffer.
  auto* low_passed_channel = &low_passed_buffer_[0];
  lo_pass_filter_.Filter(input, low_passed_channel);

  // High-pass filter the input and put it in the output channel (unmodified).
  hi_pass_filter_.Filter(input, output);
  // Iterate through all the samples in the |low_passed_buffer_| and apply
  // the bass boost. Then, combine with the high-passed part in order to form
  // the shelf-filtered output. Note: phase flip of the low-passed signal is
  // required  to form the correct filtered output.
  ConstantGain(/*offset_index=*/0, -kBassBoost, *low_passed_channel, output,
               /*accumulate_output=*/true);

  if (enable_hrtf) {
    // Delay the output to compensate for the average HRTF group delay.
    delay_filter_.InsertData(*output);
    delay_filter_.GetDelayedData(delay_compensation_, output);
  }
}

}  // namespace vraudio
