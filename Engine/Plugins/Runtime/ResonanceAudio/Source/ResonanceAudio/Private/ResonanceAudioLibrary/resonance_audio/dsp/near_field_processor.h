/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_NEAR_FIELD_PROCESSOR_H_
#define RESONANCE_AUDIO_DSP_NEAR_FIELD_PROCESSOR_H_

#include "base/audio_buffer.h"
#include "dsp/biquad_filter.h"
#include "dsp/delay_filter.h"

namespace vraudio {

// Class which applies an approximate near field effect to a source mono input.
// The effect consists of a +6dB bass boost (shelf-filter) as well as phase
// correction (HRTF group delay compensation) when the signal is to be combined
// with the binaural output.
//
// For more information: [internal ref]
class NearFieldProcessor {
 public:
  // Constructor of the |NearFieldProcessor|, uses the following parameters:
  //
  // @param sample_rate Sampling rate in [Hz].
  // @param frames_per_buffer Number of frames per buffer in the input/output
  //     signal.
  NearFieldProcessor(int sample_rate, size_t frames_per_buffer);

  // Returns currently used delay compensation in samples.
  size_t GetDelayCompensation() const { return delay_compensation_; }

  // Applies approximate near field effect to the source mono input signal.
  //
  // @param input Mono input channel.
  // @param output Pointer to mono output channel.
  // @param enable_hrtf Whether to enable delay compensation for HRTF filtering.
  void Process(const AudioBuffer::Channel& input, AudioBuffer::Channel* output,
               bool enable_hrtf);

 private:
  // Number of frames per buffer.
  const size_t frames_per_buffer_;

  // Delay compensation computed as average group delay of the HRTF filter
  // minus average group delay of the shelf-filter. Should be disabled when
  // using with stereo-panned sound sources.
  const size_t delay_compensation_;

  // Biquad filters that apply frequency splitting of the input mono signal.
  BiquadFilter lo_pass_filter_;
  BiquadFilter hi_pass_filter_;

  // Buffer for the low-passed signal. We do not modify the high-passed signal
  // so we can write it directly to the output channel.
  AudioBuffer low_passed_buffer_;

  // Delay filter used to delay the incoming input mono buffer.
  DelayFilter delay_filter_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_NEAR_FIELD_PROCESSOR_H_
