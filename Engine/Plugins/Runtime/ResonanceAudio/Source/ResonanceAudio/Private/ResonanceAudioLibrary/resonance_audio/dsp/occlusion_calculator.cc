/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/occlusion_calculator.h"

#include <cmath>

#include "base/logging.h"
#include "base/misc_math.h"

namespace vraudio {

float CalculateDirectivity(float alpha, float order,
                           const SphericalAngle& spherical_angle) {
  // Clamp alpha weighting.
  const float alpha_clamped = std::min(std::max(alpha, 0.0f), 1.0f);

  // Check for zero-valued alpha (omnidirectional).
  if (alpha_clamped < std::numeric_limits<float>::epsilon()) {
    return 1.0f;
  } else {
    const float gain = (1.0f - alpha_clamped) +
                       alpha_clamped * (std::cos(spherical_angle.azimuth()) *
                                        std::cos(spherical_angle.elevation()));

    return std::pow(std::abs(gain), std::max(order, 1.0f));
  }
}

float CalculateOcclusionFilterCoefficient(float directivity,
                                          float occlusion_intensity) {
  DCHECK_GE(occlusion_intensity, 0.0f);

  const float occlusion_factor =
      1.0f / IntegerPow(occlusion_intensity + 1.0f, 4);
  return std::max(0.0f, 1.0f - directivity * occlusion_factor);
}

}  // namespace vraudio
