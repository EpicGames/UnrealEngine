/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_OCCLUSION_CALCULATOR_H_
#define RESONANCE_AUDIO_DSP_OCCLUSION_CALCULATOR_H_

#include "base/spherical_angle.h"

namespace vraudio {

// Calculates directivity gain value for supplied source and listener
// parameters.
//
// @param alpha Balance between dipole pattern and omnidirectional pattern for
//     source emission. By varying this value, differing directivity patterns
//     can be formed. Value in range [0, 1]. 2D visualization for several
//     values: http://goo.gl/GhKvoc.
// @param order Order of directivity function. Higher values will result in
//     increased directivity. Value in range [1, +inf]. 2D visualization for
//     several orders:
//     http://goo.gl/sNrm1a.
// @param spherical_angle Spherical angle of the listener relative to the
//     audio source which is being shaped.
// @return Gain value in range [0, 1].
float CalculateDirectivity(float alpha, float order,
                           const SphericalAngle& spherical_angle);

// This function calculates a |MonoPoleFilter| coefficient based upon the
// directivity and occlusion values. The coefficient calculation was designed
// via empirical methods.
//
// @param directivity Gain value calculated based upon the directivity.
// @param occlusion_intensity Gain value calculated based upon the degree of
// occlusion.
// @return Filter coefficient for a mono pole low pass filter.
float CalculateOcclusionFilterCoefficient(float directivity,
                                          float occlusion_intensity);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_OCCLUSION_CALCULATOR_H_
