/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "dsp/partitioned_fft_filter.h"

#include <algorithm>

#include "pffft.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"
#include "base/simd_utils.h"

#include "dsp/utils.h"

namespace vraudio {

static size_t clamp_to_non_zero(const size_t& val) { return (val < 1)? 1 : val; }

PartitionedFftFilter::PartitionedFftFilter(size_t filter_size,
                                           size_t frames_per_buffer,
                                           FftManager* fft_manager)
    : PartitionedFftFilter(filter_size, frames_per_buffer, filter_size,
                           fft_manager) {}

PartitionedFftFilter::PartitionedFftFilter(size_t filter_size,
                                           size_t frames_per_buffer,
                                           size_t max_filter_size,
                                           FftManager* fft_manager)
    : fft_manager_(fft_manager),
      fft_size_(fft_manager_->GetFftSize()),
      chunk_size_(fft_size_ / 2),
      frames_per_buffer_(frames_per_buffer),
      max_filter_size_(
          CeilToMultipleOfFramesPerBuffer(max_filter_size, frames_per_buffer_)),
      max_num_partitions_(max_filter_size_ / frames_per_buffer_),
      filter_size_(
          CeilToMultipleOfFramesPerBuffer(filter_size, frames_per_buffer_)),
      num_partitions_(filter_size_ / frames_per_buffer_),
      kernel_freq_domain_buffer_(max_num_partitions_, fft_size_),
      buffer_selector_(0),
      curr_front_buffer_(0),
      freq_domain_buffer_(max_num_partitions_, fft_size_),
      filtered_time_domain_buffers_(kNumStereoChannels, fft_size_),
      freq_domain_accumulator_(kNumMonoChannels, fft_size_),
      temp_zeropad_buffer_(kNumMonoChannels, chunk_size_),
      temp_kernel_chunk_buffer_(kNumMonoChannels, frames_per_buffer_) {
  // Ensure that |frames_per_buffer_| is less than or equal to the final
  // partition |chunk_size_|.
  CHECK(fft_manager_);
  CHECK_LE(frames_per_buffer_, chunk_size_);
  CHECK_GE(filter_size_, filter_size);
  CHECK_GE(max_filter_size_, max_filter_size);
  // Ensure that |filter_size_| does not exceed |max_filter_size_|.
  CHECK_LE(filter_size, max_filter_size_);
  // Make sure all partitions have the same size.
  CHECK_EQ(num_partitions_ * frames_per_buffer_, filter_size_);
  CHECK_EQ(max_num_partitions_ * frames_per_buffer_, max_filter_size_);

  Clear();
}

void PartitionedFftFilter::Clear() {
  // Reset valid part of the filter |FreqDomainBuffer|s to zero.
  for (size_t i = 0; i < num_partitions_; ++i) {
    kernel_freq_domain_buffer_[i].Clear();
    freq_domain_buffer_[i].Clear();
  }
  // Reset filter state to zero.
  filtered_time_domain_buffers_.Clear();
}

void PartitionedFftFilter::ResetFreqDomainBuffers(size_t new_filter_size) {

  // Update the filter size.
  DCHECK_GT(new_filter_size, 0U);
  filter_size_ =
      CeilToMultipleOfFramesPerBuffer(new_filter_size, frames_per_buffer_);
  DCHECK_LE(filter_size_, max_filter_size_);

  const size_t old_num_partitions = num_partitions_;
  num_partitions_ = std::max(static_cast<size_t>(1), filter_size_ / frames_per_buffer_);
  const size_t min_num_partitions =
      std::min(old_num_partitions, num_partitions_);

  if (curr_front_buffer_ > 0) {

    FreqDomainBuffer temp_freq_domain_buffer(min_num_partitions, fft_size_);
    // Copy in |min_num_partitions| to |temp_freq_domain_buffer|, starting with
    // the partition at |curr_front_buffer_| to be moved back to the beginning
    // of |freq_domain_buffer| .
    for (size_t i = 0; i < min_num_partitions; ++i) {
      temp_freq_domain_buffer[i] =
          freq_domain_buffer_[(curr_front_buffer_ + i) % old_num_partitions];
    }
    // Replace the partitions.
    for (size_t i = 0; i < min_num_partitions; ++i) {
      freq_domain_buffer_[i] = temp_freq_domain_buffer[i];
    }
    curr_front_buffer_ = 0;
  }
  // Clear out the remaining partitions in case the filter size grew.
  for (size_t i = old_num_partitions; i < num_partitions_; ++i) {
    freq_domain_buffer_[i].Clear();
  }
}

void PartitionedFftFilter::ReplacePartition(
    size_t partition_index, const AudioBuffer::Channel& kernel_chunk) {
  DCHECK_GE(partition_index, 0U);
  DCHECK_LT(partition_index, num_partitions_);
  DCHECK_EQ(kernel_chunk.size(), frames_per_buffer_);

  fft_manager_->FreqFromTimeDomain(
      kernel_chunk, &kernel_freq_domain_buffer_[partition_index]);
}

void PartitionedFftFilter::SetFilterLength(size_t new_filter_size) {
  DCHECK_GT(new_filter_size, 0U);
  new_filter_size =
      CeilToMultipleOfFramesPerBuffer(new_filter_size, frames_per_buffer_);
  DCHECK_LE(new_filter_size, max_filter_size_);

  const size_t new_num_partitions = new_filter_size / frames_per_buffer_;
  DCHECK_LE(new_num_partitions, max_num_partitions_);

  // Clear out the remaining partitions in case the filter size grew.
  for (size_t i = num_partitions_; i < new_num_partitions; ++i) {
    kernel_freq_domain_buffer_[i].Clear();
  }
  // Call |ResetFreqDomainBuffers| to make sure that the input buffers are also
  // correctly resized.
  ResetFreqDomainBuffers(new_filter_size);
}

void PartitionedFftFilter::SetTimeDomainKernel(
    const AudioBuffer::Channel& kernel) {


  // Precomputes a set of floor(|filter_size_|/(|fft_size|/2)) frequency domain
  // kernels, one for each partition of the |kernel|. This allows to reduce
  // computational complexity if a fixed set of filter kernels is needed. The
  // size of the |kernel| can be arbitrarily long but must not be less than
  // |fft_size_|/2. Filter lengths which are multiples of |fft_size_|/2 are most
  // efficient.

  // Calculate the new number of partitions as filter length may have changed.
  // This number is set to 1 if the filter length is smaller than
  // |frames_per_buffer_|.
  const size_t new_num_partitions =
      CeilToMultipleOfFramesPerBuffer(kernel.size(), frames_per_buffer_) /
      frames_per_buffer_;

  auto& padded_channel = temp_kernel_chunk_buffer_[0];
  // Break up time domain filter into chunks and FFT each of these separately.
  for (size_t partition = 0; partition < new_num_partitions; ++partition) {
    DCHECK_LE(partition * frames_per_buffer_, kernel.size());
    const float* chunk_begin_itr =
        kernel.begin() + partition * frames_per_buffer_;
    const size_t num_frames_to_copy =
        std::min<size_t>(frames_per_buffer_, kernel.end() - chunk_begin_itr);

    std::copy_n(chunk_begin_itr, num_frames_to_copy, padded_channel.begin());
    // This fill only occurs on the very last partition.
    std::fill(padded_channel.begin() + num_frames_to_copy, padded_channel.end(),
              0.0f);
    fft_manager_->FreqFromTimeDomain(padded_channel,
                                     &kernel_freq_domain_buffer_[partition]);
  }

  if (new_num_partitions != num_partitions_) {
    const size_t new_filter_size = new_num_partitions * frames_per_buffer_;
    ResetFreqDomainBuffers(new_filter_size);
  }
}

void PartitionedFftFilter::SetFreqDomainKernel(const FreqDomainBuffer& kernel) {
  DCHECK_LE(kernel.num_channels(), max_num_partitions_);
  DCHECK_EQ(kernel.num_frames(), fft_size_);

  const size_t new_num_partitions = kernel.num_channels();
  for (size_t i = 0; i < new_num_partitions; ++i) {
    kernel_freq_domain_buffer_[i] = kernel[i];
  }
  if (new_num_partitions != num_partitions_) {
    const size_t new_filter_size = new_num_partitions * frames_per_buffer_;
    ResetFreqDomainBuffers(new_filter_size);
  }
}

void PartitionedFftFilter::Filter(const FreqDomainBuffer::Channel& input) {


  DCHECK_EQ(input.size(), fft_size_);
  std::copy_n(input.begin(), fft_size_,
              freq_domain_buffer_[curr_front_buffer_].begin());
  buffer_selector_ = !buffer_selector_;
  freq_domain_accumulator_.Clear();
  auto* accumulator_channel = &freq_domain_accumulator_[0];

  for (size_t i = 0; i < num_partitions_; ++i) {
    // Complex vector product in frequency domain with filter kernel.
    const size_t modulo_index = (curr_front_buffer_ + i) % num_partitions_;

    // Perform inverse scaling along with accumulation of last fft buffer.
    fft_manager_->FreqDomainConvolution(freq_domain_buffer_[modulo_index],
                                        kernel_freq_domain_buffer_[i],
                                        accumulator_channel);
  }
  // Our modulo based index.
  curr_front_buffer_ =
      (curr_front_buffer_ + num_partitions_ - 1) % clamp_to_non_zero(num_partitions_);
  // Perform inverse FFT transform of |freq_domain_buffer_| and store the
  // result back in |filtered_time_domain_buffers_|.
  fft_manager_->TimeFromFreqDomain(
      *accumulator_channel, &filtered_time_domain_buffers_[buffer_selector_]);
}

void PartitionedFftFilter::GetFilteredSignal(AudioBuffer::Channel* output) {

  DCHECK(output);
  DCHECK_EQ(output->size(), frames_per_buffer_);

  const size_t curr_buffer = buffer_selector_;
  const size_t prev_buffer = !buffer_selector_;

  // Overlap add.
  if (frames_per_buffer_ == chunk_size_) {
    AddPointwise(chunk_size_, &(filtered_time_domain_buffers_[curr_buffer][0]),
                 &filtered_time_domain_buffers_[prev_buffer][chunk_size_],
                 &((*output)[0]));
  } else {
    // If we have a non power of two |frames_per_buffer| we will have to perform
    // the overlap add and then a copy, as buffer lengths are not a multiple of
    // |chunk_size_|. NOTE: Indexing into |filtered_time_domain_buffers_| for a
    // non power of two |frames_per_buffer_| means that the |input_b| parameter
    // of |AddPointwiseOutOfPlace| may not be aligned in memory and thus we will
    // not be able to use SIMD for the overlap add operation.
    const auto& first_channel = filtered_time_domain_buffers_[curr_buffer];
    const auto& second_channel = filtered_time_domain_buffers_[prev_buffer];
    auto& output_channel = temp_zeropad_buffer_[0];
    for (size_t i = 0; i < frames_per_buffer_; ++i) {
      output_channel[i] =
          first_channel[i] + second_channel[i + frames_per_buffer_];
    }
    std::copy_n(temp_zeropad_buffer_[0].begin(), frames_per_buffer_,
                output->begin());
  }
}

}  // namespace vraudio
