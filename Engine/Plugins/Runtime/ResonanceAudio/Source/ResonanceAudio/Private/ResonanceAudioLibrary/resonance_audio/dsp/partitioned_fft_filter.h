/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_PARTITIONED_FFT_FILTER_H_
#define RESONANCE_AUDIO_DSP_PARTITIONED_FFT_FILTER_H_

#include <vector>

#include "base/audio_buffer.h"
#include "base/misc_math.h"
#include "dsp/fft_manager.h"

struct PFFFT_Setup;

namespace vraudio {

// Class performing a FFT-based overlap and add FIR convolution.
// Given an FFT size N and a filter size M > N/2; the filter is broken in to
// floor(M/(N/2)) partitions in the time domain. This results in a set of
// frequency domain "filters" of length (N/2)+1.
class PartitionedFftFilter {
 public:
  // Typedef declares the data type for storing frequency domain buffers. Each
  // channel stores the kernel for a partition.
  typedef AudioBuffer FreqDomainBuffer;

  // Constructor preallocates memory based on the |filter_size|. This can be
  // used for simplicity if the filter size will be constant after creation.
  //
  // @param filter_size Length of the time domain filter in samples. This will
  //     be increased such that it becomes a multiple of |chunk_size_|.
  // @param frames_per_buffer Number of points in each time domain input buffer.
  // @param fft_manager Pointer to a manager to perform FFT transformations.
  PartitionedFftFilter(size_t filter_size, size_t frames_per_buffer,
                       FftManager* fft_manager);

  // Constructor preallocates memory based on the |max_filter_size|. The
  // |fft_size_| will be twice |frames_per_buffer| if this is a power of two and
  // twice the next larger power of two if it is not.
  //
  // @param filter_size Length of the time domain filter in samples. This will
  //     be increased such that it becomes a multiple of |chunk_size_|.
  // @param frames_per_buffer Number of points in each time domain input buffer.
  // @param max_filter_size Maximum length that |filter_size| can get.
  // @param fft_manager Pointer to a manager for all fft related functionality.
  PartitionedFftFilter(size_t filter_size, size_t frames_per_buffer,
                       size_t max_filter_size, FftManager* fft_manager);

  // Initializes the FIR filter from a time domain kernel.
  //
  // @parem kernel Time domain filter to be used for processing.
  void SetTimeDomainKernel(const AudioBuffer::Channel& kernel);

  // Initializes the FIR filter from a precomputed frequency domain kernel.
  //
  // @param kernel Frequency domain filter to be used for processing.
  void SetFreqDomainKernel(const FreqDomainBuffer& kernel);

  // Replaces a partition indicated by the |partition_index| with
  // |kernel_chunk|'s frequency domain equivalent.
  //
  // @param partition_index Location (partition) of the time domain filter we
  //     wish to replace.
  // @param kernel_chunk |fft_size_|/2 length chunk of a filter used to
  //     replace the |partition_index|th partition.
  void ReplacePartition(size_t partition_index,
                        const AudioBuffer::Channel& kernel_chunk);

  // Alters the filter length by adding or removing partitions in the frequency
  // domain. If |new_filter_size| is not a multiple of |chunk_size_| (i.e.
  // frames per buffer), then the time domain filter kernel will be zeropadded
  // to a multiple of |chunk_size_|.
  //
  // @param new_filter_size New length of the time domain filter kernel.
  void SetFilterLength(size_t new_filter_size);

  // Processes a block of frequency domain samples. The size of the input
  // block must be |fft_size_|.
  //
  // @param Frequency domain input buffer.
  void Filter(const FreqDomainBuffer::Channel& input);

  // Returns block of filtered signal output of size |fft_size_|/2.
  //
  // @param output Time domain block filtered with the given kernel.
  void GetFilteredSignal(AudioBuffer::Channel* output);

  // Resets the filter state.
  void Clear();

 private:
  friend class PartitionedFftFilterFrequencyBufferTest;

  // Adjusts the |num_partitions_| and the size of |freq_domain_buffers_| for
  // use with a new time domain filter kernel greater in length than the
  // previous kernel.
  //
  // @param new_kernel_size Length of the time domain filter kernel.
  void ResetFreqDomainBuffers(size_t new_kernel_size);

  // Manager for all FFT related functionality (not owned).
  FftManager* const fft_manager_;

  // Number of points in the |fft_manager_|s FFT.
  const size_t fft_size_;

  // Size of each partition of the filter in time domain.
  const size_t chunk_size_;

  // Number of frames in each buffer of input data.
  const size_t frames_per_buffer_;

  // Maximum filter size in samples.
  const size_t max_filter_size_;

  // Maximum partition count.
  const size_t max_num_partitions_;

  // Filter size in samples.
  size_t filter_size_;

  // Partition Count.
  size_t num_partitions_;

  // Kernel buffer in frequency domain.
  FreqDomainBuffer kernel_freq_domain_buffer_;

  // Buffer selector to switch between two filtered signal buffers.
  size_t buffer_selector_;

  // The freq_domain_buffer we will write new incoming audio into.
  size_t curr_front_buffer_;

  // Frequency domain buffer used to perform filtering.
  FreqDomainBuffer freq_domain_buffer_;

  // Two buffers that are consecutively filled with filtered signal output.
  AudioBuffer filtered_time_domain_buffers_;

  // Accumulator for the outputs from each convolution partition
  FreqDomainBuffer freq_domain_accumulator_;

  // Temporary time domain buffer to store output when zero padding has been
  // applied due to non power of two input buffer lengths.
  AudioBuffer temp_zeropad_buffer_;

  // Temporary time domain buffer to hold time domain kernel chunks during
  // conversion of a kernel from time to frequency domain.
  AudioBuffer temp_kernel_chunk_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_PARTITIONED_FFT_FILTER_H_
