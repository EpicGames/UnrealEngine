/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_REFLECTION_H_
#define RESONANCE_AUDIO_DSP_REFLECTION_H_

namespace vraudio {

// Describes a room reflection that contains information on its time of arrival
// and magnitude.
struct Reflection {
  // Time of arrival of the reflection in seconds.
  float delay_time_seconds = 0.0f;

  // Magnitude of the reflection.
  float magnitude = 0.0f;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_REFLECTION_H_
