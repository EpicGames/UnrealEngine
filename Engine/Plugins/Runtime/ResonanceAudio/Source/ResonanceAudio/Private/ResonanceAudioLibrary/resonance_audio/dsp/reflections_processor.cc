/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/reflections_processor.h"

#include <algorithm>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "dsp/filter_coefficient_generators.h"
#include "dsp/gain.h"
#include "dsp/shoe_box_room.h"

namespace vraudio {

namespace {

// Maximum allowed delay time for a reflection. Above 2s, the effective output
// level of a reflection will fall below -60dB and thus perceived dynamic
// changes should be negligible.
const size_t kMaxDelayTimeSeconds = 2;

// Returns the maximum delay time in the given set of reflections.
float FindMaxReflectionDelayTime(const std::vector<Reflection>& reflections) {
  float max_delay_time = 0.0f;
  for (const auto& reflection : reflections) {
    max_delay_time = std::max(max_delay_time, reflection.delay_time_seconds);
  }
  return max_delay_time;
}

}  // namespace

ReflectionsProcessor::ReflectionsProcessor(int sample_rate,
                                           size_t frames_per_buffer)
    : sample_rate_(sample_rate),
      frames_per_buffer_(frames_per_buffer),
      max_delay_samples_(kMaxDelayTimeSeconds * sample_rate_),
      low_pass_filter_(0.0f),
      temp_mono_buffer_(kNumMonoChannels, frames_per_buffer_),
      current_reflection_buffer_(kNumFirstOrderAmbisonicChannels,
                                 frames_per_buffer),
      target_reflection_buffer_(kNumFirstOrderAmbisonicChannels,
                                frames_per_buffer),
      target_reflections_(kNumRoomSurfaces),
      crossfade_(false),
      crossfader_(frames_per_buffer_),
      num_frames_to_process_on_empty_input_(0),
      delays_(kNumRoomSurfaces),
      delay_filter_(max_delay_samples_, frames_per_buffer),
      delay_buffer_(kNumRoomSurfaces, frames_per_buffer),
      gains_(kNumRoomSurfaces),
      gain_processors_(kNumRoomSurfaces) {
  DCHECK_GT(sample_rate_, 0);
  DCHECK_GT(frames_per_buffer_, 0U);
}

void ReflectionsProcessor::Update(
    const ReflectionProperties& reflection_properties,
    const WorldPosition& listener_position) {

  // Initialize the low-pass filter.
  const float low_pass_coefficient = ComputeLowPassMonoPoleCoefficient(
      reflection_properties.cutoff_frequency, sample_rate_);
  low_pass_filter_.SetCoefficient(low_pass_coefficient);
  // Update the target reflections.
  WorldPosition relative_listener_position;
  GetRelativeDirection(
      WorldPosition(reflection_properties.room_position),
      WorldRotation(reflection_properties.room_rotation).conjugate(),
      listener_position, &relative_listener_position);
  ComputeReflections(relative_listener_position,
                     WorldPosition(reflection_properties.room_dimensions),
                     reflection_properties.coefficients, &target_reflections_);
  // Additional |frames_per_buffer_| to process needed to compensate the
  // crossfade between the current and target reflections.
  num_frames_to_process_on_empty_input_ =
      frames_per_buffer_ +
      static_cast<size_t>(FindMaxReflectionDelayTime(target_reflections_) *
                          static_cast<float>(sample_rate_));
  // Reflections have been updated so crossfade is required.
  crossfade_ = true;
}

void ReflectionsProcessor::Process(const AudioBuffer& input,
                                   AudioBuffer* output) {
  DCHECK_EQ(input.num_channels(), kNumMonoChannels);
  DCHECK_EQ(input.num_frames(), frames_per_buffer_);
  DCHECK(output);
  DCHECK_GE(output->num_channels(), kNumFirstOrderAmbisonicChannels);
  DCHECK_EQ(output->num_frames(), frames_per_buffer_);
  // Prefilter mono input.
  const AudioBuffer::Channel& input_channel = input[0];
  AudioBuffer::Channel* temp_channel = &temp_mono_buffer_[0];
  const bool filter_success =
      low_pass_filter_.Filter(input_channel, temp_channel);
  const AudioBuffer::Channel& low_pass_channel =
      filter_success ? *temp_channel : input_channel;
  delay_filter_.InsertData(low_pass_channel);
  // Process reflections.
  if (crossfade_) {
    ApplyReflections(&current_reflection_buffer_);
    UpdateGainsAndDelays();
    ApplyReflections(&target_reflection_buffer_);
    crossfader_.ApplyLinearCrossfade(target_reflection_buffer_,
                                     current_reflection_buffer_, output);
    crossfade_ = false;
  } else {
    ApplyReflections(output);
  }
}

void ReflectionsProcessor::UpdateGainsAndDelays() {
  for (size_t i = 0; i < kNumRoomSurfaces; ++i) {
    delays_[i] =
        std::min(max_delay_samples_,
                 static_cast<size_t>(target_reflections_[i].delay_time_seconds *
                                     static_cast<float>(sample_rate_)));
    gains_[i] = target_reflections_[i].magnitude;
  }
}

void ReflectionsProcessor::ApplyReflections(AudioBuffer* output) {
  DCHECK(output);
  DCHECK_GE(output->num_channels(), kNumFirstOrderAmbisonicChannels);
  (*output).Clear();
  for (size_t i = 0; i < kNumRoomSurfaces; ++i) {
    auto* delay_channel = &delay_buffer_[i];
    delay_filter_.GetDelayedData(delays_[i], delay_channel);
    const bool zero_gain = IsGainNearZero(gains_[i]) &&
                           IsGainNearZero(gain_processors_[i].GetGain());
    if (!zero_gain) {
      gain_processors_[i].ApplyGain(gains_[i], *delay_channel, delay_channel,
                                    false /* accumulate_output */);
      // Applies fast Ambisonic reflections encoding.
      (*output)[0] += *delay_channel;
      switch (i) {
        case 0: /* left wall reflection */
          (*output)[1] += *delay_channel;
          break;
        case 1: /* right wall reflection */
          (*output)[1] -= *delay_channel;
          break;
        case 2: /* floor reflection */
          (*output)[2] -= *delay_channel;
          break;
        case 3: /* ceiling reflection */
          (*output)[2] += *delay_channel;
          break;
        case 4: /* front wall reflection */
          (*output)[3] += *delay_channel;
          break;
        case 5: /* back wall reflection */
          (*output)[3] -= *delay_channel;
          break;
      }
    } else {
      // Make sure the gain processor is initialized.
      gain_processors_[i].Reset(0.0f);
    }
  }
}

}  // namespace vraudio
