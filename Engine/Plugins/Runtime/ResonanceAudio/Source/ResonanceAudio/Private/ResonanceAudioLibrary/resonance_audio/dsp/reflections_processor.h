/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_REFLECTIONS_PROCESSOR_H_
#define RESONANCE_AUDIO_DSP_REFLECTIONS_PROCESSOR_H_

#include <utility>
#include <vector>

#include "api/resonance_audio_api.h"
#include "base/audio_buffer.h"
#include "base/misc_math.h"
#include "dsp/delay_filter.h"
#include "dsp/gain_processor.h"
#include "dsp/mono_pole_filter.h"
#include "dsp/reflection.h"
#include "utils/buffer_crossfader.h"

namespace vraudio {

// Class that accepts single mono buffer as input and outputs a first order
// ambisonic buffer of the mix of all the rooms early reflections computed for
// the buffer.
//
// The input consists of a mono mix of all the sound objects in the system. The
// reflections are assumed to be aligned with the aabb axes and thus the first
// order ambisonic axes.
class ReflectionsProcessor {
 public:
  // Constructs a |ReflectionsProcessor|.
  //
  // @param sample_rate System sampling rate.
  // @param frames_per_buffer System frames per buffer.
  ReflectionsProcessor(int sample_rate, size_t frames_per_buffer);

  // Updates reflections according to the new |ReflectionProperties|.
  //
  // @param reflection_properties New reflection properties.
  // @param listener_position New listener position.
  void Update(const ReflectionProperties& reflection_properties,
              const WorldPosition& listener_position);

  // Processes a mono |AudioBuffer| into an ambisonic |AudioBuffer|.
  //
  // @param input Mono input buffer.
  // @param output Ambisonic output buffer.
  void Process(const AudioBuffer& input, AudioBuffer* output);

  // Returns the number of frames required to keep processing on empty input
  // signal. This value can be used to avoid any potential artifacts on the
  // final output signal when the input signal stream is empty.
  size_t num_frames_to_process_on_empty_input() const {
    return num_frames_to_process_on_empty_input_;
  }

 private:
  // Updates |gains_| and |delays_| vectors of the |ReflectionsProcessor|.
  void UpdateGainsAndDelays();

  // Applies |target_reflections_| and fast-encodes them into first order
  // ambisonics.
  //
  // @param output Ambisonic output buffer.
  void ApplyReflections(AudioBuffer* output);

  // System sampling rate.
  const int sample_rate_;

  // System number of frames per buffer.
  const size_t frames_per_buffer_;

  // Maximum allowed delay time for a reflection (in samples).
  const size_t max_delay_samples_;

  // Low pass filter to be applied to input signal.
  MonoPoleFilter low_pass_filter_;

  // Audio buffer to store mono low pass filtered buffers during processing.
  AudioBuffer temp_mono_buffer_;

  // Audio buffers to store FOA reflections buffers during crossfading.
  AudioBuffer current_reflection_buffer_;
  AudioBuffer target_reflection_buffer_;

  // Target reflections filled with new data when |Update| is called.
  std::vector<Reflection> target_reflections_;

  // Indicates whether relfections have been updated and a crossfade is needed.
  bool crossfade_;

  // Buffer crossfader to apply linear crossfade during reflections update.
  BufferCrossfader crossfader_;

  // Number of frames needed to keep processing on empty input signal.
  size_t num_frames_to_process_on_empty_input_;

  // Number of samples of delay to be applied for each reflection.
  std::vector<size_t> delays_;

  // Delay filter to delay the incoming buffer.
  DelayFilter delay_filter_;

  // Delay buffer used to store delayed reflections before scaling and encoding.
  AudioBuffer delay_buffer_;

  // Gains to be applied for each reflection.
  std::vector<float> gains_;

  // |GainProcessor|s to apply |gains_|.
  std::vector<GainProcessor> gain_processors_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_REFLECTIONS_PROCESSOR_H_
