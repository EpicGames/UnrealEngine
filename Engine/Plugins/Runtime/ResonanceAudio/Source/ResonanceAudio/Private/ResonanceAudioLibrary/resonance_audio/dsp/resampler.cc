/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "dsp/resampler.h"

#include <functional>
#include <numeric>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"
#include "base/simd_utils.h"

#include "dsp/utils.h"

namespace vraudio {

namespace {

// (kTransitionBandwidthRatio / 2) * (sample_rate / cutoff_frequency)
// = filter_length.
// The value below was chosen empirically as a tradeoff between execution time
// and filter rolloff wrt. cutoff frequency.
const size_t kTransitionBandwidthRatio = 13;

// Maximum number of channels internally based upon the maximum supported
// ambisonic order.
const size_t kMaxNumChannels =
    (kMaxSupportedAmbisonicOrder + 1) * (kMaxSupportedAmbisonicOrder + 1);

}  // namespace

Resampler::Resampler()
    : up_rate_(0),
      down_rate_(0),
      time_modulo_up_rate_(0),
      last_processed_sample_(0),
      num_channels_(0),
      coeffs_per_phase_(0),
      transposed_filter_coeffs_(kNumMonoChannels, kMaxSupportedNumFrames),
      temporary_filter_coeffs_(kNumMonoChannels, kMaxSupportedNumFrames),
      state_(kMaxNumChannels, kMaxSupportedNumFrames) {
  state_.Clear();
}

void Resampler::Process(const AudioBuffer& input, AudioBuffer* output) {

  // See "Digital Signal Processing, 4th Edition, Prolakis and Manolakis,
  // Pearson, Chapter 11 (specificaly Figures 11.5.10 and 11.5.13).
  DCHECK_EQ(input.num_channels(), num_channels_);
  const size_t input_length = input.num_frames();
  DCHECK_GE(output->num_frames(), GetNextOutputLength(input_length));
  DCHECK_LE(output->num_frames(), GetMaxOutputLength(input_length));
  DCHECK_EQ(output->num_channels(), num_channels_);
  output->Clear();

  if (up_rate_ == down_rate_) {
    *output = input;
    return;
  }

  // |input_sample| is the last processed sample.
  size_t input_sample = last_processed_sample_;
  // |output_sample| is the output.
  size_t output_sample = 0;

  const auto& filter_coefficients = transposed_filter_coeffs_[0];

  while (input_sample < input_length) {
    size_t filter_index = time_modulo_up_rate_ * coeffs_per_phase_;
    size_t offset_input_index = input_sample - coeffs_per_phase_ + 1;
    const int offset = -static_cast<int>(offset_input_index);

    if (offset > 0) {
      // We will need to draw data from the |state_| buffer.
      const int state_num_frames = static_cast<int>(coeffs_per_phase_ - 1);
      int state_index = state_num_frames - offset;
      while (state_index < state_num_frames) {
        for (size_t channel = 0; channel < num_channels_; ++channel) {
          (*output)[channel][output_sample] +=
              state_[channel][state_index] * filter_coefficients[filter_index];
        }
        state_index++;
        filter_index++;
      }
      // Move along by |offset| samples up as far as |input|.
      offset_input_index += offset;
    }

    // We now move back to where |input_sample| "points".
    while (offset_input_index <= input_sample) {
      for (size_t channel = 0; channel < num_channels_; ++channel) {
        (*output)[channel][output_sample] +=
            input[channel][offset_input_index] *
            filter_coefficients[filter_index];
      }
      offset_input_index++;
      filter_index++;
    }
    output_sample++;

    time_modulo_up_rate_ += down_rate_;
    // Advance the input pointer.
    input_sample += time_modulo_up_rate_ / up_rate_;
    // Decide which phase of the polyphase filter to use next.
    time_modulo_up_rate_ %= up_rate_;
  }
  DCHECK_GE(input_sample, input_length);
  last_processed_sample_ = input_sample - input_length;

  // Take care of the |state_| buffer.
  const int samples_left_in_input =
      static_cast<int>(coeffs_per_phase_) - 1 - static_cast<int>(input_length);
  if (samples_left_in_input > 0) {
    for (size_t channel = 0; channel < num_channels_; ++channel) {
      // Copy end of the |state_| buffer to the beginning.
      auto& state_channel = state_[channel];
      DCHECK_GE(static_cast<int>(state_channel.size()), samples_left_in_input);
      std::copy_n(state_channel.end() - samples_left_in_input,
                  samples_left_in_input, state_channel.begin());
      // Then copy input to the end of the buffer.
      std::copy_n(input[channel].begin(), input_length,
                  state_channel.end() - input_length);
    }
  } else {
    for (size_t channel = 0; channel < num_channels_; ++channel) {
      // Copy the last of the |input| samples into the |state_| buffer.
      DCHECK_GT(coeffs_per_phase_, 0U);
      DCHECK_GT(input[channel].size(), coeffs_per_phase_ - 1);
      std::copy_n(input[channel].end() - (coeffs_per_phase_ - 1),
                  coeffs_per_phase_ - 1, state_[channel].begin());
    }
  }
}

size_t Resampler::GetMaxOutputLength(size_t input_length) const {
  if (up_rate_ == down_rate_) {
    return input_length;
  }
  DCHECK_GT(down_rate_, 0U);
  // The + 1 takes care of the case where:
  // (time_modulo_up_rate_ + up_rate_ * last_processed_sample_) <
  // ((input_length * up_rate_) % down_rate_)
  // The output length will be equal to the return value or the return value -1.
  return (input_length * up_rate_) / down_rate_ + 1;
}

size_t Resampler::GetNextOutputLength(size_t input_length) const {
  if (up_rate_ == down_rate_) {
    return input_length;
  }
  const size_t max_length = GetMaxOutputLength(input_length);
  if ((time_modulo_up_rate_ + up_rate_ * last_processed_sample_) >=
      ((input_length * up_rate_) % down_rate_)) {
    return max_length - 1;
  }
  return max_length;
}

void Resampler::SetRateAndNumChannels(int source_frequency,
                                      int destination_frequency,
                                      size_t num_channels) {

  // Convert sampling rates to be relatively prime.
  DCHECK_GT(source_frequency, 0);
  DCHECK_GT(destination_frequency, 0);
  DCHECK_GT(num_channels, 0U);
  const int greatest_common_divisor =
      FindGcd(destination_frequency, source_frequency);
  const size_t destination =
      static_cast<size_t>(destination_frequency / greatest_common_divisor);
  const size_t source =
      static_cast<size_t>(source_frequency / greatest_common_divisor);

  // Obtain the size of the |state_| before |coeffs_per_phase_| is updated in
  // |GenerateInterpolatingFilter()|.
  const size_t old_state_size =
      coeffs_per_phase_ > 0 ? coeffs_per_phase_ - 1 : 0;
  if ((destination != up_rate_) || (source != down_rate_)) {
    up_rate_ = destination;
    down_rate_ = source;
    if (up_rate_ == down_rate_) {
      return;
    }
    // Create transposed multirate filters from sincs.
    GenerateInterpolatingFilter(source_frequency);
    // Reset the time variable as it may be longer than the new filter length if
    // we switched from upsampling to downsampling via a call to SetRate().
    time_modulo_up_rate_ = 0;
  }

  // Update the |state_| buffer.
  if (num_channels_ != num_channels) {
    num_channels_ = num_channels;
    InitializeStateBuffer(old_state_size);
  }
}

bool Resampler::AreSampleRatesSupported(int source, int destination) {
  DCHECK_GT(source, 0);
  DCHECK_GT(destination, 0);
  // Determines whether sample rates are supported based upon whether our
  // maximul filter lenhgth is big enough to hold the corresponding
  // interpolation filter.
  const int max_rate =
      std::max(source, destination) / FindGcd(source, destination);
  size_t filter_length = max_rate * kTransitionBandwidthRatio;
  filter_length += filter_length % 2;
  return filter_length <= kMaxSupportedNumFrames;
}

void Resampler::ResetState() {

  time_modulo_up_rate_ = 0;
  last_processed_sample_ = 0;
  state_.Clear();
}

void Resampler::InitializeStateBuffer(size_t old_state_num_frames) {
  // Update the |state_| buffer if it is null or if the number of coefficients
  // per phase in the polyphase filter has changed.
  if (up_rate_ == down_rate_ || num_channels_ == 0) {
    return;
  }
  // If the |state_| buffer is to be kept. For example in the case of a change
  // in either source or destination sampling rate, maintaining the old |state_|
  // buffers contents allows a glitch free transition.
  const size_t new_state_num_frames =
      coeffs_per_phase_ > 0 ? coeffs_per_phase_ - 1 : 0;
  if (old_state_num_frames != new_state_num_frames) {
    const size_t min_size =
        std::min(new_state_num_frames, old_state_num_frames);
    const size_t max_size =
        std::max(new_state_num_frames, old_state_num_frames);
    for (size_t channel = 0; channel < num_channels_; ++channel) {
      auto& state_channel = state_[channel];
      DCHECK_LT(state_channel.begin() + max_size, state_channel.end());
      std::fill(state_channel.begin() + min_size,
                state_channel.begin() + max_size, 0.0f);
    }
  }
}

void Resampler::GenerateInterpolatingFilter(int sample_rate) {
  // See "Digital Signal Processing, 4th Edition, Prolakis and Manolakis,
  // Pearson, Chapter 11 (specificaly Figures 11.5.10 and 11.5.13).
  const size_t max_rate = std::max(up_rate_, down_rate_);
  const float cutoff_frequency =
      static_cast<float>(sample_rate) / static_cast<float>(2 * max_rate);
  size_t filter_length = max_rate * kTransitionBandwidthRatio;
  filter_length += filter_length % 2;
  auto* filter_channel = &temporary_filter_coeffs_[0];
  filter_channel->Clear();
  GenerateSincFilter(cutoff_frequency, static_cast<float>(sample_rate),
                     filter_length, filter_channel);

  // Pad out the filter length so that it can be arranged in polyphase fashion.
  const size_t transposed_length =
      filter_length + max_rate - (filter_length % max_rate);
  coeffs_per_phase_ = transposed_length / max_rate;
  ArrangeFilterAsPolyphase(filter_length, *filter_channel);
}

void Resampler::ArrangeFilterAsPolyphase(size_t filter_length,
                                         const AudioBuffer::Channel& filter) {
  // Coefficients are transposed and flipped.
  // Suppose |up_rate_| is 3, and the input number of coefficients is 10,
  // h[0], ..., h[9].
  // Then the |transposed_filter_coeffs_| buffer will look like this:
  // h[9], h[6], h[3], h[0],   flipped phase 0 coefs.
  //  0,   h[7], h[4], h[1],   flipped phase 1 coefs (zero-padded).
  //  0,   h[8], h[5], h[2],   flipped phase 2 coefs (zero-padded).
  transposed_filter_coeffs_.Clear();
  auto& transposed_coefficients_channel = transposed_filter_coeffs_[0];
  for (size_t i = 0; i < up_rate_; ++i) {
    for (size_t j = 0; j < coeffs_per_phase_; ++j) {
      if (j * up_rate_ + i < filter_length) {
        const size_t coeff_index =
            (coeffs_per_phase_ - 1 - j) + i * coeffs_per_phase_;
        transposed_coefficients_channel[coeff_index] = filter[j * up_rate_ + i];
      }
    }
  }
}

void Resampler::GenerateSincFilter(float cutoff_frequency, float sample_rate,
                                   size_t filter_length,
                                   AudioBuffer::Channel* buffer) {

  DCHECK_GT(sample_rate, 0.0f);
  const float angular_cutoff_frequency =
      kTwoPi * cutoff_frequency / sample_rate;

  const size_t half_filter_length = filter_length / 2;
  GenerateHannWindow(true /* Full Window */, filter_length, buffer);
  auto* buffer_channel = &buffer[0];

  for (size_t i = 0; i < filter_length; ++i) {
    if (i == half_filter_length) {
      (*buffer_channel)[half_filter_length] *= angular_cutoff_frequency;
    } else {
      const float denominator =
          static_cast<float>(i) - (static_cast<float>(filter_length) / 2.0f);
      DCHECK_GT(std::abs(denominator), kEpsilonFloat);
      (*buffer_channel)[i] *=
          std::sin(angular_cutoff_frequency * denominator) / denominator;
    }
  }
  // Normalize.
  const float normalizing_factor =
      static_cast<float>(up_rate_) /
      std::accumulate(buffer_channel->begin(), buffer_channel->end(), 0.0f);
  ScalarMultiply(filter_length, normalizing_factor, buffer_channel->begin(),
                 buffer_channel->begin());
}

}  // namespace vraudio
