/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_RESAMPLER_H_
#define RESONANCE_AUDIO_DSP_RESAMPLER_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Class that provides rational resampling of audio data.
class Resampler {
 public:
  Resampler();

  // Resamples an |AudioBuffer| of input data sampled at |source_frequency| to
  // |destination_frequency|.
  //
  // @param input Input data to be resampled.
  // @param output Resampled output data.
  void Process(const AudioBuffer& input, AudioBuffer* output);

  // Returns the maximum length which the output buffer will be, given the
  // current source and destination frequencies and input length. The actual
  // output length will either be this or one less.
  //
  // @param input_length Length of the input.
  // @return Maximum length of the output.
  size_t GetMaxOutputLength(size_t input_length) const;

  // Returns the next length which the output buffer will be, given the
  // current source and destination frequencies and input length.
  //
  // @param input_length Length of the input.
  // @return Next length of the output.
  size_t GetNextOutputLength(size_t input_length) const;

  // Sets the source and destination sampling rate as well as the number of
  // channels. Note this method only resets the filter state number of channel
  // changes.
  //
  // @param source_frequency Sampling rate of input data.
  // @param destination_frequency Desired output sampling rate.
  // @param num_channels Number of channels to process.
  void SetRateAndNumChannels(int source_frequency, int destination_frequency,
                             size_t num_channels);

  // Returns whether the sampling rates provided are supported by the resampler.
  //
  // @param source Source sampling rate.
  // @param destination Destination sampling rate.
  // @return True if the sampling rate pair are supported.
  static bool AreSampleRatesSupported(int source, int destination);

  // Resets the inner state of the |Resampler| allowing its use repeatedly on
  // different data streams.
  void ResetState();

 private:
  friend class PolyphaseFilterTest;
  // Initializes the |state_| buffer. Called when sampling rate is changed or
  // the state is reset.
  //
  // @param size_t old_state_num_frames Number of frames in the |state_| buffer
  //     previous to the most recent call to |GenerateInterpolatingFilter|.
  void InitializeStateBuffer(size_t old_state_num_frames);

  // Generates a windowed sinc to act as the interpolating/anti-aliasing filter.
  //
  // @param sample_rate The system sampling rate.
  void GenerateInterpolatingFilter(int sample_rate);

  // Arranges the anti aliasing filter coefficients in polyphase filter format.
  //
  // @param filter_length Number of frames in |filter| containing filter
  //     coefficients.
  // @param filter Vector of filter coefficients.
  void ArrangeFilterAsPolyphase(size_t filter_length,
                                const AudioBuffer::Channel& filter);

  // Generates Hann windowed sinc function anti aliasing filters.
  //
  // @param cutoff_frequency Transition band (-3dB) frequency of the filter.
  // @param sample_rate The system sampling rate.
  // @param filter_length Number of frames in |buffer| containing filter
  //     coefficients.
  // @param buffer |AudioBuffer::Channel| to contain the filter coefficients.
  void GenerateSincFilter(float cutoff_frequency, float sample_rate,
                          size_t filter_length, AudioBuffer::Channel* buffer);

  // Rate of the interpolator section of the rational sampling rate converter.
  size_t up_rate_;

  // Rate of the decimator section of the rational sampling rate convereter.
  size_t down_rate_;

  // Time variable for the polyphase filter.
  size_t time_modulo_up_rate_;

  // Marks the last processed sample of the input.
  size_t last_processed_sample_;

  // Number of channels in the |AudioBuffer|s processed.
  size_t num_channels_;

  // Number of filter coefficients in each phase of the polyphase filter.
  size_t coeffs_per_phase_;

  // Filter coefficients stored in polyphase form.
  AudioBuffer transposed_filter_coeffs_;

  // Filter coefficients in planar form, used for calculating the transposed
  // filter.
  AudioBuffer temporary_filter_coeffs_;

  // Buffer holding the samples of input required between input buffers.
  AudioBuffer state_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_RESAMPLER_H_
