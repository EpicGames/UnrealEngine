/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/reverb_onset_compensator.h"

#include <algorithm>
#include <cmath>
#include <iterator>

#include "base/constants_and_types.h"
#include "dsp/spectral_reverb_constants_and_tables.h"
#include "dsp/utils.h"

namespace vraudio {

namespace {

// Number of reverb updaters. Twelve were chosen as this represents one update
// per buffer length at 24kHz, a number we are very unlikely to exceed.
const size_t kNumReverbUpdaters = 12;

}  // namespace

ReverbOnsetCompensator::ReverbOnsetCompensator(int sampling_rate,
                                               size_t frames_per_buffer,
                                               FftManager* fft_manager)
    : fft_manager_(fft_manager),
      sampling_rate_(sampling_rate),
      frames_per_buffer_(frames_per_buffer),
      base_curves_(kNumStereoChannels, kCorrectionCurveLength),
      adder_curves_(kNumStereoChannels, kCorrectionCurveLength),
      left_filter_(CeilToMultipleOfFramesPerBuffer(kCorrectionCurveLength,
                                                   frames_per_buffer_),
                   frames_per_buffer_, fft_manager_),
      right_filter_(CeilToMultipleOfFramesPerBuffer(kCorrectionCurveLength,
                                                    frames_per_buffer_),
                    frames_per_buffer_, fft_manager_),
      delay_filter_(CeilToMultipleOfFramesPerBuffer(kCorrectionCurveLength,
                                                    frames_per_buffer_),
                    frames_per_buffer_),
      num_active_processors_(0),
      temp_kernel_buffer_(kNumStereoChannels, frames_per_buffer_),
      temp_freq_buffer_(kNumMonoChannels, fft_manager_->GetFftSize()) {
  CHECK(fft_manager_);
  DCHECK_GT(sampling_rate_, 0);
  DCHECK_GT(frames_per_buffer_, 0U);

  temp_kernel_buffer_.Clear();
  temp_freq_buffer_.Clear();

  GenerateNoiseVectors();
  GenerateCorrectionCurves();

  // Insert reverb updaters.
  for (size_t i = 0; i < kNumReverbUpdaters; ++i) {
    update_processors_.emplace_front(new ReverbOnsetUpdateProcessor(
        frames_per_buffer_, sampling_rate_, &base_curves_, &adder_curves_));
  }
}

void ReverbOnsetCompensator::Process(const AudioBuffer& input,
                                     AudioBuffer* output) {
  DCHECK(output);
  DCHECK_EQ(kNumMonoChannels, input.num_channels());
  DCHECK_EQ(frames_per_buffer_, input.num_frames());
  DCHECK_EQ(kNumStereoChannels, output->num_channels());
  DCHECK_EQ(frames_per_buffer_, output->num_frames());

  delay_filter_.InsertData(input[0]);
  delay_filter_.GetDelayedData(kCompensationOnsetLength, &(*output)[0]);

  // Process reverb updates.
  AudioBuffer::Channel* kernel_channel_left = &temp_kernel_buffer_[0];
  AudioBuffer::Channel* kernel_channel_right = &temp_kernel_buffer_[1];

  size_t processor_index = 0;
  while (processor_index < num_active_processors_) {
    auto current_processor = update_processors_.begin();
    std::advance(current_processor, processor_index);
    const size_t partition_index =
        (*current_processor)->GetCurrentPartitionIndex();
    if ((*current_processor)
            ->Process(bandpassed_noise_left_, bandpassed_noise_right_,
                      kernel_channel_left, kernel_channel_right)) {
      left_filter_.ReplacePartition(partition_index, *kernel_channel_left);
      right_filter_.ReplacePartition(partition_index, *kernel_channel_right);
      ++processor_index;
    } else {
      // Update of the |current_processor| is finished, move it to the end of
      // the list and reduce the number of active processors.
      update_processors_.splice(update_processors_.end(), update_processors_,
                                current_processor);
      --num_active_processors_;
    }
  }

  // Filter the input (Using the output buffer due to the delay operation).
  fft_manager_->FreqFromTimeDomain((*output)[0], &temp_freq_buffer_[0]);

  left_filter_.Filter(temp_freq_buffer_[0]);
  right_filter_.Filter(temp_freq_buffer_[0]);

  left_filter_.GetFilteredSignal(&(*output)[0]);
  right_filter_.GetFilteredSignal(&(*output)[1]);
}

void ReverbOnsetCompensator::Update(const float* rt60_values, float gain) {
  DCHECK(rt60_values);
  // Reset a reverb update processor from the end of the list and place it at
  // the front. If the list is full, rotate the list and reuse the oldest active
  // processor.
  std::list<std::unique_ptr<ReverbOnsetUpdateProcessor>>::iterator
      new_processor;
  if (num_active_processors_ < kNumReverbUpdaters) {
    new_processor = update_processors_.end();
    std::advance(new_processor, -1);
  } else {
    new_processor = update_processors_.begin();
  }

  (*new_processor)->SetReverbTimes(rt60_values);
  (*new_processor)->SetGain(gain);

  if (new_processor != update_processors_.begin()) {
    auto list_item = update_processors_.begin();
    std::advance(list_item, num_active_processors_);
    if (list_item != new_processor) {
      update_processors_.splice(list_item, update_processors_, new_processor,
                                std::next(new_processor));
    }
    ++num_active_processors_;
  } else {
    std::rotate(update_processors_.begin(),
                std::next(update_processors_.begin()),
                update_processors_.end());
  }
}

void ReverbOnsetCompensator::GenerateCorrectionCurves() {
  // Copy into the adder curves such that the memory is aligned.
  std::copy(kLowCorrectionCurve, kLowCorrectionCurve + kCorrectionCurveLength,
            adder_curves_[0].begin());
  std::copy(kHighCorrectionCurve, kHighCorrectionCurve + kCorrectionCurveLength,
            adder_curves_[1].begin());

  // Evaluate the polynomials to generate the base curves. Here the 'low' and
  // 'high' names refer to the reverberation times.
  AudioBuffer::Channel* low_channel = &base_curves_[0];
  AudioBuffer::Channel* high_channel = &base_curves_[1];
  for (size_t i = 0; i < kCorrectionCurveLength; ++i) {
    // Scaled independent variable (Allowed better conditioning).
    const float conditioning_scalar =
        (static_cast<float>(i) - kCurveOffset) * kCurveScale;
    (*low_channel)[i] = kLowReverberationCorrectionCurve[0];
    (*high_channel)[i] = kHighReverberationCorrectionCurve[0];
    float power = conditioning_scalar;
    for (size_t k = 1; k < kCurvePolynomialLength; ++k) {
      (*low_channel)[i] += power * kLowReverberationCorrectionCurve[k];
      (*high_channel)[i] += power * kHighReverberationCorrectionCurve[k];
      power *= conditioning_scalar;
    }
    (*low_channel)[i] = std::max((*low_channel)[i], 0.0f);
    (*high_channel)[i] = std::max((*high_channel)[i], 0.0f);
  }
}

void ReverbOnsetCompensator::GenerateNoiseVectors() {
  const size_t num_octave_bands = GetNumReverbOctaveBands(sampling_rate_);
  const size_t noise_length = CeilToMultipleOfFramesPerBuffer(
      kCorrectionCurveLength, frames_per_buffer_);
  for (size_t band = 0; band < num_octave_bands; ++band) {
    // Generate preset tail.
    bandpassed_noise_left_.emplace_back(kNumMonoChannels, noise_length);
    GenerateBandLimitedGaussianNoise(kOctaveBandCentres[band], sampling_rate_,
                                     /*seed=*/1U,
                                     &bandpassed_noise_left_[band]);
    bandpassed_noise_right_.emplace_back(kNumMonoChannels, noise_length);
    GenerateBandLimitedGaussianNoise(kOctaveBandCentres[band], sampling_rate_,
                                     /*seed=*/2U,
                                     &bandpassed_noise_right_[band]);

    auto min_max = std::minmax_element(bandpassed_noise_left_[band][0].begin(),
                                       bandpassed_noise_left_[band][0].end());
    const float left_scale =
        std::max(std::fabs(*min_max.first), std::fabs(*min_max.second));
    min_max = std::minmax_element(bandpassed_noise_right_[band][0].begin(),
                                  bandpassed_noise_right_[band][0].end());
    const float right_scale =
        std::max(std::fabs(*min_max.first), std::fabs(*min_max.second));

    const float scale = std::max(left_scale, right_scale);

    ScalarMultiply(noise_length, scale, bandpassed_noise_left_[band][0].begin(),
                   bandpassed_noise_left_[band][0].begin());
    ScalarMultiply(noise_length, scale,
                   bandpassed_noise_right_[band][0].begin(),
                   bandpassed_noise_right_[band][0].begin());
  }
}

}  // namespace vraudio
