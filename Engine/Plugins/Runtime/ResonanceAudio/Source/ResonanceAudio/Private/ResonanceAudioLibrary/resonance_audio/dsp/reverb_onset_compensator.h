/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_REVERB_ONSET_COMPENSATOR_H_
#define RESONANCE_AUDIO_DSP_REVERB_ONSET_COMPENSATOR_H_

#include <list>
#include <vector>

#include "base/audio_buffer.h"
#include "dsp/delay_filter.h"
#include "dsp/fft_manager.h"
#include "dsp/partitioned_fft_filter.h"
#include "dsp/reverb_onset_update_processor.h"

namespace vraudio {

// Implements a convolutional compensator for the spectral reverb onset curve.
class ReverbOnsetCompensator {
 public:
  // Constructs a |ReverbOnsetCompensator|.
  //
  // @param sampling_rate The sampling rate in Hz.
  // @param frames_per_buffer The number of frames per buffer in the system.
  // @param fft_manager Pointer to a FftManager to perform FFT transformations.
  ReverbOnsetCompensator(int sampling_rate, size_t frames_per_buffer,
                         FftManager* fft_manager);

  // Resets the reverb with a new set of reverberation times. The new tail is
  // generated by replacing the current tail buffer by buffer.
  //
  // @param rt60_values |kNumReverbOctaveBands| values denoting the
  //     reverberation decay time to -60dB in octave bands starting at
  //     |kLowestOctaveBand|.
  // @param gain Gain to be applied across all frequencies.
  void Update(const float* rt60_values, float gain);

  // Processes a mono |AudioBuffer| with a reverberant tail.
  //
  // @param input A mono |AudioBuffer| of input data.
  // @param output Pointer to stereo output buffer.
  void Process(const AudioBuffer& input, AudioBuffer* output);

 private:
  // Generates the constituent curves which are combined to make up the
  // correction curve envelopes. These envelopes ensure the initial part of the
  // specral reverb's impulse response, which exhibits 'growing' behaviour,
  // follows the desired exponential decay.
  void GenerateCorrectionCurves();

  // Generates a pair of |kNumOctaveBands| band, octave filtered, noise buffers.
  void GenerateNoiseVectors();

  // Manager for all FFT related functionality (not owned).
  FftManager* const fft_manager_;

  // The system sampling rate.
  const int sampling_rate_;

  // The system number of frames per buffer.
  const size_t frames_per_buffer_;

  // Pre-generated band-passed noise to be used as a base for the reverb tail.
  std::vector<AudioBuffer> bandpassed_noise_left_;
  std::vector<AudioBuffer> bandpassed_noise_right_;

  // The constituent curves used to generate the onset compensation envelopes.
  AudioBuffer base_curves_;
  AudioBuffer adder_curves_;

  // Filter for processing the left reverberant channel.
  PartitionedFftFilter left_filter_;

  // Filter for processing the right reverberant channel.
  PartitionedFftFilter right_filter_;

  // Delay filter used to ensure the compensation curve starts at the same point
  // as the spectral reverb.
  DelayFilter delay_filter_;

  // Number of active update processors.
  size_t num_active_processors_;

  // Active reverb update processors to replace the corresponding filter
  // partitions of the reverb tail within each process call.
  std::list<std::unique_ptr<ReverbOnsetUpdateProcessor>> update_processors_;

  // Temporary buffer used to process filter kernel partitions.
  AudioBuffer temp_kernel_buffer_;

  // Temporary buffer to hold FFT frequency domain output.
  PartitionedFftFilter::FreqDomainBuffer temp_freq_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_REVERB_ONSET_COMPENSATOR_H_
