/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/reverb_onset_update_processor.h"

#include <algorithm>

#include "base/constants_and_types.h"
#include "base/simd_utils.h"
#include "dsp/spectral_reverb_constants_and_tables.h"
#include "dsp/utils.h"

namespace vraudio {

namespace {

// Find the absolute difference between two size_t values.
inline size_t absdiff(size_t lhs, size_t rhs) {
  return lhs > rhs ? lhs - rhs : rhs - lhs;
}

}  // namespace

ReverbOnsetUpdateProcessor::ReverbOnsetUpdateProcessor(
    size_t frames_per_buffer, int sampling_rate, AudioBuffer* base_curves,
    AudioBuffer* adder_curves)
    : sampling_rate_(sampling_rate),
      tail_update_cursor_(0),
      tail_length_(CeilToMultipleOfFramesPerBuffer(kCorrectionCurveLength,
                                                   frames_per_buffer)),
      gain_(1.0f),
      curve_indices_(GetNumReverbOctaveBands(sampling_rate_), kInvalidIndex),
      pure_decay_coefficients_(curve_indices_.size(), 0.0f),
      pure_decay_exponents_(curve_indices_.size(), 0.0f),
      band_buffer_(kNumStereoChannels, frames_per_buffer),
      envelope_buffer_(kNumMonoChannels, frames_per_buffer),
      base_curves_(base_curves),
      adder_curves_(adder_curves) {}

void ReverbOnsetUpdateProcessor::SetReverbTimes(const float* rt60_values) {
  DCHECK(rt60_values);
  const size_t num_octave_bands = curve_indices_.size();
  const float sampling_rate_float = static_cast<float>(sampling_rate_);
  tail_update_cursor_ = 0;
  // Choose curves for each band.
  for (size_t band = 0; band < num_octave_bands; ++band) {
    curve_indices_[band] =
        GetFeedbackIndexFromRt60(rt60_values[band], sampling_rate_float);
    // Deal with the case where only the convolution is needed.
    if (curve_indices_[band] == kInvalidIndex) {
      const float min_reverb_time =
          kMinReverbTimeForFeedback48kHz *
          (sampling_rate_float / kDefaultSpectralReverbSampleRate);
      const float effective_rt =
          rt60_values[band] <= min_reverb_time ? rt60_values[band] : 0.0f;
      pure_decay_exponents_[band] =
          std::abs(effective_rt) > kEpsilonFloat
              ? std::exp(kNegativeLog1000 /
                         (sampling_rate_float * effective_rt))
              : 0.0f;
      pure_decay_coefficients_[band] = pure_decay_exponents_[band];
    }
  }
}

bool ReverbOnsetUpdateProcessor::Process(
    const std::vector<AudioBuffer>& bandpassed_noise_left,
    const std::vector<AudioBuffer>& bandpassed_noise_right,
    AudioBuffer::Channel* kernel_channel_left,
    AudioBuffer::Channel* kernel_channel_right) {
  if (tail_update_cursor_ >= tail_length_) {
    // Processing the reverb tail is finished.
    tail_update_cursor_ = 0;
    return false;
  }
  const size_t frames_per_buffer = band_buffer_.num_frames();
  DCHECK(kernel_channel_left);
  DCHECK(kernel_channel_right);
  DCHECK_EQ(bandpassed_noise_left.size(), curve_indices_.size());
  DCHECK_EQ(bandpassed_noise_right.size(), curve_indices_.size());
  DCHECK_EQ(bandpassed_noise_left[0].num_frames(), tail_length_);
  DCHECK_EQ(bandpassed_noise_right[0].num_frames(), tail_length_);
  DCHECK_GE(tail_length_, kCorrectionCurveLength);
  DCHECK_EQ(kernel_channel_left->size(), frames_per_buffer);
  DCHECK_EQ(kernel_channel_right->size(), frames_per_buffer);

  // Clear for accumulation per frequency band.
  kernel_channel_left->Clear();
  kernel_channel_right->Clear();

  AudioBuffer::Channel& band_channel_left = band_buffer_[0];
  AudioBuffer::Channel& band_channel_right = band_buffer_[1];
  // Define the number of samples we are still able to copy from the multiplier
  // and adder curves.
  const size_t copy_length =
      frames_per_buffer + tail_update_cursor_ <= kCorrectionCurveLength
          ? frames_per_buffer
          : absdiff(kCorrectionCurveLength, tail_update_cursor_);
  AudioBuffer::Channel* envelope_channel = &envelope_buffer_[0];
  // Compute the band buffer for each band response.
  for (size_t band = 0; band < curve_indices_.size(); ++band) {
    const AudioBuffer::Channel& noise_channel_left =
        bandpassed_noise_left[band][0];
    const AudioBuffer::Channel& noise_channel_right =
        bandpassed_noise_right[band][0];
    // Fill the band buffer with the next noise buffer and apply gain.
    ScalarMultiply(frames_per_buffer, gain_,
                   noise_channel_left.begin() + tail_update_cursor_,
                   band_channel_left.begin());
    ScalarMultiply(frames_per_buffer, gain_,
                   noise_channel_right.begin() + tail_update_cursor_,
                   band_channel_right.begin());
    // Skip the band if we have an invalid index
    const int curve_index = curve_indices_[band];
    if (curve_index != kInvalidIndex) {
      // Apply the correct compensation curve to the buffer.
      const float scale = kCurveCorrectionMultipliers[curve_index];
      AudioBuffer::Channel* adder_curve_channel;
      if (tail_update_cursor_ < kCorrectionCurveLength) {
        // Use either the high frequency or low frequency curve.
        if (static_cast<size_t>(curve_index) >= kCurveChangeoverIndex) {
          adder_curve_channel = &(*adder_curves_)[1];
          std::copy_n((*base_curves_)[1].begin() + tail_update_cursor_,
                      copy_length, envelope_channel->begin());
        } else {
          adder_curve_channel = &(*adder_curves_)[0];
          std::copy_n((*base_curves_)[0].begin() + tail_update_cursor_,
                      copy_length, envelope_channel->begin());
        }
        // Construct the correct envelope (chunk thereof).
        ScalarMultiplyAndAccumulate(
            copy_length, scale,
            adder_curve_channel->begin() + tail_update_cursor_,
            envelope_channel->begin());
        // Ensure the end part of the envelope does not contain spurious data.
        std::fill(envelope_channel->begin() + copy_length,
                  envelope_channel->end(), 0.0f);
      } else {
        // If we have moved past the length of the correction curve, fill the
        // envelope chunk with zeros.
        envelope_channel->Clear();
      }

      // Apply that envelope to the given band and accumulate into the output.
      MultiplyAndAccumulatePointwise(
          frames_per_buffer, envelope_channel->begin(),
          band_channel_left.begin(), kernel_channel_left->begin());
      MultiplyAndAccumulatePointwise(
          frames_per_buffer, envelope_channel->begin(),
          band_channel_right.begin(), kernel_channel_right->begin());
    } else {
      // If the decay time is too short for the spectral reverb to make a
      // contribution (0.15s @48kHz), the compensation filter will consist of
      // the entire tail.
      for (size_t frame = 0; frame < frames_per_buffer; ++frame) {
        (*kernel_channel_left)[frame] +=
            pure_decay_coefficients_[band] * band_channel_left[frame];
        (*kernel_channel_right)[frame] +=
            pure_decay_coefficients_[band] * band_channel_right[frame];
        // Update the decay coefficient.
        pure_decay_coefficients_[band] *= pure_decay_exponents_[band];
      }
    }
  }
  // Update the cursor.
  tail_update_cursor_ += frames_per_buffer;

  return true;
}

}  // namespace vraudio
