/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_REVERB_ONSET_UPDATE_PROCESSOR_H_
#define RESONANCE_AUDIO_DSP_REVERB_ONSET_UPDATE_PROCESSOR_H_

#include <vector>

#include "base/audio_buffer.h"

namespace vraudio {

// Updater for the |ReverbOnsetCompensator|.
class ReverbOnsetUpdateProcessor {
 public:
  // Constructs a update processor for the sprectral reverb onset compensator.
  //
  // @param frames_per_buffer_ System buffer length in frames.
  // @param sampling_rate System sample rate.
  // @param base_curves Constituent curve used for envelope generation.
  // @param ader_curves Constituent curve used for envelope generation.
  ReverbOnsetUpdateProcessor(size_t frames_per_buffer, int sampling_rate,
                             AudioBuffer* base_curves,
                             AudioBuffer* adder_curves);

  // Sets reverberation times in different frequency bands.
  //
  // @param rt60_values |kNumReverbOctaveBands| values denoting the
  //     reverberation decay time to -60dB in octave bands starting at
  //     |kLowestOctaveBand|.
  void SetReverbTimes(const float* rt60_values);

  // Sets the gain applied to the overall compensation envelope.
  //
  // @param gain Gain applied to overall envelope.
  void SetGain(float gain) { gain_ = gain; }

  // Processes the next tail update.
  //
  // @param bandpassed_noise_left Pre-computed bandpassed noise buffer.
  // @param bandpassed_noise_right Pre-computed bandpassed noise buffer.
  // @param kernel_channel_left Kernel channel to fill in the processed output.
  // @param kernel_channel_right Kernel channel to fill in the processed output.
  // @return True if the tail update continues.
  bool Process(const std::vector<AudioBuffer>& bandpassed_noise_left,
               const std::vector<AudioBuffer>& bandpassed_noise_right,
               AudioBuffer::Channel* kernel_channel_left,
               AudioBuffer::Channel* kernel_channel_right);

  // Returns the partition index of the current update state.
  //
  // @return Current partition index.
  size_t GetCurrentPartitionIndex() const {
    const size_t frames_per_buffer = band_buffer_.num_frames();
    DCHECK_NE(frames_per_buffer, 0U);
    return tail_update_cursor_ / frames_per_buffer;
  }

  // Disable copy and assignment operator.
  ReverbOnsetUpdateProcessor(ReverbOnsetUpdateProcessor const&) = delete;
  void operator=(ReverbOnsetUpdateProcessor const&) = delete;

 private:
  // System sample rate.
  int sampling_rate_;

  // Current frame position of the reverb tail to be updated.
  size_t tail_update_cursor_;

  // Length of the new reverb tail to be replaced in frames.
  size_t tail_length_;

  // Gain applied to the reverb compensation.
  float gain_;

  // Indices of the multiplication factor to be used to create the onset
  // compensation curve at each frequency band.
  std::vector<int> curve_indices_;

  // Decay coefficients per each band of the reverb tail, used below 0.15s
  // @48kHz.
  std::vector<float> pure_decay_coefficients_;

  // Decay exponential per each band of the reverb tail, used below 0.15s
  // @48kHz.
  std::vector<float> pure_decay_exponents_;

  // Temporary buffers used to process the decayed noise per each band.
  AudioBuffer band_buffer_;
  AudioBuffer envelope_buffer_;

  // Pointers to audio buffers owned by the |ReverbOnsetCompensator| storing the
  // constituent curves used to generate the onset compensation envelopes.
  AudioBuffer* base_curves_;
  AudioBuffer* adder_curves_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_REVERB_ONSET_UPDATE_PROCESSOR_H_
