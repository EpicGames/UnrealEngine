/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/sh_hrir_creator.h"

#include "third_party/SADIE_hrtf_database/generated/hrtf_assets.h"
#include "ambisonics/utils.h"
#include "base/logging.h"
#include "dsp/resampler.h"
#include "utils/planar_interleaved_conversion.h"

namespace vraudio {

std::unique_ptr<AudioBuffer> CreateShHrirsFromWav(const Wav& wav,
                                                  int target_sample_rate_hz,
                                                  Resampler* resampler) {
  DCHECK(resampler);
  const size_t num_channels = wav.GetNumChannels();
  CHECK(IsValidAmbisonicOrder(num_channels));

  const size_t sh_hrir_length = wav.interleaved_samples().size() / num_channels;
  std::unique_ptr<AudioBuffer> sh_hrirs(
      new AudioBuffer(num_channels, sh_hrir_length));
  FillAudioBuffer(wav.interleaved_samples(), num_channels, sh_hrirs.get());

  const int wav_sample_rate_hz = wav.GetSampleRateHz();
  CHECK_GT(wav_sample_rate_hz, 0);
  CHECK_GT(target_sample_rate_hz, 0);
  if (wav_sample_rate_hz != target_sample_rate_hz) {
    if (!Resampler::AreSampleRatesSupported(wav_sample_rate_hz,
                                            target_sample_rate_hz)) {
      LOG(FATAL) << "Unsupported sampling rates for loading HRIRs: "
                 << wav_sample_rate_hz << ", " << target_sample_rate_hz;
    }
    resampler->ResetState();
    // Resample the SH HRIRs if necessary.
    resampler->SetRateAndNumChannels(wav_sample_rate_hz, target_sample_rate_hz,
                                     num_channels);
    std::unique_ptr<AudioBuffer> resampled_sh_hrirs(new AudioBuffer(
        num_channels, resampler->GetNextOutputLength(sh_hrir_length)));
    resampler->Process(*sh_hrirs, resampled_sh_hrirs.get());
    return resampled_sh_hrirs;
  }
  return sh_hrirs;
}

std::unique_ptr<AudioBuffer> CreateShHrirsFromAssets(
    const std::string& filename, int target_sample_rate_hz,
    Resampler* resampler) {
  // Read SH HRIR from asset store.
  size_t sh_hrir_data_size = 0;
  const unsigned char* sh_hrir_data = sadie::HrtfAssets::GetFile(filename, sh_hrir_data_size);
  CHECK_NOTNULL(sh_hrir_data);

  std::unique_ptr<const Wav> wav = Wav::CreateOrNull(sh_hrir_data, sh_hrir_data_size);
  CHECK_NOTNULL(wav.get());
  return CreateShHrirsFromWav(*wav, target_sample_rate_hz, resampler);
}

}  // namespace vraudio
