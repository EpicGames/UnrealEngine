/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_SH_HRIR_CREATOR_H_
#define RESONANCE_AUDIO_DSP_SH_HRIR_CREATOR_H_

#include "base/audio_buffer.h"
#include "dsp/resampler.h"
#include "utils/wav.h"

namespace vraudio {

// Creates a multichannel audio buffer of Spherical Harmonic-encoded Head
// Related Impulse Responses (SH HRIRs) from Wav SH HRIR assets. It also
// checks if the channel count of the SH HRIR file is correct and resamples the
// SH HRIRs if necessary to match the system (target) sampling rate.
//
// @param wav |Wav| instance that contains SH HRIRs.
// @param target_sample_rate_hz Target sampling rate in Hertz.
// @param resampler Pointer to a resampler used to convert HRIRs to the system
// rate,
//     (This resampler's internal state will be reset on each function call).
// @return Unique pointer to |AudioBuffer| where the SH HRIRs will be written.
std::unique_ptr<AudioBuffer> CreateShHrirsFromWav(const Wav& wav,
                                                  int target_sample_rate_hz,
                                                  Resampler* resampler);

// Creates a SH HRIR multichannel audio buffer from assets.
//
// @param filename Name of the Wav file that contains SH HRIRs.
// @param target_sample_rate_hz Target sampling rate in Hertz.
// @param resampler Pointer to a resampler used to convert HRIRs to the system
// rate.
// @return Unique pointer to |AudioBuffer| where the SH HRIRs will be written.
std::unique_ptr<AudioBuffer> CreateShHrirsFromAssets(
    const std::string& filename, int target_sample_rate_hz,
    Resampler* resampler);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_SH_HRIR_CREATOR_H_
