/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/shoe_box_room.h"

#include "base/constants_and_types.h"

namespace vraudio {

void ComputeReflections(const WorldPosition& relative_listener_position,
                        const WorldPosition& room_dimensions,
                        const float* reflection_coefficients,
                        std::vector<Reflection>* reflections) {
  DCHECK(reflection_coefficients);
  DCHECK(reflections);
  DCHECK_EQ(reflections->size(), kNumRoomSurfaces);
  const WorldPosition kOrigin(0.0f, 0.0f, 0.0f);
  if (!IsPositionInAabb(relative_listener_position, kOrigin, room_dimensions)) {
    // Listener is outside the room, skip computation.
    std::fill(reflections->begin(), reflections->end(), Reflection());
    return;
  }
  // Calculate the distance of the listener to each wall.
  // Since all the sources are 'attached' to the listener in the computation
  // of reflections, the distance travelled is arbitrary. So, we add 1.0f to
  // the computed distance in order to avoid delay time approaching 0 and the
  // magnitude approaching +inf.
  const WorldPosition offsets = 0.5f * room_dimensions;
  const float distances_travelled[kNumRoomSurfaces] = {
      offsets[0] + relative_listener_position[0] + 1.0f,
      offsets[0] - relative_listener_position[0] + 1.0f,
      offsets[1] + relative_listener_position[1] + 1.0f,
      offsets[1] - relative_listener_position[1] + 1.0f,
      offsets[2] + relative_listener_position[2] + 1.0f,
      offsets[2] - relative_listener_position[2] + 1.0f};

  for (size_t i = 0; i < kNumRoomSurfaces; ++i) {
    // Convert distances to time delays in seconds.
    (*reflections)[i].delay_time_seconds =
        distances_travelled[i] / kSpeedOfSound;
    // Division by distance is performed here as we don't want this applied more
    // than once.
    (*reflections)[i].magnitude =
        reflection_coefficients[i] / distances_travelled[i];
  }
}

}  // namespace vraudio
