/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_SHOE_BOX_ROOM_H_
#define RESONANCE_AUDIO_DSP_SHOE_BOX_ROOM_H_

#include <vector>

#include "base/misc_math.h"
#include "dsp/reflection.h"

namespace vraudio {

// Computes a set of reflections from each surface of a shoe-box room model.
// Uses a simplified calculation method which assumes that all the sources are
// 'attached' to the listener. Also, assumes that the listener is inside the
// shoe-box room.
//
// @param relative_listener_position Relative listener position to the center of
//     the room.
// @param room_dimensions Dimensions of the room.
// @param reflection_coefficients Reflection coefficients.
// @return List of computed reflections by the image source method.
void ComputeReflections(const WorldPosition& relative_listener_position,
                        const WorldPosition& room_dimensions,
                        const float* reflection_coefficients,
                        std::vector<Reflection>* reflections);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_SHOE_BOX_ROOM_H_
