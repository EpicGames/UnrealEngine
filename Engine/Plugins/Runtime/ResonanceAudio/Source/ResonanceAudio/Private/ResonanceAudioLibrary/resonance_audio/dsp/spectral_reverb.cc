/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "dsp/spectral_reverb.h"

#include <algorithm>
#include <cstdlib>
#include <numeric>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_utils.h"
#include "dsp/spectral_reverb_constants_and_tables.h"
#include "dsp/utils.h"

namespace vraudio {

namespace {

// FFT length and length of time domain data processed.
const size_t kFftSize = 4096;

// Length of magnitude and phase buffers.
const size_t kMagnitudeLength = kFftSize / 2 + 1;

// Number of overlaps per kFftSize time domain chunk of input.
const size_t kNumOverlap = 4;

// Number of samples per overlap.
const size_t kOverlapLength = kFftSize / kNumOverlap;

// Three quarters FFT size, used for copying chunks of input.
const size_t kThreeQuarterFftSize = kFftSize - kOverlapLength;

// Number of channels needed to overlap add output data.
const size_t kNumQuadChannels = 4;

// Number of buffers of delay applies to the magnitude spectrum.
const size_t kMagnitudeDelay = 3;

// Length of a buffer of noise used to provide random phase.
const size_t kNoiseLength = 16384;

// Length of the noise buffer from which a phase buffer can start.
const size_t kAvailableNoiselength = kNoiseLength - kMagnitudeLength;

// Returns a random integer in the range provided. Used to index into the random
// buffer for phase values.
inline size_t GetRandomIntegerInRange(size_t min, size_t max) {
  DCHECK_GE(max, min);
  return min + static_cast<size_t>(std::rand()) % (max - min);
}

}  // namespace

SpectralReverb::SpectralReverb(int sample_rate, size_t frames_per_buffer)
    : sample_rate_(sample_rate),
      frames_per_buffer_(frames_per_buffer),
      magnitude_delay_index_(0),
      overlap_add_index_(0),
      fft_manager_(kFftSize / 2),
      sin_cos_random_phase_buffer_(kNumStereoChannels, kNoiseLength),
      unscaled_window_(kNumMonoChannels, kFftSize),
      window_(kNumMonoChannels, kFftSize),
      feedback_(kNumMonoChannels, kMagnitudeLength),
      magnitude_compensation_(kNumMonoChannels, kMagnitudeLength),
      magnitude_delay_(kMagnitudeDelay, kMagnitudeLength),
      fft_size_input_(kNumMonoChannels, kFftSize),
      input_circular_buffer_(kFftSize + frames_per_buffer_, frames_per_buffer_,
                             kOverlapLength),
      output_circular_buffers_(kNumStereoChannels),
      out_time_buffer_(kNumQuadChannels, kFftSize),
      temp_freq_buffer_(kNumStereoChannels, kFftSize),
      scaled_magnitude_buffer_(kNumMonoChannels, kMagnitudeLength),
      temp_magnitude_buffer_(kNumMonoChannels, kMagnitudeLength),
      temp_phase_buffer_(kNumStereoChannels, kMagnitudeLength),
      output_accumulator_(kNumStereoChannels),
      is_gain_near_zero_(false),
      is_feedback_near_zero_(false) {
  DCHECK_GT(sample_rate, 0);
  DCHECK_GT(frames_per_buffer_, 0U);

  GenerateRandomPhaseBuffer();
  GenerateAnalysisWindow();
  InitializeCircularBuffersAndAccumulators();
  fft_size_input_.Clear();
  magnitude_compensation_.Clear();
}

void SpectralReverb::SetGain(float gain) {
  DCHECK_GE(gain, 0.0f);
  ScalarMultiply(window_.num_frames(), gain, &unscaled_window_[0][0],
                 &window_[0][0]);
  // If the gain is less than -60dB we will bypass all processing.
  is_gain_near_zero_ = gain <= kNegative60dbInAmplitude;
  // If we are to bypass processing we clear the circular buffer so that we
  // don't process old input when the spectral reverb is restarted.
  if (is_gain_near_zero_ || is_feedback_near_zero_) {
    input_circular_buffer_.Clear();
  }
}

void SpectralReverb::SetRt60PerOctaveBand(const float* rt60_values) {
  DCHECK(rt60_values);
  const float sample_rate_float = static_cast<float>(sample_rate_);
  // Fill the entries in the feedback channel with the feedback values up to
  // that frequency. The feedback channels entries are per frequency in the same
  // way as the magnitude vectors. Also fill the magnitude compensation vector
  // such that the peak value at each frequency for any given reverberation time
  // will be |kDefaultReverbGain|.
  AudioBuffer::Channel* feedback_channel = &feedback_[0];
  feedback_channel->Clear();
  AudioBuffer::Channel* magnitude_compensation_channel =
      &magnitude_compensation_[0];
  magnitude_compensation_channel->Clear();
  const float frequency_step = sample_rate_float / static_cast<float>(kFftSize);
  int index = GetFeedbackIndexFromRt60(rt60_values[0], sample_rate_float);
  float current_feedback =
      index == kInvalidIndex ? 0.0f : kSpectralReverbFeedback[index];
  float magnitude_compensation_value =
      index == kInvalidIndex ? 0.0f : kMagnitudeCompensation[index];
  const size_t max_frequency_bin =
      std::min(static_cast<size_t>(
                   (kOctaveBandCentres[kNumReverbOctaveBands - 1] * kSqrtTwo) /
                   frequency_step),
               feedback_channel->size());
  // The upper edge of the octave band is sqrt(2) * centre_frequency :
  // https://en.wikipedia.org/wiki/Octave_band#Octave_Bands
  float upper_octave_band_edge = kOctaveBandCentres[0] * kSqrtTwo;
  for (size_t i = 0, j = 0; i < max_frequency_bin; ++i) {
    const float current_freq = static_cast<float>(i) * frequency_step;
    if (current_freq > upper_octave_band_edge) {
      ++j;
      DCHECK_LT(j, kNumReverbOctaveBands);
      upper_octave_band_edge = kOctaveBandCentres[j] * kSqrtTwo;
      index = GetFeedbackIndexFromRt60(rt60_values[j], sample_rate_float);
      current_feedback =
          index == kInvalidIndex ? 0.0f : kSpectralReverbFeedback[index];
      magnitude_compensation_value =
          index == kInvalidIndex ? 0.0f : kMagnitudeCompensation[index];
    }
    (*feedback_channel)[i] = current_feedback;
    (*magnitude_compensation_channel)[i] = magnitude_compensation_value;
  }
  // If the sum of all feedback values is below the minimum feedback value, it
  // is safe to assume we can bypass the spectral reverb entirely.
  is_feedback_near_zero_ =
      std::accumulate(feedback_channel->begin(), feedback_channel->end(),
                      0.0f) < kSpectralReverbFeedback[0];
  // If we are to bypass processing we clear the circular buffer so that we
  // don't process old input when the spectral reverb is restarted.
  if (is_gain_near_zero_ || is_feedback_near_zero_) {
    input_circular_buffer_.Clear();
  }
}

void SpectralReverb::Process(const AudioBuffer::Channel& input,
                             AudioBuffer::Channel* left_out,
                             AudioBuffer::Channel* right_out) {
  DCHECK_EQ(input.size(), left_out->size());
  DCHECK_EQ(input.size(), right_out->size());
  DCHECK_EQ(input.size(), frames_per_buffer_);


  if (is_gain_near_zero_ || is_feedback_near_zero_) {
    left_out->Clear();
    right_out->Clear();
    return;
  }

  // Here we insert |frames_per_buffer_| samples on each function call. Then,
  // once there are |kOverlapLength| samples in the input circular buffer we
  // retrieve |kOverlapLength| samples from it and process |kFftSize| samples of
  // input at a time, sliding along by |kOverlapLength| samples. We then place
  // |kOverlapLength| samples into the output buffer and we will extract
  // |frames_per_buffer_| samples from the output buffer on each function call.
  input_circular_buffer_.InsertBuffer(input);
  while (input_circular_buffer_.GetOccupancy() >= kOverlapLength) {
    std::copy_n(&fft_size_input_[0][kOverlapLength], kThreeQuarterFftSize,
                &fft_size_input_[0][0]);
    input_circular_buffer_.RetrieveBufferWithOffset(kThreeQuarterFftSize,
                                                    &fft_size_input_[0]);
    fft_manager_.FreqFromTimeDomain(fft_size_input_[0], &temp_freq_buffer_[0]);
    fft_manager_.GetCanonicalFormatFreqBuffer(temp_freq_buffer_[0],
                                              &temp_freq_buffer_[1]);
    fft_manager_.MagnitudeFromCanonicalFreqBuffer(temp_freq_buffer_[1],
                                                  &scaled_magnitude_buffer_[0]);
    // Apply the magnitude compensation to the input magnitude spectrum before
    // feedback is applied.
    MultiplyPointwise(kMagnitudeLength, magnitude_compensation_[0].begin(),
                      scaled_magnitude_buffer_[0].begin(),
                      scaled_magnitude_buffer_[0].begin());
    // Generate time domain reverb blocks.
    GetNextReverbBlock(magnitude_delay_index_, &out_time_buffer_[0],
                       &out_time_buffer_[1]);
    magnitude_delay_index_ = (magnitude_delay_index_ + 1) % kMagnitudeDelay;
    GetNextReverbBlock(magnitude_delay_index_, &out_time_buffer_[2],
                       &out_time_buffer_[3]);

    // Combine the reverb blocks for both left and right output.
    AddPointwise(kFftSize, out_time_buffer_[0].begin(),
                 out_time_buffer_[2].begin(), out_time_buffer_[0].begin());
    AddPointwise(kFftSize, out_time_buffer_[1].begin(),
                 out_time_buffer_[3].begin(), out_time_buffer_[1].begin());

    // Window the left and right output (While applying inverse FFT scaling).
    MultiplyPointwise(kFftSize, out_time_buffer_[0].begin(), window_[0].begin(),
                      out_time_buffer_[0].begin());
    MultiplyPointwise(kFftSize, out_time_buffer_[1].begin(), window_[0].begin(),
                      out_time_buffer_[1].begin());

    // Next perform the addition and the submission into the output buffer.
    AccumulateOverlap(0 /*channel*/, out_time_buffer_[0]);
    AccumulateOverlap(1 /*channel*/, out_time_buffer_[1]);
    overlap_add_index_ = (overlap_add_index_ + 1) % kNumOverlap;
  }
  output_circular_buffers_[0]->RetrieveBuffer(left_out);
  output_circular_buffers_[1]->RetrieveBuffer(right_out);
}

void SpectralReverb::AccumulateOverlap(size_t channel_index,
                                       const AudioBuffer::Channel& buffer) {
  // Use a modulo indexed multi channel audio buffer with each channel of length
  // |kOverlapLength| to perform an overlap add.
  for (size_t i = 0, index = overlap_add_index_; i < kNumOverlap;
       ++i, index = (index + 1) % kNumOverlap) {
    float* accumulator_start_point =
        output_accumulator_[channel_index][index].begin();
    AddPointwise(kOverlapLength, buffer.begin() + i * kOverlapLength,
                 accumulator_start_point, accumulator_start_point);
  }
  output_circular_buffers_[channel_index]->InsertBuffer(
      output_accumulator_[channel_index][overlap_add_index_]);
  output_accumulator_[channel_index][overlap_add_index_].Clear();
}

void SpectralReverb::GenerateAnalysisWindow() {
  // Genarate a pseudo tukey window from three overlapping hann windows, scaled
  // by the inverse fft scale.
  AudioBuffer::Channel* window_channel = &window_[0];
  // Use the unscaled window buffer as temporary storage.
  GenerateHannWindow(true /* full */, kMagnitudeLength, &unscaled_window_[0]);
  float* hann_window = &unscaled_window_[0][0];
  // Scale the hann window such that the sum of three will have unity peak.
  const float kThreeQuarters = 0.75f;
  ScalarMultiply(kMagnitudeLength, kThreeQuarters, hann_window, hann_window);
  for (size_t offset = 0; offset < kThreeQuarterFftSize;
       offset += kOverlapLength) {
    float* tripple_hann_window = &(*window_channel)[offset];
    AddPointwise(kMagnitudeLength, hann_window, tripple_hann_window,
                 tripple_hann_window);
  }
  fft_manager_.ApplyReverseFftScaling(window_channel);
  unscaled_window_[0] = *window_channel;
}

void SpectralReverb::GenerateRandomPhaseBuffer() {
  AudioBuffer::Channel* sin_phase_channel = &sin_cos_random_phase_buffer_[0];
  AudioBuffer::Channel* cos_phase_channel = &sin_cos_random_phase_buffer_[1];
  // Initially use the sin channel to store the random data before taking sine
  // and cosine.
  GenerateUniformNoise(/*min=*/0.0f, /*max=*/kPi, /*seed=*/1U,
                       sin_phase_channel);
  for (size_t i = 0; i < sin_cos_random_phase_buffer_.num_frames(); ++i) {

    (*cos_phase_channel)[i] = std::cos((*sin_phase_channel)[i]);
    (*sin_phase_channel)[i] = std::sin((*sin_phase_channel)[i]);
  }
}

void SpectralReverb::GetNextReverbBlock(size_t delay_index,
                                        AudioBuffer::Channel* left_channel,
                                        AudioBuffer::Channel* right_channel) {
  DCHECK(left_channel);
  DCHECK(right_channel);

  // Generate reverb magnitude by combining the delayed magnitude with the
  // current magnitude.
  AudioBuffer::Channel* temp_magnitude_channel = &temp_magnitude_buffer_[0];
  *temp_magnitude_channel = scaled_magnitude_buffer_[0];
  MultiplyAndAccumulatePointwise(
      kMagnitudeLength, magnitude_delay_[delay_index].begin(),
      feedback_[0].begin(), temp_magnitude_channel->begin());
  // Reinsert this new reverb magnitude into the delay buffer.
  magnitude_delay_[delay_index] = *temp_magnitude_channel;

  for (size_t i = 0; i < kNumStereoChannels; ++i) {
    // Extract a random phase buffer.
    const size_t random_offset =
        GetRandomIntegerInRange(0, kAvailableNoiselength);
    // We gaurantee an aligned offset as when SSE is used we need it.
    const size_t phase_offset = FindNextAlignedArrayIndex(
        random_offset, sizeof(float), kMemoryAlignmentBytes);
    // Convert from magnitude and phase to a time domain output.
    fft_manager_.CanonicalFreqBufferFromMagnitudeAndSinCosPhase(
        phase_offset, (*temp_magnitude_channel),
        sin_cos_random_phase_buffer_[0], sin_cos_random_phase_buffer_[1],
        &temp_freq_buffer_[0]);

    AudioBuffer::Channel* out_channel = i == 0 ? left_channel : right_channel;
    fft_manager_.GetPffftFormatFreqBuffer(temp_freq_buffer_[0],
                                          &temp_freq_buffer_[1]);
    fft_manager_.TimeFromFreqDomain(temp_freq_buffer_[1], out_channel);
  }
}

void SpectralReverb::InitializeCircularBuffersAndAccumulators() {
  AudioBuffer zeros(kNumMonoChannels, kOverlapLength);
  zeros.Clear();
  for (size_t channel = 0; channel < kNumStereoChannels; ++channel) {
    // Prefill the |output_circular_buffers_| with |kOverlapLength| /
    // |frames_per_buffer_| calls worth of zeros.
    output_circular_buffers_[channel].reset(
        new CircularBuffer(kOverlapLength + frames_per_buffer_, kOverlapLength,
                           frames_per_buffer_));
    // Due to differences in the |frames_per_buffer_| used for input and output
    // and |kOverlapLength| used for processing, a certain number of buffers of
    // zeros must be inserted into the output buffers such that enough input can
    // build up to process |kOverlapLength| worth, and enough output will build
    // up to return |frames_per_buffer_| worth.
    const size_t zeroed_buffers_of_output = kOverlapLength / frames_per_buffer_;
    for (size_t i = 0; i < zeroed_buffers_of_output; ++i) {
      output_circular_buffers_[channel]->InsertBuffer(zeros[0]);
    }
    output_accumulator_[channel] = AudioBuffer(kNumOverlap, kOverlapLength);
    output_accumulator_[channel].Clear();
  }
}

}  // namespace vraudio
