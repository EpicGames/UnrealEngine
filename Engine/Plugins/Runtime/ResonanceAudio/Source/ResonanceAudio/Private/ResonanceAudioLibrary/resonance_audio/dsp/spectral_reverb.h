/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_H_
#define RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"
#include "dsp/circular_buffer.h"
#include "dsp/fft_manager.h"

namespace vraudio {

// Implements a spectral reverb producing a decorrelated stereo output. See:
// [1] E. Vickers, J-L Wu, P.G. Krishnan, R. N. K. Sadanandam, "Frequency Domain
//     Artificial Reverberation using Spectral Magnitude Decay",
//     https://goo.gl/hv1pdJ.
class SpectralReverb {
 public:
  // Constructs a spectral reverb.
  //
  // @param sample_rate The system sample rate.
  // @param frames_per_buffer System frames per buffer of input and output.
  //     Note that this class expects power of two buffers of input and output.
  SpectralReverb(int sample_rate, size_t frames_per_buffer);

  // Sets the overall gain to be applied to the output of the reverb.
  //
  // @param gain Gain to be applied to the reverb output, min value 0.0f.
  void SetGain(float gain);

  // Sets the |SpectralReverb|'s reverberation times in different frequency
  // bands. Supports times between:
  // (0.15 * 48000 / |sample_rate|)s and (25 * 48000 / |sample_rate|)s.
  //
  // @param rt60_values |kNumReverbOctaveBands| values denoting the
  //     reverberation decay time to -60dB in octave bands starting at
  //     |kLowestOctaveBand|.
  void SetRt60PerOctaveBand(const float* rt60_values);

  // Applies reverb to an input channel of audio data and produces a stereo
  // output.
  //
  // @param input Mono inpu data.
  // @param left_out Left channel of reverberated output.
  // @param right_out Right channel of reverberated output.
  void Process(const AudioBuffer::Channel& input,
               AudioBuffer::Channel* left_out, AudioBuffer::Channel* right_out);

 private:
  // Uses an AudioBuffer with four channels to overlap add and insert the final
  // reverb into the output circular buffers.
  //
  // @param channel_index Denotes the (left or right) channel to output to.
  // @param buffer The buffer to be added onto the pre-existing reverb output.
  void AccumulateOverlap(size_t channel_index,
                         const AudioBuffer::Channel& buffer);

  // Generates a window function which is a normalized sum of three overlapping
  // (50%) hann windows of length (|kFftSize| / 2) that also incorporates the
  // inverse fft scaling.
  void GenerateAnalysisWindow();

  // Generates a large buffer of sines and cosines of random noise between 0 and
  // pi to be randomly indexed into in order to cheaply generate highly
  // decorrelated phase buffers,
  void GenerateRandomPhaseBuffer();

  // Obtains the next stero pair of time domain reverb blocks which can then be
  // summed together in an overlap add fashion to provide the reverb output.
  //
  // @param delay_index An index into the frequency domain magnitude ring
  //     buffer.
  // @param left_channel Channel to contain the left partial reverb output.
  // @param right_channel Channel to contain the right partial reverb output.
  void GetNextReverbBlock(size_t delay_index,
                          AudioBuffer::Channel* left_channel,
                          AudioBuffer::Channel* right_channel);

  // Initializes the output circular buffers such that they contain zeros if the
  // value of |frames_per_buffer_| is sufficiently smaller than that of
  // |kOverlapLength| that buffering of input will be required prior to
  // processing. Also allocates memory for the output accumulators.
  void InitializeCircularBuffersAndAccumulators();

  // System sample rate.
  const int sample_rate_;

  // System frames per buffer.
  const size_t frames_per_buffer_;

  // Indices into the magnitude and overlap add delay lines, modulo of their
  // respective lengths.
  size_t magnitude_delay_index_;
  size_t overlap_add_index_;

  // Manages the time-frequency transforms and phase/magnitude-frequency
  // transforms.
  FftManager fft_manager_;

  // Buffer containing sines and cosines of random values between 0 and pi to be
  // used for phase.
  AudioBuffer sin_cos_random_phase_buffer_;

  // Buffer containing a triple overlapping hann window for windowing time
  // domain data.
  AudioBuffer unscaled_window_;

  // Buffer containing a triple overlapping hann window for windowing time
  // domain data, this window has been scaled by the output gain factor.
  AudioBuffer window_;

  // Buffer containing RT60 tuned feedback values.
  AudioBuffer feedback_;

  // Buffer used to store scaling values which account for the different initial
  // peak magnitudes for different RT60s.
  AudioBuffer magnitude_compensation_;

  // Buffer that acts as the frequency domain magnitde delay.
  AudioBuffer magnitude_delay_;

  // Buffer to contain a linear |kFftSize| chunk of input data.
  AudioBuffer fft_size_input_;

  // Circular buffers to sit at the input and output of the |Process()| method
  // to allow |frames_per_buffer_| to differ from |kFftSize|.
  CircularBuffer input_circular_buffer_;
  std::vector<std::unique_ptr<CircularBuffer>> output_circular_buffers_;

  // Time domain buffer used to store reverb before the overlap add operation.
  AudioBuffer out_time_buffer_;

  // Temporary frequency domain buffer, used to store frequency domain data when
  // transforming between Pffft and Canonical format frequency domain data.
  AudioBuffer temp_freq_buffer_;

  // Buffer used to store feedback scaled magnitude values.
  AudioBuffer scaled_magnitude_buffer_;

  // Buffer used for the accumulation of scaled magnitude buffers.
  AudioBuffer temp_magnitude_buffer_;

  // Buffer used to store randomized phase.
  AudioBuffer temp_phase_buffer_;

  // Buffers used to calculate the overlap add at the output.
  std::vector<AudioBuffer> output_accumulator_;

  // Processing of the spectral reverb is bypassed when the feedback values are
  // all approximately zero OR when the gain is set to near zero.
  bool is_gain_near_zero_;
  bool is_feedback_near_zero_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_H_
