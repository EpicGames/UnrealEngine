/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_CONSTANTS_AND_TABLES_H_
#define RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_CONSTANTS_AND_TABLES_H_

#include <algorithm>
#include <cstddef>

namespace vraudio {

// Sampling rate for which the spectral reverb feedback coefficients are
// calculated.
static const float kDefaultSpectralReverbSampleRate = 48000.0f;

// Minimum reverb time supported by the feedback table below (seconds).
static const float kMinReverbTimeForFeedback48kHz = 0.15f;

// Step size between each reverberation time supported by the feedback table
// (seconds).
static const float kTimeStepSizeForFeedback48kHz = 0.01f;

// Number of entries in the feedback table below.
static const size_t kNumFeedbackValues = 2485;

// Value denoting an invalid index into the feedback and magnitude correction
// tables. The feedback and magnitude correction factors will be 0.0 when this
// value is returned from the index lookup method.
const int kInvalidIndex = -1;

// Number of frames of zeros before the reverb onset curve.
const size_t kCompensationOnsetLength = 1024;

// Length in samples of the spectral reverb onset correction curves.
const size_t kCorrectionCurveLength = 6144;

// Index of the feedback factor and curve multipliers. below which the low
// frequency correction curves must be used.
const size_t kCurveChangeoverIndex = 51;

// Offset and scaling values for the compensation curve polynomials. Used to
// ensure the polynomial fitting was well conditioned.
const float kCurveOffset = 3584.5f;
const float kCurveScale = 0.00027897893709025f;

// Number of coefficients in the curve generation polynomials.
const size_t kCurvePolynomialLength = 5;

// Coefficients of the curves making up the first part of the compensation
// envelopes, split into low and high frequency part.
const float kHighReverberationCorrectionCurve[kCurvePolynomialLength] = {
    0.207891278205479f, -0.32101050261694f, 0.124608132159297f,
    0.0119620847734548f, -0.0093840571415877f};

const float kLowReverberationCorrectionCurve[kCurvePolynomialLength] = {
    0.0277040197264401f, -0.0394421854392145f, 0.0519110103856154f,
    -0.0778183422829366f, 0.0343221444553963f};

// Offset from zero for indexing into the feedback array.
const size_t kReverbLookupOffset = static_cast<size_t>(
    kMinReverbTimeForFeedback48kHz / kTimeStepSizeForFeedback48kHz);

// Returns an index to the feedback factor corresponding to the given
// reverberation time. Supports times between:
// (0.15 * 48000 / |sample_rate|)s and (25 * 48000 / |sample_rate|)s.
//
// @param reverberation_time RT60 in seconds.
// @return Index to the closest spectral reverb feedback factor in
//     |kSpectralReverbFeedback|, or |kInvalidIndex| if |reverberation_time|
//     is too small.
static inline int GetFeedbackIndexFromRt60(float reverberation_time,
                                           float sample_rate) {
  // Scaling factor on reverb times applied due to current |sample_rate|. All
  // feedback factors calculated correspond to 48kHz.
  const float scale_factor = sample_rate / kDefaultSpectralReverbSampleRate;
  if (reverberation_time * scale_factor < kMinReverbTimeForFeedback48kHz) {
    return kInvalidIndex;
  }
  return static_cast<int>(
      std::min(kNumFeedbackValues - 1,
               static_cast<size_t>((reverberation_time * scale_factor) /
                                   kTimeStepSizeForFeedback48kHz) -
                   kReverbLookupOffset));
}

// Mapping between reverberation times at a given frequency and the
// corresponding feedback value required in the specral reverb. This table
// covers reverberation times between kMinReverbTimeForFeedback and
// kNumFeedbackValues * kTimeStepSizeForFeedback, i.e. 0.15s and 25s @48kHz.
static const float kSpectralReverbFeedback[kNumFeedbackValues] = {
    0.231626448720861f, 0.231626448720861f, 0.271965742827169f,
    0.291599040391330f, 0.313450848869688f, 0.332864905289138f,
    0.350285532738638f, 0.366049684043223f, 0.383143567924904f,
    0.398574170268545f, 0.427500186399158f, 0.441038795823222f,
    0.453430889363735f, 0.466397294153677f, 0.478272702354267f,
    0.490519904155818f, 0.501757394995813f, 0.512125955040727f,
    0.521739757418252f, 0.548135448526531f, 0.548135448526531f,
    0.563990087718933f, 0.571497329505391f, 0.578784680803617f,
    0.585892965998708f, 0.592858971592466f, 0.599714945736390f,
    0.606488097764666f, 0.613200097727167f, 0.619866575922435f,
    0.626496622430680f, 0.633092286646765f, 0.639648076813205f,
    0.646150459553150f, 0.652577359403386f, 0.658897658347319f,
    0.665070695347976f, 0.671045765880981f, 0.676761621467567f,
    0.682145969207546f, 0.682145969207546f, 0.687114971312322f,
    0.691572744637866f, 0.695410860217716f, 0.700728670360266f,
    0.701931035813262f, 0.700570291691920f, 0.718477641957808f,
    0.718477641957808f, 0.721948612036913f, 0.725341703212013f,
    0.728658944837357f, 0.731902322614612f, 0.735073779107144f,
    0.738175214254303f, 0.741208485885712f, 0.744175410235544f,
    0.747077762456819f, 0.749917277135678f, 0.752695648805674f,
    0.755414532462056f, 0.758075544076055f, 0.760680261109167f,
    0.763230223027440f, 0.765726931815758f, 0.768171852492127f,
    0.770566413621959f, 0.772912007832361f, 0.775209992326414f,
    0.777461689397460f, 0.779668386943394f, 0.781831338980938f,
    0.783951766159935f, 0.786030856277629f, 0.788069764792952f,
    0.790069615340812f, 0.792031500246373f, 0.793956481039341f,
    0.795845588968255f, 0.797699825514762f, 0.799520162907914f,
    0.801307544638442f, 0.803062885973050f, 0.804787074468694f,
    0.806480970486870f, 0.808145407707900f, 0.809781193645214f,
    0.811389110159637f, 0.812969913973676f, 0.814524337185802f,
    0.816053087784737f, 0.817556850163737f, 0.819036285634880f,
    0.820492032943351f, 0.821924708781723f, 0.823334908304248f,
    0.824723205641138f, 0.826090154412852f, 0.827436288244380f,
    0.828762121279528f, 0.830068148695207f, 0.831354847215712f,
    0.833872075291032f, 0.835103470659940f, 0.836317269790432f,
    0.837513864858018f, 0.838693632671302f, 0.841004120020598f,
    0.841004120020598f, 0.842135520967878f, 0.843251458511968f,
    0.844352240341239f, 0.846509506717154f, 0.847566547291731f,
    0.848609545235926f, 0.849638751975006f, 0.850654409224479f,
    0.851656749504368f, 0.852645996653501f, 0.853622366343797f,
    0.854586066594550f, 0.855537298286716f, 0.856476255677194f,
    0.857403126913115f, 0.858318094546124f, 0.859221336046671f,
    0.860113024318288f, 0.860993328211881f, 0.861862413040010f,
    0.862720441091179f, 0.863567572144118f, 0.864403963982066f,
    0.865229772907064f, 0.866045154254231f, 0.866850262906056f,
    0.867645253806680f, 0.868430282476178f, 0.869205505524854f,
    0.869971081167516f, 0.870727169737766f, 0.871473934202282f,
    0.872211540675109f, 0.872940158931938f, 0.873659962924396f,
    0.874371131294327f, 0.875073847888079f, 0.876454690240667f,
    0.877133214343288f, 0.877804084385866f, 0.878467517951544f,
    0.879123740913686f, 0.879772987950149f, 0.880415503057582f,
    0.881051540065697f, 0.881681363151570f, 0.882305247353907f,
    0.882923479087349f, 0.883536356656744f, 0.884144190771436f,
    0.884747305059551f, 0.885346036582283f, 0.885940736348172f,
    0.886531769827401f, 0.887119517466070f, 0.887704375200491f,
    0.888286754971462f, 0.888867085238566f, 0.889355868225441f,
    0.889898852584961f, 0.890436987253855f, 0.890970321870216f,
    0.891498905661491f, 0.892022787446309f, 0.892542015636295f,
    0.893056638237894f, 0.893566702854191f, 0.894072256686731f,
    0.894573346537340f, 0.895070018809945f, 0.895562319512397f,
    0.896533988268773f, 0.897013446374392f, 0.897488713016890f,
    0.897959832251035f, 0.898426847746442f, 0.898889802789393f,
    0.899348740284655f, 0.899803702757303f, 0.900254732354541f,
    0.900701870847520f, 0.901145159633161f, 0.901584639735976f,
    0.902020351809884f, 0.902452336140039f, 0.902880632644642f,
    0.903305280876771f, 0.903726320026192f, 0.904143788921187f,
    0.904557726030372f, 0.904968169464516f, 0.905375156978364f,
    0.905778725972456f, 0.906178913494948f, 0.906575756243433f,
    0.906969290566763f, 0.907359552466865f, 0.907746577600566f,
    0.908130401281413f, 0.908511058481491f, 0.908888583833246f,
    0.909263011631306f, 0.909634375834297f, 0.910002710066671f,
    0.910368047620521f, 0.910730421457403f, 0.911446408184725f,
    0.911800085361980f, 0.912150927399534f, 0.912498965633569f,
    0.912844231080653f, 0.913186754439560f, 0.913526566093093f,
    0.913863696109903f, 0.914198174246311f, 0.914530029948127f,
    0.914859292352470f, 0.915185990289593f, 0.915510152284696f,
    0.915831806559755f, 0.916150981035335f, 0.916467703332418f,
    0.916782000774218f, 0.917093900388001f, 0.917403428906911f,
    0.917710612771787f, 0.918015478132983f, 0.918318050852190f,
    0.918916420379010f, 0.919212267483073f, 0.919505922541691f,
    0.919797410000546f, 0.920086754027583f, 0.920373978514824f,
    0.920659107080196f, 0.920942163069346f, 0.921223169557465f,
    0.921502149351104f, 0.921779124990001f, 0.922054118748897f,
    0.922327152639358f, 0.922598248411594f, 0.922867427556284f,
    0.923134711306391f, 0.923400120638986f, 0.923663676277069f,
    0.923925398691385f, 0.924185308102252f, 0.924443424481376f,
    0.924699767553671f, 0.924954356799086f, 0.925207211454416f,
    0.925458350515133f, 0.925707792737198f, 0.925955556638886f,
    0.926201660502607f, 0.926446122376723f, 0.926688960077372f,
    0.926930191190288f, 0.927169833072619f, 0.927407902854752f,
    0.927644417442129f, 0.927879393517071f, 0.928112847540597f,
    0.928344795754244f, 0.928575254181890f, 0.928804238631573f,
    0.929031764697310f, 0.929257847760921f, 0.929705745358969f,
    0.929927589612438f, 0.930148050305480f, 0.930367141786232f,
    0.930584878201552f, 0.930801273498844f, 0.931016341427878f,
    0.931230095542612f, 0.931442549203009f, 0.931653715576860f,
    0.931863607641606f, 0.932072238186154f, 0.932279619812703f,
    0.932485764938561f, 0.932690685797968f, 0.932894394443911f,
    0.933096902749953f, 0.933298222412048f, 0.933498364950364f,
    0.933697341711099f, 0.933895163868310f, 0.934091842425725f,
    0.934287388218568f, 0.934481811915380f, 0.934675124019838f,
    0.934867334872576f, 0.935058454653005f, 0.935437460919395f,
    0.935625366974452f, 0.935812221099035f, 0.935998032693751f,
    0.936182811008910f, 0.936182811008910f, 0.936366565146343f,
    0.936731036563891f, 0.936911771321662f, 0.937091516860660f,
    0.937270281567636f, 0.937448073691783f, 0.937624901346560f,
    0.937800772511514f, 0.937975695034098f, 0.938149676631492f,
    0.938322724892425f, 0.938494847278995f, 0.938666051128488f,
    0.938836343655203f, 0.939005731952265f, 0.939174222993452f,
    0.939341823635016f, 0.939508540617498f, 0.939674380567555f,
    0.939839349999772f, 0.940003455318495f, 0.940166702819640f,
    0.940329098692519f, 0.940490649021660f, 0.940651359788628f,
    0.940811236873844f, 0.940970286058407f, 0.941128513025914f,
    0.941285923364282f, 0.941442522567564f, 0.941598316037775f,
    0.941753309086713f, 0.941907506937771f, 0.942060914727768f,
    0.942213537508766f, 0.942365380249885f, 0.942516447839133f,
    0.942666745085221f, 0.942816276719382f, 0.942965047397198f,
    0.943113061700413f, 0.943260324138759f, 0.943406839151775f,
    0.943552611110629f, 0.943697644319934f, 0.943841943019572f,
    0.944128353536651f, 0.944270473526585f, 0.944411875355484f,
    0.944552562966882f, 0.944692540250507f, 0.944831811044098f,
    0.944970379135229f, 0.945108248263126f, 0.945245422120492f,
    0.945381904355324f, 0.945517698572732f, 0.945652808336767f,
    0.945787237172233f, 0.945920988566514f, 0.946054065971390f,
    0.946186472804861f, 0.946318212452965f, 0.946449288271601f,
    0.946579703588348f, 0.946709461704285f, 0.946838565895813f,
    0.946967019416476f, 0.947094825498780f, 0.947221987356014f,
    0.947348508184070f, 0.947474391163268f, 0.947599639460168f,
    0.947724256229401f, 0.947848244615479f, 0.947971607754624f,
    0.948016355753743f, 0.948131749028067f, 0.948256528604491f,
    0.948381070023128f, 0.948495820690419f, 0.948619906515573f,
    0.948734238085242f, 0.948857871450992f, 0.948971786801436f,
    0.949094970810335f, 0.949208472791043f, 0.949321777358027f,
    0.949444301946475f, 0.949557197308729f, 0.949669897241678f,
    0.949782402382633f, 0.949894713365863f, 0.950016165228152f,
    0.950128073740068f, 0.950239790030961f, 0.950351314722867f,
    0.950462648434873f, 0.950573791783135f, 0.950684745380899f,
    0.950786286676309f, 0.950896878289179f, 0.951007281923720f,
    0.951117498181945f, 0.951227527663044f, 0.951328224451603f,
    0.951437897607799f, 0.951547385718536f, 0.951647587765244f,
    0.951756722847544f, 0.952281634952469f, 0.952380709437317f,
    0.952479632641358f, 0.952587376840327f, 0.952685985134598f,
    0.952784443487780f, 0.952891682098038f, 0.952989828324488f,
    0.953087825932233f, 0.953185675345440f, 0.953283376986503f,
    0.953380931276052f, 0.953478338632963f, 0.953584434118178f,
    0.953681535598181f, 0.953778491429264f, 0.953875302023227f,
    0.953963185967004f, 0.954059720427442f, 0.954156110839052f,
    0.954252357607017f, 0.954348461134851f, 0.954444421824404f,
    0.954627224309327f, 0.954627224309327f, 0.954722771738916f,
    0.954818177885333f, 0.954999925992284f, 0.955086293022387f,
    0.955181163332660f, 0.955267288091851f, 0.955361892609296f,
    0.955447776377266f, 0.955542116504091f, 0.955627760550641f,
    0.955721837678226f, 0.955807243263361f, 0.955892535735487f,
    0.955986227146404f, 0.956071282999438f, 0.956156226611782f,
    0.956241058262726f, 0.956334244095430f, 0.956418841531991f,
    0.956503327866496f, 0.956587703374087f, 0.956671968328910f,
    0.956756123004115f, 0.956840167671866f, 0.956924102603339f,
    0.957007928068733f, 0.957091644337270f, 0.957175251677203f,
    0.957258750355817f, 0.957342140639439f, 0.957425422793436f,
    0.957508597082225f, 0.957591663769275f, 0.957666332005129f,
    0.957749194971355f, 0.957831951094582f, 0.957914600634626f,
    0.957988894306034f, 0.958071342050505f, 0.958153683960035f,
    0.958227701401528f, 0.958309842929826f, 0.958383680442363f,
    0.958465622508191f, 0.958539280915875f, 0.958621024431827f,
    0.958694504553236f, 0.958776050425793f, 0.958849353074026f,
    0.958930702203609f, 0.959003828186335f, 0.959076870765751f,
    0.959157931586863f, 0.959230798681224f, 0.959303582928693f,
    0.959384357364767f, 0.959456967296096f, 0.959529494931521f,
    0.959601940447006f, 0.959674304017958f, 0.959746585819239f,
    0.959826803241069f, 0.959898912989265f, 0.959970941508314f,
    0.960042888970841f, 0.960114755548935f, 0.960186541414150f,
    0.960258246737504f, 0.960329871689484f, 0.960401416440053f,
    0.960472881158639f, 0.960544266014151f, 0.960607652309644f,
    0.960678886771683f, 0.960750041855675f, 0.960821117728505f,
    0.960892114556548f, 0.960955156623649f, 0.961026004596989f,
    0.961096774003302f, 0.961167465006980f, 0.961230235767696f,
    0.961300779124251f, 0.961433815192630f, 0.961433815192630f,
    0.961504133859284f, 0.961566574305389f, 0.961636746788933f,
    0.961699057549504f, 0.961769084422862f, 0.961901147838506f,
    0.961963200748498f, 0.961963200748498f, 0.962032938107125f,
    0.962094862846269f, 0.962164456293847f, 0.962226253361271f,
    0.962287990519784f, 0.962357373349702f, 0.962418983565341f,
    0.962480534213617f, 0.962611132054796f, 0.962672497254291f,
    0.962733803332908f, 0.962795050398145f, 0.962863882940590f,
    0.962925004958022f, 0.962986068296167f, 0.963047073061348f,
    0.963108019359607f, 0.963168907296701f, 0.963229736978111f,
    0.963290508509035f, 0.963351221994394f, 0.963411877538831f,
    0.963472475246711f, 0.963533015222125f, 0.963593497568891f,
    0.963653922390548f, 0.963714289790367f, 0.963774599871346f,
    0.963834852736211f, 0.963895048487418f, 0.963947672999015f,
    0.964007761937330f, 0.964067794055055f, 0.964127769453608f,
    0.964187688234142f, 0.964240070801961f, 0.964299883695432f,
    0.964359640260232f, 0.964471532344270f, 0.964531127521006f,
    0.964583227407251f, 0.964642717773864f, 0.964642717773864f,
    0.964702152384023f, 0.964813442340238f, 0.964865310872176f,
    0.964924537272103f, 0.964976315013494f, 0.965035437824303f,
    0.965035437824303f, 0.965146144638393f, 0.965197741696581f,
    0.965256658352829f, 0.965308165515470f, 0.965366979602982f,
    0.965418397165533f, 0.965469773015420f, 0.965528437277655f,
    0.965528437277655f, 0.965630969126429f, 0.965689484282883f,
    0.965740640709502f, 0.965791755822677f, 0.965842829685402f,
    0.965901149384097f, 0.965952135064471f, 0.966003079691463f,
    0.966053983327487f, 0.966104846034818f, 0.966155667875590f,
    0.966213700018171f, 0.966264434496339f, 0.966315128302334f,
    0.966365781497721f, 0.966416394143929f, 0.966466966302251f,
    0.966517498033843f, 0.966567989399724f, 0.966618440460782f,
    0.966668851277767f, 0.966719221911296f, 0.966769552421851f,
    0.966819842869783f, 0.966870093315306f, 0.966920303818505f,
    0.966970474439330f, 0.967020605237600f, 0.967063542843005f,
    0.967113599843356f, 0.967163617191341f, 0.967213594946271f,
    0.967263533167330f, 0.967306305935844f, 0.967356170893433f,
    0.967405996485532f, 0.967505529807874f, 0.967548138931234f,
    0.967597813233595f, 0.967647448454308f, 0.967689961867422f,
    0.967739524661887f, 0.967789048540268f, 0.967831466648599f,
    0.967880918407252f, 0.967923274783712f, 0.967972654620856f,
    0.968021995812854f, 0.968064257548057f, 0.968113527121939f,
    0.968155727554356f, 0.968204925706321f, 0.968247065003786f,
    0.968296191929245f, 0.968338270258917f, 0.968387326152496f,
    0.968429343680863f, 0.968478328736411f, 0.968520285629296f,
    0.968562214592319f, 0.968611096462447f, 0.968652965031151f,
    0.968701776537117f, 0.968743584875698f, 0.968785365470961f,
    0.968834074480683f, 0.968875795084627f, 0.968917488055148f,
    0.968959153426817f, 0.969007728191248f, 0.969090912086360f,
    0.969132462833532f, 0.969180904050232f, 0.969222395427913f,
    0.969263859458335f, 0.969305296175547f, 0.969346705613531f,
    0.969394982191855f, 0.969436332641141f, 0.969477655918411f,
    0.969518952057392f, 0.969560221091750f, 0.969601463055090f,
    0.969642677980956f, 0.969683865902829f, 0.969725026854133f,
    0.969766160868228f, 0.969807267978414f, 0.969848348217932f,
    0.969889401619961f, 0.969930428217621f, 0.969971428043973f,
    0.970012401132016f, 0.970053347514690f, 0.970094267224876f,
    0.970135160295395f, 0.970176026759010f, 0.970216866648424f,
    0.970257679996279f, 0.970298466835163f, 0.970339227197601f,
    0.970373173964662f, 0.970413885871233f, 0.970454571393199f,
    0.970495230562863f, 0.970535863412469f, 0.970650847168530f,
    0.970691379422253f, 0.970731885510953f, 0.970765620620724f,
    0.970806078823052f, 0.970846510950618f, 0.970886917035171f,
    0.970920568901195f, 0.970960927322491f, 0.971001259790416f,
    0.971034850377613f, 0.971075135346213f, 0.971115394450634f,
    0.971148923969595f, 0.971189135737796f, 0.971222625853735f,
    0.971262790390816f, 0.971302929209645f, 0.971336358601896f,
    0.971376450351361f, 0.971409840562608f, 0.971449885346942f,
    0.971483236463907f, 0.971523234387005f, 0.971556546496124f,
    0.971596497661541f, 0.971629770848972f, 0.971669675359932f,
    0.971702909711553f, 0.971742767670943f, 0.971775963272355f,
    0.971809141288259f, 0.971848931719255f, 0.971921815239525f,
    0.971954916087967f, 0.971987999445650f, 0.972027676411655f,
    0.972060721351105f, 0.972093748855591f, 0.972133358871293f,
    0.972166348080007f, 0.972199319909290f, 0.972238863187940f,
    0.972271796843339f, 0.972304713174582f, 0.972344189928433f,
    0.972377068207112f, 0.972409929216652f, 0.972442772974193f,
    0.972482162734699f, 0.972514968598050f, 0.972547757264083f,
    0.972580528749833f, 0.972613283072310f, 0.972646020248501f,
    0.972685282250713f, 0.972717981764744f, 0.972750664186678f,
    0.972783329533400f, 0.972815977821766f, 0.972848609068610f,
    0.972881223290743f, 0.972920337908315f, 0.972952914735131f,
    0.972985474590863f, 0.973018017492218f, 0.973050543455880f,
    0.973083052498507f, 0.973115544636735f, 0.973148019887177f,
    0.973180478266418f, 0.973212919791023f, 0.973245344477530f,
    0.973277752342457f, 0.973310143402295f, 0.973342517673512f,
    0.973374875172553f, 0.973407215915839f, 0.973439539919767f,
    0.973471847200711f, 0.973504137775022f, 0.973562218760164f,
    0.973594462642685f, 0.973626689880490f, 0.973658900489817f,
    0.973691094486881f, 0.973723271887874f, 0.973755432708964f,
    0.973787576966297f, 0.973813280457093f, 0.973845394940276f,
    0.973877492904786f, 0.973909574366681f, 0.973941639341995f,
    0.973967279462671f, 0.973999314802478f, 0.974031333700481f,
    0.974063336172629f, 0.974088926334435f, 0.974120899280163f,
    0.974152855844570f, 0.974184796043519f, 0.974210336430194f,
    0.974242247211220f, 0.974274141671084f, 0.974299645498340f,
    0.974331510619852f, 0.974363359464321f, 0.974388826830985f,
    0.974420646416359f, 0.974452449768633f, 0.974477880773140f,
    0.974509654945262f, 0.974541412928054f, 0.974566807668448f,
    0.974598536549719f, 0.974630249285257f, 0.974655607859195f,
    0.974687291571531f, 0.974712626946994f, 0.974744281686313f,
    0.974769593903399f, 0.974801219719758f, 0.974826508818462f,
    0.974858105761792f, 0.974889686686078f, 0.974914939902112f,
    0.974946492030788f, 0.974971722230190f, 0.975003245612865f,
    0.975028452835272f, 0.975053649858377f, 0.975085131806375f,
    0.975110305909561f, 0.975141759232295f, 0.975166910454954f,
    0.975198335201602f, 0.975223463583028f, 0.975254859802644f,
    0.975279965382029f, 0.975305060839729f, 0.975336415939985f,
    0.975361488652084f, 0.975392815344700f, 0.975417865350151f,
    0.975442905276238f, 0.975474191021417f, 0.975499208296949f,
    0.975524215518001f, 0.975555460416840f, 0.975605419639930f,
    0.975605419639930f, 0.975636623793157f, 0.975661575850883f,
    0.975686517903629f, 0.975711449958978f, 0.975742600980867f,
    0.975767510569758f, 0.975792410185843f, 0.975823520693185f,
    0.975848397898037f, 0.975898122470337f, 0.975922969852819f,
    0.975922969852819f, 0.975954015123685f, 0.976003655334843f,
    0.976028460584524f, 0.976053255940233f, 0.976084236232837f,
    0.976109009354376f, 0.976133772606159f, 0.976158525995621f,
    0.976183269530186f, 0.976208003217274f, 0.976208003217274f,
    0.976238906489402f, 0.976263618046739f, 0.976288319780646f,
    0.976313011698507f, 0.976362366115577f, 0.976387028629510f,
    0.976411681356846f, 0.976436324304925f, 0.976467114249117f,
    0.976467114249117f, 0.976491735220672f, 0.976516346436775f,
    0.976565539631852f, 0.976590121625417f, 0.976614693892712f,
    0.976639256441014f, 0.976663809277591f, 0.976688352409702f,
    0.976712885844598f, 0.976737409589524f, 0.976761923651714f,
    0.976761923651714f, 0.976786428038397f, 0.976810922756793f,
    0.976835407814114f, 0.976859883217562f, 0.976884348974336f,
    0.976908805091622f, 0.976933251576602f, 0.976957688436448f,
    0.977098013951529f, 0.977122386028731f, 0.977146748536122f,
    0.977171101480806f, 0.977195444869879f, 0.977213696145079f,
    0.977238022828887f, 0.977262339976560f, 0.977286647595167f,
    0.977310945691766f, 0.977335234273410f, 0.977353444469693f,
    0.977377716417114f, 0.977401978868934f, 0.977426231832178f,
    0.977450475313860f, 0.977474709320991f, 0.977492878612876f,
    0.977517096056381f, 0.977541304044568f, 0.977565502584416f,
    0.977583645293024f, 0.977607827315053f, 0.977631999907899f,
    0.977631999907899f, 0.977674279277313f, 0.977698425975728f,
    0.977722563270990f, 0.977722563270990f, 0.977764780932182f,
    0.977788892404450f, 0.977812994499489f, 0.977831064921111f,
    0.977855150622588f, 0.977879226965781f, 0.977897278085939f,
    0.977921338068673f, 0.977945388712013f, 0.977963420569662f,
    0.977987454885549f, 0.978011479880872f, 0.978029492514857f,
    0.978053501215636f, 0.978077500614628f, 0.978095494063675f,
    0.978119477200933f, 0.978137458461494f, 0.978161425357848f,
    0.978185382983022f, 0.978203345121770f, 0.978227286538691f,
    0.978245236529027f, 0.978269161758424f, 0.978293077747100f,
    0.978311008678442f, 0.978334908512089f, 0.978352827334900f,
    0.978376711034146f, 0.978394617763882f, 0.978418485349318f,
    0.978436380001407f, 0.978460231493585f, 0.978478114083426f,
    0.978501949502858f, 0.978519820045822f, 0.978543639412983f,
    0.978561497924412f, 0.978585301259738f, 0.978603147754944f,
    0.978626935078835f, 0.978644769573101f, 0.978668540905916f,
    0.978686363414501f, 0.978710118776561f, 0.978727929314691f,
    0.978751668726282f, 0.978769467309155f, 0.978793190790523f,
    0.978810977433311f, 0.978828758963556f, 0.978852459722509f,
    0.978870229334292f, 0.978893914212034f, 0.978911671920457f,
    0.978929424532057f, 0.978953086757244f, 0.978970827487009f,
    0.978994473879778f, 0.979012202742743f, 0.979029926524532f,
    0.979053550334292f, 0.979071262270710f, 0.979088969135123f,
    0.979112570402636f, 0.979130265443050f, 0.979147955420607f,
    0.979171534186528f, 0.979189212361401f, 0.979206885482540f,
    0.979230441787417f, 0.979248103127132f, 0.979265759422211f,
    0.979289293306483f, 0.979306937841341f, 0.979324577340638f,
    0.979348088844638f, 0.979365716604861f, 0.979383339338575f,
    0.979400957048490f, 0.979424439518260f, 0.979442045516508f,
    0.979459646499977f, 0.979477242471370f, 0.979500695974660f,
    0.979518280261699f, 0.979535859545650f, 0.979553433829205f,
    0.979576858433533f, 0.979594421059955f, 0.979611978694939f,
    0.979629531341168f, 0.979647079001324f, 0.979670468130047f,
    0.979688004166443f, 0.979723061310450f, 0.979740582423400f,
    0.979763936178014f, 0.979763936178014f, 0.979781445700440f,
    0.979816449859165f, 0.979833944500784f, 0.979851434187449f,
    0.979874746066551f, 0.979874746066551f, 0.979909697391286f,
    0.979927165637130f, 0.979944628942151f, 0.979962087308993f,
    0.979979540740298f, 0.979996989238702f, 0.980020246234401f,
    0.980037683232959f, 0.980055115307399f, 0.980072542460351f,
    0.980089964694443f, 0.980107382012301f, 0.980124794416549f,
    0.980142201909809f, 0.980159604494699f, 0.980177002173838f,
    0.980194394949842f, 0.980211782825323f, 0.980229165802893f,
    0.980252335491858f, 0.980269707051114f, 0.980287073721149f,
    0.980304435504567f, 0.980321792403967f, 0.980339144421948f,
    0.980356491561107f, 0.980373833824038f, 0.980391171213333f,
    0.980408503731583f, 0.980425831381378f, 0.980443154165302f,
    0.980460472085940f, 0.980477785145876f, 0.980495093347688f,
    0.980512396693956f, 0.980529695187255f, 0.980546988830161f,
    0.980564277625245f, 0.980581561575078f, 0.980598840682228f,
    0.980616114949261f, 0.980627628439596f, 0.980644894645466f,
    0.980662156018053f, 0.980679412559917f, 0.980696664273614f,
    0.980713911161699f, 0.980731153226725f, 0.980748390471244f,
    0.980765622897805f, 0.980782850508954f, 0.980800073307237f,
    0.980817291295196f, 0.980845977259090f, 0.980863182431940f,
    0.980880382803776f, 0.980897578377130f, 0.980914769154535f,
    0.980931955138521f, 0.980931955138521f, 0.980949136331616f,
    0.980966312736347f, 0.980994929443608f, 0.981012093091701f,
    0.981012093091701f, 0.981029251960675f, 0.981046406053047f,
    0.981063555371332f, 0.981109263565884f, 0.981126395400670f,
    0.981126395400670f, 0.981143522473080f, 0.981172057017508f,
    0.981189171402487f, 0.981206281034263f, 0.981223385915333f,
    0.981240486048191f, 0.981251883500099f, 0.981268975724811f,
    0.981286063207956f, 0.981303145952021f, 0.981320223959493f,
    0.981331606667595f, 0.981348676786265f, 0.981365742174963f,
    0.981382802836166f, 0.981394173985141f, 0.981411226772692f,
    0.981428274839352f, 0.981445318187591f, 0.981456677799626f,
    0.981473713289321f, 0.981490744067179f, 0.981507770135665f,
    0.981519118232888f, 0.981536136457894f, 0.981553149980093f,
    0.981564489716769f, 0.981581495406417f, 0.981598496399810f,
    0.981615492699399f, 0.981626820959334f, 0.981643809441364f,
    0.981660793236125f, 0.981672113163134f, 0.981689089151212f,
    0.981706060458539f, 0.981717372064213f, 0.981734335575712f,
    0.981751294412966f, 0.981762597708870f, 0.981779548761129f,
    0.981796495145636f, 0.981807790143312f, 0.981824728753634f,
    0.981841662702683f, 0.981852949413651f, 0.981869875599303f,
    0.981886797130148f, 0.981898075565905f, 0.981914989344119f,
    0.981926262613694f, 0.981943168645996f, 0.981960070033971f,
    0.981971335046964f, 0.981988228699756f, 0.982005117714653f,
    0.982016374482499f, 0.982033255772922f, 0.982044507393347f,
    0.982061380965979f, 0.982089493304948f, 0.982106354542990f,
    0.982117592800940f, 0.982134446338523f, 0.982145679465052f,
    0.982162525308829f, 0.982179366539510f, 0.982190591464981f,
    0.982207425012481f, 0.982218644818040f, 0.982235470688987f,
    0.982246685379053f, 0.982263503580068f, 0.982274713159051f,
    0.982291523696748f, 0.982302728169054f, 0.982319531050036f,
    0.982330730420066f, 0.982347525650929f, 0.982364316294885f,
    0.982375507510410f, 0.982392290514783f, 0.982403476639445f,
    0.982420252010811f, 0.982431433048991f, 0.982448200793916f,
    0.982459376749988f, 0.982476136875028f, 0.982487307753362f,
    0.982498476601823f, 0.982515226070027f, 0.982526389846857f,
    0.982543131710882f, 0.982554290420436f, 0.982571024686812f,
    0.982582178333442f, 0.982598905008691f, 0.982610053596740f,
    0.982626772687374f, 0.982637916221183f, 0.982654627733708f,
    0.982665766217610f, 0.982676902682761f, 0.982693603596846f,
    0.982704735018149f, 0.982721428369703f, 0.982732554751479f,
    0.982749240546979f, 0.982760361893543f, 0.982771481227230f,
    0.982788156455121f, 0.982799270759628f, 0.982815938446979f,
    0.982827047726609f, 0.982838154997492f, 0.982854812138917f,
    0.982865914390938f, 0.982882564007288f, 0.982893661244735f,
    0.982904756477552f, 0.982921395569597f, 0.982932485793838f,
    0.982949117376225f, 0.982960202596166f, 0.982971285815582f,
    0.982987906895218f, 0.982998985116313f, 0.983010061339272f,
    0.983026671928701f, 0.983037743159306f, 0.983048812394161f,
    0.983065412505902f, 0.983076476754361f, 0.983087539009450f,
    0.983104128656004f, 0.983115185930644f, 0.983126241214290f,
    0.983142820408134f, 0.983164919037780f, 0.983164919037780f,
    0.983181487791369f, 0.983203572508969f, 0.983203572508969f,
    0.983220130834737f, 0.983242201656853f, 0.983242201656853f,
    0.983258749567212f, 0.983280806510381f, 0.983280806510381f,
    0.983297344017722f, 0.983308366547671f, 0.983330405670727f,
    0.983330405670727f, 0.983346929820930f, 0.983368955102724f,
    0.983379964780070f, 0.983396475593535f, 0.983396475593535f,
    0.983418483103573f, 0.983418483103573f, 0.983434983558095f,
    0.983445981396647f, 0.983467971162045f, 0.983467971162045f,
    0.983484458315984f, 0.983495447291238f, 0.983517419339052f,
    0.983517419339052f, 0.983533893212681f, 0.983555851498819f,
    0.983566827695369f, 0.983577801928461f, 0.983594259598068f,
    0.983605228925419f, 0.983616196291637f, 0.983627161697383f,
    0.983627161697383f, 0.983643606131571f, 0.983654566639040f,
    0.983676481780187f, 0.983687436415187f, 0.983703864700081f,
    0.983703864700081f, 0.983714814445903f, 0.983736708074663f,
    0.983747651958919f, 0.983758593890641f, 0.983775003128660f,
    0.983785940181924f, 0.983829668899257f, 0.983846059132147f,
    0.983856983520667f, 0.983867905963228f, 0.983878826460486f,
    0.983900661621713f, 0.983917032891085f, 0.983917032891085f,
    0.983927944642585f, 0.983938854452384f, 0.983960668249489f,
    0.983971572238103f, 0.983982474287629f, 0.983998823727552f,
    0.983998823727552f, 0.984020616200059f, 0.984031509531754f,
    0.984042400927945f, 0.984053290389283f, 0.984064177916417f,
    0.984075063509998f, 0.984085947170676f, 0.984102269038919f,
    0.984113147870179f, 0.984124024770807f, 0.984134899741452f,
    0.984145772782762f, 0.984156643895383f, 0.984167513079964f,
    0.984178380337151f, 0.984189245667592f, 0.984200109071931f,
    0.984210970550817f, 0.984227259160331f, 0.984238115828405f,
    0.984248970573283f, 0.984259823395610f, 0.984270674296031f,
    0.984281523275189f, 0.984292370333728f, 0.984303215472291f,
    0.984314058691521f, 0.984324899992062f, 0.984335739374555f,
    0.984346576839642f, 0.984357412387966f, 0.984368246020168f,
    0.984379077736889f, 0.984389907538769f, 0.984400735426450f,
    0.984411561400571f, 0.984422385461773f, 0.984433207610695f,
    0.984444027847976f, 0.984460254620969f, 0.984471070081943f,
    0.984481883633511f, 0.984492695276311f, 0.984503505010981f,
    0.984514312838157f, 0.984525118758478f, 0.984535922772579f,
    0.984546724881096f, 0.984557525084666f, 0.984568323383925f,
    0.984579119779507f, 0.984589914272047f, 0.984600706862181f,
    0.984611497550543f, 0.984622286337767f, 0.984633073224487f,
    0.984643858211335f, 0.984649249992506f, 0.984670812370678f,
    0.984681590712963f, 0.984692367158224f, 0.984703141707094f,
    0.984703141707094f, 0.984713914360202f, 0.984724685118182f,
    0.984735453981663f, 0.984746220951275f, 0.984756986027650f,
    0.984767749211417f, 0.984778510503206f, 0.984789269903646f,
    0.984800027413365f, 0.984810783032994f, 0.984821536763160f,
    0.984832288604491f, 0.984843038557615f, 0.984853786623160f,
    0.984869905183637f, 0.984880648532976f, 0.984891389996930f,
    0.984902129576124f, 0.984912867271185f, 0.984923603082736f,
    0.984934337011405f, 0.984945069057816f, 0.984955799222593f,
    0.984966527506360f, 0.984977253909743f, 0.984987978433363f,
    0.984993339990457f, 0.985004061695604f, 0.985014781522546f,
    0.985025499471908f, 0.985036215544310f, 0.985046929740375f,
    0.985057642060725f, 0.985068352505981f, 0.985079061076764f,
    0.985084414659422f, 0.985095120419660f, 0.985105824306976f,
    0.985116526321991f, 0.985127226465325f, 0.985137924737595f,
    0.985148621139423f, 0.985159315671427f, 0.985164662236437f,
    0.985175353964864f, 0.985186043825012f, 0.985196731817500f,
    0.985207417942944f, 0.985218102201962f, 0.985228784595170f,
    0.985244804687940f, 0.985255482419322f, 0.985266158287053f,
    0.985266158287053f, 0.985276832291746f, 0.985292839806936f,
    0.985303509156728f, 0.985314176645634f, 0.985324842274270f,
    0.985324842274270f, 0.985335506043249f, 0.985351498211202f,
    0.985362157333723f, 0.985372814598732f, 0.985383470006843f,
    0.985383470006843f, 0.985399449638666f, 0.985410100407211f,
    0.985420749320998f, 0.985431396380641f, 0.985431396380641f,
    0.985442041586749f, 0.985458005921870f, 0.985468646496813f,
    0.985479285220358f, 0.985479285220358f, 0.985495239835637f,
    0.985505873933352f, 0.985516506181800f, 0.985527136581592f,
    0.985527136581592f, 0.985543078716377f, 0.985553704497184f,
    0.985564328431462f, 0.985574950519818f, 0.985585570762857f,
    0.985590880192573f, 0.985601497668777f, 0.985612113301180f,
    0.985622727090390f, 0.985628033293987f, 0.985638644319542f,
    0.985649253503417f, 0.985659860846216f, 0.985665163827402f,
    0.985675768409722f, 0.985686371152477f, 0.985696972056273f,
    0.985702271818749f, 0.985712869965236f, 0.985723466274270f,
    0.985734060746455f, 0.985739357293918f, 0.985749949011959f,
    0.985760538894657f, 0.985776420278752f, 0.985787005575724f,
    0.985797589039459f, 0.985802880084049f, 0.985813460799053f,
    0.985824039682320f, 0.985834616734451f, 0.985839904574028f,
    0.985850478880578f, 0.985861051357492f, 0.985866336910022f,
    0.985876906643602f, 0.985887474549043f, 0.985898040626941f,
    0.985903322980749f, 0.985913886318456f, 0.985924447830116f,
    0.985929727901365f, 0.985940286675075f, 0.985950843624230f,
    0.985956121414786f, 0.985966675628229f, 0.985977228018608f,
    0.985987778586520f, 0.985993053187237f, 0.986003601022566f,
    0.986014147036915f, 0.986019419361409f, 0.986029962645406f,
    0.986040504109912f, 0.986045774160041f, 0.986056312896423f,
    0.986066849814798f, 0.986072117592419f, 0.986082651784897f,
    0.986093184160851f, 0.986098449667817f, 0.986108979320098f,
    0.986119507157337f, 0.986124770395501f, 0.986135295511287f,
    0.986145818813510f, 0.986151079784721f, 0.986161600367710f,
    0.986166859979636f, 0.986177377844719f, 0.986187893898605f,
    0.986193151246533f, 0.986203664584727f, 0.986214176113198f,
    0.986219431198972f, 0.986229940013964f, 0.986240447020705f,
    0.986245699846166f, 0.986256204141637f, 0.986261455611793f,
    0.986271957197316f, 0.986282456976943f, 0.986287706189730f,
    0.986298203261614f, 0.986308698529071f, 0.986313945486323f,
    0.986324438048239f, 0.986329683653050f, 0.986340173510745f,
    0.986350661566360f, 0.986355904918569f, 0.986366390272160f,
    0.986382114926572f, 0.986382114926572f, 0.986392595779718f,
    0.986397835531571f, 0.986413552089130f, 0.986424027546565f,
    0.986424027546565f, 0.986434501206598f, 0.986450208327797f,
    0.986455443136794f, 0.986465911408123f, 0.986465911408123f,
    0.986481610449544f, 0.986492074234293f, 0.986497305454026f,
    0.986507766548573f, 0.986512996423533f, 0.986523454829188f,
    0.986533911442975f, 0.986539139078099f, 0.986549593005172f,
    0.986554819297267f, 0.986565270538934f, 0.986570495488651f,
    0.986580944046217f, 0.986591390815106f, 0.986596613528979f,
    0.986607057615941f, 0.986612278989175f, 0.986622720395512f,
    0.986627940428760f, 0.986638379155774f, 0.986643597849686f,
    0.986654033898679f, 0.986659251253905f, 0.986669684626176f,
    0.986680116214689f, 0.986685331340216f, 0.986695760254172f,
    0.986700974042745f, 0.986711400283442f, 0.986716612735710f,
    0.986727036304445f, 0.986732247421057f, 0.986742668319126f,
    0.986747878100727f, 0.986758296329425f, 0.986763504776665f,
    0.986773920337285f, 0.986779127450809f, 0.986789540344645f,
    0.986794746125101f, 0.986805156353445f, 0.986810360801477f,
    0.986820768365621f, 0.986831174154347f, 0.986836376383108f,
    0.986846779509784f, 0.986851980407841f, 0.986862380873755f,
    0.986867580441755f, 0.986877978248194f, 0.986883176486778f,
    0.986893571635030f, 0.986898768544842f, 0.986909161036194f,
    0.986914356617876f, 0.986924746453612f, 0.986935134519650f,
    0.986945520816562f, 0.986950713301773f, 0.986961096946065f,
    0.986966288105286f, 0.986976669098239f, 0.986981858932111f,
    0.986992237275006f, 0.986997425784170f, 0.987007801478288f,
    0.987012988663383f, 0.987023361710002f, 0.987028547571668f,
    0.987038917972068f, 0.987044102510944f, 0.987054470266402f,
    0.987059653483127f, 0.987059653483127f, 0.987070018594919f,
    0.987080381944977f, 0.987090743533868f, 0.987095923668051f,
    0.987106282616252f, 0.987111461430409f, 0.987121817739193f,
    0.987126995233961f, 0.987137348904601f, 0.987142525080615f,
    0.987147700817186f, 0.987158050972279f, 0.987163225390944f,
    0.987173572910860f, 0.987178746012253f, 0.987189090898262f,
    0.987194262683019f, 0.987204604936389f, 0.987209775405143f,
    0.987214945435371f, 0.987225284180528f, 0.987230452895598f,
    0.987240789011073f, 0.987245956411619f, 0.987256289898679f,
    0.987261455985333f, 0.987292443313425f, 0.987276951618636f,
    0.987302768922823f, 0.987313092783174f, 0.987318254057631f,
    0.987328575295461f, 0.987333735258974f, 0.987344053875544f,
    0.987349212528741f, 0.987354370745308f, 0.987364685868824f,
    0.987369842775915f, 0.987380155281111f, 0.987385310879356f,
    0.987390466041458f, 0.987400775057515f, 0.987405928911610f,
    0.987416235312279f, 0.987421387858994f, 0.987426539970055f,
    0.987436842885494f, 0.987441993690012f, 0.987447144059154f,
    0.987457443491593f, 0.987462592555029f, 0.987472889376680f,
    0.987478037135035f, 0.987483184458503f, 0.987493477801052f,
    0.987503769404885f, 0.987503769404885f, 0.987514059270557f,
    0.987519203551756f, 0.987529490811229f, 0.987534633789641f,
    0.987539776333929f, 0.987550060120412f, 0.987555201362745f,
    0.987560342171233f, 0.987570622486945f, 0.987580901068104f,
    0.987580901068104f, 0.987591177915262f, 0.987596315688765f,
    0.987606589935962f, 0.987611726409794f, 0.987616862450543f,
    0.987627133233060f, 0.987632267974967f, 0.987637402284067f,
    0.987647669604116f, 0.987652802615204f, 0.987657935193759f,
    0.987668199053549f, 0.987673330334921f, 0.987678461184036f,
    0.987688721585774f, 0.987693851138533f, 0.987698980259311f,
    0.987709237205201f, 0.987714365030449f, 0.987719492423992f,
    0.987729745916237f, 0.987734872015076f, 0.987739997682485f,
    0.987750247723285f, 0.987760496039188f, 0.987760496039188f,
    0.987770742630743f, 0.987780987498498f, 0.987780987498498f,
    0.987791230643004f, 0.987796351569210f, 0.987801472064809f,
    0.987811711764460f, 0.987816830968650f, 0.987827068086100f,
    0.987827068086100f, 0.987837303482766f, 0.987842420535977f,
    0.987847537159197f, 0.987857769115940f, 0.987862884449599f,
    0.987867999353541f, 0.987878227872547f, 0.987883341487748f,
    0.987888454673505f, 0.987898679756961f, 0.987903791654796f,
    0.987908903123460f, 0.987914014163022f, 0.987924234955111f,
    0.987929344707774f, 0.987934454031607f, 0.987944671393056f,
    0.987949779430808f, 0.987959994220708f, 0.987959994220708f,
    0.987970207296921f, 0.987975313192566f, 0.987995732493650f,
    0.988000836248887f, 0.988005939576247f, 0.988016144947605f,
    0.988021246991739f, 0.988026348608267f, 0.988031449797257f,
    0.988041650892892f, 0.988046750799674f, 0.988051850279188f,
    0.988062047956687f, 0.988067146154806f, 0.988072243925930f,
    0.988077341270126f, 0.988087534678001f, 0.988092630741817f,
    0.988097726378975f, 0.988102821589542f, 0.988113010731178f,
    0.988118104662380f, 0.988123198167263f, 0.988128291245894f,
    0.988138476124667f, 0.988143567924946f, 0.988148659299242f,
    0.988153750247623f, 0.988163930866911f, 0.988169020537952f,
    0.988174109783348f, 0.988179198603165f, 0.988189374966338f,
    0.988194462509826f, 0.988199549628007f, 0.988204636320946f,
    0.988214808431370f, 0.988219893848989f, 0.988224978841637f,
    0.988230063409379f, 0.988235147552284f, 0.988245314563850f,
    0.988250397432645f, 0.988255479876871f, 0.988260561896595f,
    0.988270724662807f, 0.988275805409428f, 0.988280885731816f,
    0.988285965630037f, 0.988296124154249f, 0.988301202780374f,
    0.988306280982600f, 0.988311358760995f, 0.988316436115625f,
    0.988331665637599f, 0.988336741297841f, 0.988341816534654f,
    0.988346891348103f, 0.988357039705180f, 0.988362113248942f,
    0.988367186369607f, 0.988372259067245f, 0.988377331341919f,
    0.988387474622650f, 0.988392545628840f, 0.988397616212334f,
    0.988402686373199f, 0.988407756111503f, 0.988417894320692f,
    0.988422962791711f, 0.988428030840435f, 0.988433098466929f,
    0.988438165671262f, 0.988448298813708f, 0.988448298813708f,
    0.988453364751954f, 0.988463495362826f, 0.988468560035585f,
    0.988478688116080f, 0.988478688116080f, 0.988488814510321f,
    0.988493877075263f, 0.988498939218840f, 0.988509062242168f,
    0.988509062242168f, 0.988514123122051f, 0.988519183580837f,
    0.988529303235375f, 0.988534362431262f, 0.988534362431262f,
    0.988544479560602f, 0.988549537494187f, 0.988554595007138f,
    0.988564708771400f, 0.988569765022844f, 0.988579876264688f,
    0.988584931255221f, 0.988589985825582f, 0.988595039975838f,
    0.988605147016298f, 0.988605147016298f, 0.988615252377130f,
    0.988620304427851f, 0.988625356058862f, 0.988630407270231f,
    0.988635458062022f, 0.988640508434303f, 0.988650607920595f,
    0.988655657034738f, 0.988660705729634f, 0.988665754005349f,
    0.988670801861948f, 0.988680896318064f, 0.988680896318064f,
    0.988690989098507f, 0.988696034860517f, 0.988701080203806f,
    0.988706125128441f, 0.988711169634487f, 0.988716213722009f,
    0.988721257391074f, 0.988731343474094f, 0.988736385888181f,
    0.988741427884073f, 0.988746469461837f, 0.988751510621536f,
    0.988761591687008f, 0.988771671081014f, 0.988771671081014f,
    0.988781748804078f, 0.988786787039171f, 0.988791824856726f,
    0.988796862256806f, 0.988801899239479f, 0.988806935804810f,
    0.988817007683707f, 0.988817007683707f, 0.988822042997403f,
    0.988827077894019f, 0.988837146436271f, 0.988842180082038f,
    0.988847213310986f, 0.988852246123180f, 0.988862310497568f,
    0.988862310497568f, 0.988867342059893f, 0.988872373205725f,
    0.988877403935130f, 0.988887464144920f, 0.988892493625434f,
    0.988897522689783f, 0.988902551338030f, 0.988912607386482f,
    0.988912607386482f, 0.988922661771312f, 0.988927688340031f,
    0.988932714493039f, 0.988937740230403f, 0.988942765552186f,
    0.988947790458455f, 0.988952814949272f, 0.988957839024706f,
    0.988967885929677f, 0.988967885929677f, 0.988972908759345f,
    0.988982953173371f, 0.988987974757859f, 0.988992995927416f,
    0.988998016682109f, 0.989003037022000f, 0.989008056947156f,
    0.989013076457642f, 0.989018095553521f, 0.989028132501721f,
    0.989033150354172f, 0.989038167792276f, 0.989043184816098f,
    0.989048201425703f, 0.989053217621155f, 0.989058233402521f,
    0.989063248769862f, 0.989068263723246f, 0.989073278262736f,
    0.989078292388397f, 0.989083306100294f, 0.989093332283054f,
    0.989093332283054f, 0.989098344754046f, 0.989103356811533f,
    0.989108368455578f, 0.989113379686246f, 0.989123400907711f,
    0.989128410898637f, 0.989133420476444f, 0.989138429641197f,
    0.989143438392961f, 0.989148446731799f, 0.989153454657776f,
    0.989158462170958f, 0.989163469271408f, 0.989173482234369f,
    0.989173482234369f, 0.989178488097010f, 0.989188498584932f,
    0.989193503210343f, 0.989198507423472f, 0.989203511224385f,
    0.989208514613144f, 0.989213517589816f, 0.989218520154462f,
    0.989223522307149f, 0.989228524047940f, 0.989233525376900f,
    0.989238526294092f, 0.989243526799581f, 0.989248526893431f,
    0.989253526575707f, 0.989258525846471f, 0.989263524705789f,
    0.989268523153725f, 0.989278518815704f, 0.989283516029877f,
    0.989288512832923f, 0.989293509224907f, 0.989298505205893f,
    0.989303500775945f, 0.989308495935126f, 0.989313490683502f,
    0.989318485021135f, 0.989323478948089f, 0.989328472464430f,
    0.989333465570220f, 0.989338458265524f, 0.989343450550404f,
    0.989348442424926f, 0.989353433889154f, 0.989358424943150f,
    0.989363415586979f, 0.989368405820704f, 0.989373395644390f,
    0.989378385058100f, 0.989383374061898f, 0.989388362655847f,
    0.989393350840012f, 0.989398338614456f, 0.989403325979243f,
    0.989408312934436f, 0.989413299480100f, 0.989418285616298f,
    0.989423271343093f, 0.989428256660550f, 0.989433241568731f,
    0.989438226067700f, 0.989443210157522f, 0.989448193838259f,
    0.989453177109976f, 0.989458159972735f, 0.989463142426600f,
    0.989468124471636f, 0.989473106107904f, 0.989478087335469f,
    0.989483068154395f, 0.989488048564745f, 0.989493028566581f,
    0.989498008159969f, 0.989502987344970f, 0.989507966121648f,
    0.989512944490068f, 0.989517922450292f, 0.989522900002383f,
    0.989527877146405f, 0.989532853882422f, 0.989537830210496f,
    0.989542806130691f, 0.989547781643070f, 0.989552756747697f,
    0.989557731444634f, 0.989562705733946f, 0.989567679615695f,
    0.989572653089944f, 0.989577626156756f, 0.989582598816196f,
    0.989587571068325f, 0.989592542913208f, 0.989597514350907f,
    0.989602485381486f, 0.989607456005007f, 0.989612426221533f,
    0.989617396031129f, 0.989622365433856f, 0.989627334429779f,
    0.989632303018959f, 0.989637271201460f, 0.989642238977345f,
    0.989647206346678f, 0.989652173309519f, 0.989657139865935f,
    0.989662106015986f, 0.989667071759736f, 0.989667071759736f,
    0.989672037097247f, 0.989677002028584f, 0.989681966553808f,
    0.989686930672983f, 0.989691894386171f, 0.989696857693435f,
    0.989701820594839f, 0.989706783090444f, 0.989711745180314f,
    0.989716706864512f, 0.989721668143100f, 0.989726629016141f,
    0.989731589483698f, 0.989736549545834f, 0.989741509202611f,
    0.989746468454092f, 0.989751427300340f, 0.989756385741417f,
    0.989761343777386f, 0.989761343777386f, 0.989766301408311f,
    0.989771258634252f, 0.989776215455275f, 0.989781171871439f,
    0.989786127882809f, 0.989791083489447f, 0.989796038691415f,
    0.989800993488776f, 0.989805947881593f, 0.989810901869928f,
    0.989815855453843f, 0.989820808633402f, 0.989825761408667f,
    0.989830713779699f, 0.989835665746562f, 0.989840617309318f,
    0.989840617309318f, 0.989845568468030f, 0.989850519222759f,
    0.989855469573569f, 0.989860419520521f, 0.989865369063679f,
    0.989870318203104f, 0.989875266938858f, 0.989880215271005f,
    0.989885163199606f, 0.989885163199606f, 0.989890110724724f,
    0.989895057846421f, 0.989900004564759f, 0.989904950879801f,
    0.989909896791608f, 0.989914842300244f, 0.989919787405770f,
    0.989924732108248f, 0.989929676407741f, 0.989934620304311f,
    0.989939563798019f, 0.989944506888929f, 0.989944506888929f,
    0.989949449577102f, 0.989954391862601f, 0.989959333745488f,
    0.989964275225823f, 0.989969216303671f, 0.989974156979092f,
    0.989979097252149f, 0.989984037122904f, 0.989988976591419f,
    0.989993915657756f, 0.990102473491435f, 0.989998854321976f,
    0.990003792584142f, 0.990008730444317f, 0.990013667902561f,
    0.990018604958937f, 0.990127118651726f, 0.990132046482196f,
    0.990136973912279f, 0.990141900942038f, 0.990146827571534f,
    0.990097543257357f, 0.990048218861426f, 0.990053153106457f,
    0.990058086950115f, 0.990166530088123f, 0.990171454717229f,
    0.990176378946441f, 0.990181302775822f, 0.990077818312251f,
    0.990082750149969f, 0.990087681586685f, 0.990092612622461f,
    0.990200994096251f, 0.990205915927393f, 0.990210837359071f,
    0.990215758391347f, 0.990220679024282f, 0.990220679024282f,
    0.990225599257938f, 0.990230519092376f, 0.990235438527657f,
    0.990240357563843f, 0.990245276200994f, 0.990250194439173f,
    0.990255112278440f, 0.990255112278440f, 0.990260029718857f,
    0.990264946760485f, 0.990269863403386f, 0.990274779647619f,
    0.990279695493247f, 0.990284610940331f, 0.990284610940331f,
    0.990289525988932f, 0.990294440639111f, 0.990299354890930f,
    0.990304268744449f, 0.990309182199729f, 0.990314095256832f,
    0.990314095256832f, 0.990319007915819f, 0.990323920176751f,
    0.990328832039688f, 0.990333743504692f, 0.990338654571825f,
    0.990338654571825f, 0.990343565241146f, 0.990348475512717f,
    0.990353385386600f, 0.990358294862854f, 0.990363203941541f,
    0.990363203941541f, 0.990368112622723f, 0.990373020906459f,
    0.990377928792811f, 0.990382836281839f, 0.990387743373605f,
    0.990387743373605f, 0.990392650068170f, 0.990397556365594f,
    0.990402462265938f, 0.990407367769264f, 0.990412272875631f,
    0.990412272875631f, 0.990417177585100f, 0.990422081897733f,
    0.990426985813592f, 0.990431889332734f, 0.990436792455222f,
    0.990441695181118f, 0.990446597510480f, 0.990446597510480f,
    0.990451499443371f, 0.990456400979850f, 0.990461302119979f,
    0.990466202863817f, 0.990471103211427f, 0.990471103211427f,
    0.990476003162868f, 0.990480902718201f, 0.990485801877487f,
    0.990490700640787f, 0.990490700640787f, 0.990495599008160f,
    0.990500496979667f, 0.990505394555370f, 0.990510291735329f,
    0.990515188519603f, 0.990515188519603f, 0.990520084908255f,
    0.990524980901343f, 0.990529876498930f, 0.990534771701074f,
    0.990539666507838f, 0.990539666507838f, 0.990544560919280f,
    0.990549454935462f, 0.990554348556445f, 0.990559241782288f,
    0.990564134613051f, 0.990564134613051f, 0.990569027048796f,
    0.990573919089582f, 0.990578810735471f, 0.990583701986522f,
    0.990583701986522f, 0.990588592842796f, 0.990593483304352f,
    0.990598373371253f, 0.990598373371253f, 0.990603263043557f,
    0.990608152321324f, 0.990613041204617f, 0.990617929693493f,
    0.990622817788014f, 0.990622817788014f, 0.990627705488241f,
    0.990632592794232f, 0.990637479706049f, 0.990642366223752f,
    0.990642366223752f, 0.990647252347400f, 0.990652138077054f,
    0.990657023412774f, 0.990661908354621f, 0.990661908354621f,
    0.990666792902654f, 0.990671677056933f, 0.990676560817519f,
    0.990681444184472f, 0.990681444184472f, 0.990686327157852f,
    0.990691209737718f, 0.990696091924130f, 0.990700973717151f,
    0.990700973717151f, 0.990705855116837f, 0.990710736123251f,
    0.990715616736451f, 0.990715616736451f, 0.990720496956498f,
    0.990725376783453f, 0.990730256217374f, 0.990735135258321f,
    0.990735135258321f, 0.990740013906355f, 0.990744892161536f,
    0.990749770023923f, 0.990754647493576f, 0.990754647493576f,
    0.990759524570556f, 0.990764401254922f, 0.990769277546733f,
    0.990774153446051f, 0.990774153446051f, 0.990779028952933f,
    0.990783904067441f, 0.990788778789634f, 0.990788778789634f,
    0.990793653119573f, 0.990798527057315f, 0.990803400602923f,
    0.990808273756453f, 0.990808273756453f, 0.990813146517969f,
    0.990818018887528f, 0.990822890865190f, 0.990827762451014f,
    0.990827762451014f, 0.990832633645062f, 0.990837504447392f,
    0.990842374858063f, 0.990842374858063f, 0.990847244877137f,
    0.990852114504671f, 0.990856983740726f, 0.990856983740726f,
    0.990861852585361f, 0.990866721038637f, 0.990871589100612f,
    0.990876456771346f, 0.990876456771346f, 0.990881324050898f,
    0.990886190939329f, 0.990891057436697f, 0.990891057436697f,
    0.990895923543062f, 0.990900789258484f, 0.990905654583022f,
    0.990905654583022f, 0.990910519516736f, 0.990915384059684f,
    0.990920248211927f, 0.990920248211927f, 0.990925111973523f,
    0.990929975344533f, 0.990934838325015f, 0.990939700915029f,
    0.990939700915029f, 0.990944563114635f, 0.990949424923891f,
    0.990954286342857f, 0.990954286342857f, 0.990959147371593f,
    0.990964008010157f, 0.990968868258609f, 0.990968868258609f,
    0.990973728117009f, 0.990978587585414f, 0.990983446663886f,
    0.990988305352482f, 0.990988305352482f, 0.990993163651263f,
    0.990998021560288f, 0.991002879079614f, 0.991002879079614f,
    0.991007736209303f, 0.991012592949413f, 0.991017449300003f,
    0.991017449300003f, 0.991022305261132f, 0.991027160832860f,
    0.991032016015245f, 0.991032016015245f, 0.991036870808346f,
    0.991041725212224f, 0.991046579226937f, 0.991046579226937f,
    0.991051432852543f, 0.991056286089103f, 0.991061138936675f,
    0.991061138936675f, 0.991065991395318f, 0.991070843465090f,
    0.991075695146052f, 0.991075695146052f, 0.991080546438262f,
    0.991085397341780f, 0.991090247856663f, 0.991090247856663f,
    0.991095097982971f, 0.991099947720763f, 0.991104797070098f,
    0.991109646031035f, 0.991109646031035f, 0.991114494603633f,
    0.991119342787949f, 0.991119342787949f, 0.991124190584045f,
    0.991129037991978f, 0.991133885011807f, 0.991133885011807f,
    0.991138731643591f, 0.991143577887389f, 0.991148423743260f,
    0.991148423743260f, 0.991153269211262f, 0.991158114291454f,
    0.991158114291454f, 0.991162958983895f, 0.991167803288644f,
    0.991172647205759f, 0.991172647205759f, 0.991177490735300f,
    0.991182333877324f, 0.991187176631891f, 0.991187176631891f,
    0.991192040874586f, 0.991196904726712f, 0.991196904726712f,
    0.991201768188329f, 0.991206631259494f, 0.991211493940268f,
    0.991211493940268f, 0.991216356230710f, 0.991221218130880f,
    0.991226079640835f, 0.991230940760638f, 0.991230940760638f,
    0.991235801490345f, 0.991240661830017f, 0.991245521779712f,
    0.991245521779712f, 0.991250381339491f, 0.991255240509413f,
    0.991260099289535f, 0.991260099289535f, 0.991264957679918f,
    0.991269815680622f, 0.991274673291704f, 0.991274673291704f,
    0.991279530513224f, 0.991284387345243f, 0.991289243787817f,
    0.991289243787817f, 0.991294099841006f, 0.991298955504871f,
    0.991303810779470f, 0.991303810779470f, 0.991308665664861f,
    0.991313520161105f, 0.991318374268260f, 0.991318374268260f,
    0.991323227986385f, 0.991328081315538f, 0.991332934255781f,
    0.991332934255781f, 0.991337786807170f, 0.991342638969767f,
    0.991347490743627f, 0.991347490743627f, 0.991352342128812f,
    0.991357193125380f, 0.991357193125380f, 0.991362043733390f,
    0.991366893952901f, 0.991371743783973f, 0.991371743783973f,
    0.991376593226661f, 0.991381442281029f, 0.991386290947134f,
    0.991386290947134f, 0.991391139225033f, 0.991395987114787f,
    0.991400834616454f, 0.991400834616454f, 0.991405681730093f,
    0.991410528455763f, 0.991410528455763f, 0.991415374793522f,
    0.991420220743429f, 0.991425066305544f, 0.991425066305544f,
    0.991429911479924f, 0.991434756266630f, 0.991439600665718f,
    0.991439600665718f, 0.991444444677248f, 0.991449288301280f,
    0.991449288301280f};

// Mapping between reverberation times at a given frequency and the
// corresponding magnitude compensation value required in the specral reverb to
// achieve 1.0 magnitude for any decay rate when combined with the convolution
// compensator. This table covers reverberation times between
// kMinReverbTimeForFeedback and kNumFeedbackValues * kTimeStepSizeForFeedback,
// i.e. 0.15s and 25s @48kHz.
static const float kMagnitudeCompensation[kNumFeedbackValues] = {
    17.3996715487709f, 17.4034178858004f, 16.6795907475848f, 16.3502965109599f,
    15.9879028507326f, 15.6718483944106f, 15.4042251167111f, 15.1677088170684f,
    14.9119678952536f, 14.6906933374371f, 14.2839187463763f, 14.0999475816121f,
    13.9338828421032f, 13.7652114598195f, 13.6133888181873f, 13.4638764218013f,
    13.3247278048121f, 13.1976499042192f, 13.0833841919256f, 12.7669049046296f,
    12.7761019806869f, 12.7855596748707f, 12.5979372062571f, 12.5244917829334f,
    12.4453918694998f, 12.3672592847373f, 12.3020244979211f, 12.2257529341565f,
    12.1547662449381f, 12.0845455398564f, 12.0231689748723f, 11.9586881174297f,
    11.8906434508319f, 11.8264759328308f, 11.7652619217413f, 11.7033346037227f,
    11.6379964417238f, 11.5862703081324f, 11.5299758955061f, 11.4741541469989f,
    11.4224679633351f, 11.4302928484179f, 11.3885962828649f, 11.3488611750486f,
    11.3151942251170f, 11.2687417305887f, 11.2651043092380f, 11.2873091969354f,
    11.1115917717737f, 11.1187139401529f, 11.0873969606168f, 11.0667794924678f,
    11.0355838523984f, 11.0107298415034f, 10.9796516432030f, 10.9630980289917f,
    10.9420391333514f, 10.9144740761416f, 10.8934689087169f, 10.8708578658852f,
    10.8533772904939f, 10.8325917807584f, 10.8084481007036f, 10.7910279996611f,
    10.7720483888874f, 10.7547079194992f, 10.7374196414357f, 10.7165315149605f,
    10.6992241376899f, 10.6906704210781f, 10.6696884648005f, 10.6557033369952f,
    10.6417418339839f, 10.6206945653294f, 10.6123225679390f, 10.5984100558311f,
    10.5804524022014f, 10.5664834165837f, 10.5556799009337f, 10.5448786038089f,
    10.5294926904862f, 10.5186888881000f, 10.5078877209760f, 10.4896735339343f,
    10.4818821922801f, 10.4710009885916f, 10.4601237996041f, 10.4509410528977f,
    10.4400494357799f, 10.4322036844072f, 10.4213123997993f, 10.4058408037523f,
    10.3978977400326f, 10.3899444363220f, 10.3853855448482f, 10.3697415198647f,
    10.3617555934757f, 10.3567416154720f, 10.3487308415134f, 10.3329739194789f,
    10.3248849674214f, 10.3167888305905f, 10.3116328550137f, 10.2935369581880f,
    10.2854263705648f, 10.2802322447455f, 10.2721029640593f, 10.2668785541679f,
    10.2421605382004f, 10.2455580945816f, 10.2402346533852f, 10.2417289351959f,
    10.2364188981879f, 10.2144782407169f, 10.2091021586862f, 10.2037115797613f,
    10.1983068112122f, 10.1849493802464f, 10.1794698792260f, 10.1739775209827f,
    10.1684725944209f, 10.1699972796703f, 10.1673363396048f, 10.1537525491825f,
    10.1481899931936f, 10.1454318458448f, 10.1398423821346f, 10.1370496372401f,
    10.1314343707374f, 10.1205181210259f, 10.1148376525328f, 10.1119389454472f,
    10.1062355079601f, 10.1105790660401f, 10.1076676719351f, 10.1019633798750f,
    10.0907918971796f, 10.0877973262721f, 10.0820222224882f, 10.0790001296191f,
    10.0759630305115f, 10.0729111643779f, 10.0698447671938f, 10.0585205719156f,
    10.0553915426054f, 10.0495078552707f, 10.0463551070209f, 10.0452327409897f,
    10.0420899577109f, 10.0416595363245f, 10.0301423062663f, 10.0269263405620f,
    10.0236983112226f, 10.0204584174658f, 10.0172068557167f, 10.0139438196431f,
    10.0106695001895f, 10.0100859757974f, 10.0144981569414f, 10.0027240655604f,
    9.99940079964503f, 9.99875825234152f, 9.99541139468150f, 9.99205454764473f,
    9.98868788138219f, 9.98799186037639f, 9.98460299286733f, 9.98120480749684f,
    9.97196520049622f, 9.96851944732945f, 9.96773313504258f, 9.97209005664967f,
    9.97130280431796f, 9.96784064615800f, 9.96702924221558f, 9.96354778825994f,
    9.96271284320390f, 9.95921268399594f, 9.94972407107315f, 9.94618172696550f,
    9.94527759238012f, 9.93907565972122f, 9.93815157408174f, 9.93457833610339f,
    9.94163812518103f, 9.93806970926284f, 9.93712514990341f, 9.93616969007899f,
    9.92384118906784f, 9.92284379437166f, 9.92183604073516f, 9.91819656036143f,
    9.91716993715598f, 9.91613331652545f, 9.91508681621950f, 9.91141646117235f,
    9.91035202464596f, 9.90927805124168f, 9.90819465265145f, 9.90449521562619f,
    9.89464259105904f, 9.90163545023412f, 9.90051546012025f, 9.89938659067352f,
    9.89565102186460f, 9.89450616426505f, 9.89335273158198f, 9.89219082303091f,
    9.89102053655261f, 9.88984196883182f, 9.88865521531579f, 9.88746037023253f,
    9.88367272054804f, 9.88246342154956f, 9.87235666909112f, 9.87111421978793f,
    9.86728668752777f, 9.86603072697377f, 9.86476741507816f, 9.87181651341944f,
    9.87055453050722f, 9.86928534086147f, 9.86800902634652f, 9.86672566780610f,
    9.86543534507815f, 9.86413813700939f, 9.86283412146973f, 9.85253909146675f,
    9.85120575616663f, 9.84730850541530f, 9.84340913473107f, 9.83695739915978f,
    9.83305682291249f, 9.82661110215619f, 9.82270954762832f, 9.81627003605364f,
    9.81236772402631f, 9.80593460975945f, 9.79446129569662f, 9.79056524322650f,
    9.79518744944566f, 9.78878964277671f, 9.78490541018618f, 9.76941213950054f,
    9.76551357605385f, 9.76161372925909f, 9.75771265131274f, 9.75131744052475f,
    9.74741680228965f, 9.74351508082574f, 9.73961232562214f, 9.73322796347969f,
    9.72932601448363f, 9.72542317182318f, 9.72151948243115f, 9.71761499262133f,
    9.71124407362971f, 9.70734082271271f, 9.70343690224853f, 9.69953235617044f,
    9.69562722783490f, 9.69172156002971f, 9.67869145869549f, 9.67477248299185f,
    9.67085316486435f, 9.66693354428727f, 9.67164058015198f, 9.66773182471560f,
    9.66382281819403f, 9.65991359906596f, 9.65600420530895f, 9.65209467440629f,
    9.64818504335383f, 9.64427534866659f, 9.64036562638542f, 9.63645591208342f,
    9.63254624087234f, 9.62863664740886f, 9.62713438108161f, 9.61841452486423f,
    9.61450796945654f, 9.61060162177350f, 9.59745989603663f, 9.59593610568275f,
    9.59201716961870f, 9.58809863249534f, 9.58418052465712f, 9.58026287605631f,
    9.57872704411307f, 9.57480783656106f, 9.57088917994674f, 9.56934602396844f,
    9.56542599874574f, 9.56150661276656f, 9.55758789336276f, 9.55603586291498f,
    9.56094674392723f, 9.55703713657417f, 9.55548790574812f, 9.55157724555479f,
    9.54766736708134f, 9.54611162685054f, 9.54220086685072f, 9.54064051939564f,
    9.53672899077399f, 9.53281837539013f, 9.51720800158667f, 9.51563186381870f,
    9.51171473845506f, 9.51013443950261f, 9.50621695380252f, 9.50696588424213f,
    9.50071486073663f, 9.49214537555389f, 9.48823490695630f, 9.48432562212793f,
    9.48041754157442f, 9.47651068553726f, 9.47260507399707f, 9.46870072667699f,
    9.46479766304589f, 9.45859100251033f, 9.45469300691147f, 9.45079634803183f,
    9.44690104436323f, 9.44300711415882f, 9.43911457543615f, 9.43522344598003f,
    9.43133374334554f, 9.42744548486084f, 9.42355868763001f, 9.42195400960424f,
    9.41806778052289f, 9.41374743291046f, 9.40986387622231f, 9.40598186864776f,
    9.40210142598266f, 9.39822256381766f, 9.39434529754074f, 9.39046964233973f,
    9.38659561320474f, 9.38498117319374f, 9.38110807138758f, 9.37723664318189f,
    9.37336690279205f, 9.36949886424754f, 9.36563254139424f, 9.36401291268887f,
    9.36014771343569f, 9.35628427401529f, 9.35242260755338f, 9.34856272700286f,
    9.34693907021185f, 9.34308046758153f, 9.33922369235477f, 9.33536875680650f,
    9.33374195178015f, 9.32988841242305f, 9.32381510855490f, 9.31996765124740f,
    9.31612209259297f, 9.31227844391980f, 9.31064913327039f, 9.30680704022146f,
    9.29348811817957f, 9.29184974751964f, 9.28800440783775f, 9.28636432840859f,
    9.28252074101649f, 9.27867919003329f, 9.27703723070236f, 9.27319750721753f,
    9.26935985268265f, 9.26771614205892f, 9.26388038770541f, 9.26223520182348f,
    9.25840139650094f, 9.25675479157226f, 9.25292298305829f, 9.24909331708271f,
    9.25658949582583f, 9.25276798764379f, 9.25112470249905f, 9.24730525026412f,
    9.24566068030482f, 9.24184332892114f, 9.24019752634543f, 9.23638231973555f,
    9.23690246959584f, 9.23308722316596f, 9.23143799550340f, 9.22762498397928f,
    9.22597467697943f, 9.22432329385452f, 9.22051260344843f, 9.21886021442665f,
    9.21505186252288f, 9.21339851457425f, 9.20959254004180f, 9.20578887012963f,
    9.20198751127673f, 9.19073957709422f, 9.18693490086577f, 9.18527430481475f,
    9.18361278881117f, 9.17981083017848f, 9.17814860221564f, 9.17648548821270f,
    9.17268633214397f, 9.17102256976036f, 9.16935795435656f, 9.16556168321056f,
    9.16389648126049f, 9.16223045844461f, 9.15843715205156f, 9.15677060283272f,
    9.15510326406306f, 9.15131299980263f, 9.14964519313344f, 9.14797662741422f,
    9.14630731290934f, 9.13195283996486f, 9.13028849958014f, 9.12862343577599f,
    9.12485055311793f, 9.12318514040497f, 9.12151903298212f, 9.11985224052471f,
    9.11608300035933f, 9.11441593072157f, 9.11274820378056f, 9.11107982889063f,
    9.10941081532760f, 9.10774117228957f, 9.10397656900970f, 9.10230675285809f,
    9.10063633356450f, 9.10830768064681f, 9.10664083749441f, 9.10288539850849f,
    9.10121842744103f, 9.09955087784672f, 9.09788275836467f, 9.08651657603331f,
    9.08068020302688f, 9.08108732540316f, 9.07941328828146f, 9.07773874094359f,
    9.07398723838858f, 9.07231273992199f, 9.07063775460245f, 9.06896229034989f,
    9.06521511911848f, 9.06353976158507f, 9.06186394770784f, 9.06018768515226f,
    9.05851098152150f, 9.05683384435697f, 9.05515628113895f, 9.05347829928709f,
    9.05179990616104f, 9.05012110906098f, 9.04844191522816f, 9.04676233184547f,
    9.04302460323958f, 9.04134529301830f, 9.03966561367638f, 9.03798557216934f,
    9.03630517539697f, 9.03462443020383f, 9.03294334337977f, 9.03126192166043f,
    9.02958017172773f, 9.02789810021035f, 9.02621571368424f, 9.02453301867307f,
    9.02285002164873f, 9.02116672903177f, 9.01948314719191f, 9.01779928244843f,
    9.01611514107069f, 9.01443072927854f, 9.01274605324280f, 9.01106111908564f,
    9.00937593288109f, 9.00769050065541f, 9.00600482838754f, 9.00431892200953f,
    9.00263278740694f, 9.00094643041926f, 8.99925985684034f, 8.99757307241874f,
    8.99588608285818f, 8.99419889381793f, 8.99251151091317f, 8.99082393971542f,
    8.99116133873637f, 8.98947240095708f, 8.97807476175783f, 8.97638164966389f,
    8.97468839063631f, 8.97299498992199f, 8.97130145272722f, 8.96960778421804f,
    8.96791398952060f, 8.96622007372152f, 8.96452604186820f, 8.96283189896917f,
    8.96113764999443f, 8.96145642376267f, 8.95976097981039f, 8.95806544520587f,
    8.95636982476753f, 8.95467412327692f, 8.95297834547896f, 8.95128249608231f,
    8.94958657975968f, 8.95734018337137f, 8.95765278435025f, 8.95595916486210f,
    8.95426548584055f, 8.95257175184105f, 8.95087796738447f, 8.94918413695742f,
    8.94949047792581f, 8.94779558087857f, 8.94610065184975f, 8.94440569519098f,
    8.94271071522106f, 8.93902043601862f, 8.93932070246017f, 8.93961997400109f,
    8.93792395970352f, 8.93622794393283f, 8.93453193081558f, 8.93283592444736f,
    8.93113992889301f, 8.92944394818690f, 8.92973639134932f, 8.92803947305345f,
    8.92634258225175f, 8.92464572285846f, 8.92493436813772f, 8.92323660608605f,
    8.92153888773201f, 8.91984121687354f, 8.91814359728028f, 8.91644603269380f,
    8.91474852682785f, 8.91305108336851f, 8.91135370597452f, 8.91163407099939f,
    8.90993586525176f, 8.90823773709895f, 8.90653969009158f, 8.90681648856557f,
    8.90511764518828f, 8.90341889416476f, 8.90172023894098f, 8.90199353826512f,
    8.90029411780736f, 8.89859480404659f, 8.89689560032744f, 8.89519650996990f,
    8.89546552758263f, 8.89376570958158f, 8.89206601546450f, 8.88053809639265f,
    8.88080074152891f, 8.87909746670552f, 8.87739433859366f, 8.87765453752411f,
    8.87595075137300f, 8.87424712182442f, 8.87254365189478f, 8.87280064185817f,
    8.87109654154655f, 8.86743422965210f, 8.86768885156784f, 8.86598526773548f,
    8.86428186186200f, 8.86453414821111f, 8.86478566315377f, 8.86112634248555f,
    8.86137632515970f, 8.85967194497569f, 8.85796776101988f, 8.85626377603165f,
    8.85651073575462f, 8.85480620525780f, 8.85310188252331f, 8.85334661076713f,
    8.85164176132608f, 8.84993712826266f, 8.85017965504828f, 8.84847451372278f,
    8.84676959721478f, 8.84700995202877f, 8.84530454535357f, 8.84359937176427f,
    8.84383758357059f, 8.84213193756712f, 8.84042653275027f, 8.84066263000176f,
    8.83895677018950f, 8.83725115950061f, 8.83748517015020f, 8.83577912155772f,
    8.83407332986507f, 8.83430528137662f, 8.83259906855242f, 8.83089312024770f,
    8.83112303960668f, 8.82941668662965f, 8.82771060563841f, 8.82793851936218f,
    8.82623204985186f, 8.82645864099079f, 8.82475179379158f, 8.82304523091814f,
    8.82326985960455f, 8.82156293469962f, 8.81985630119355f, 8.82007899274514f,
    8.81837201257628f, 8.81666533073787f, 8.81688611003994f, 8.81517909662492f,
    8.81539861470977f, 8.82335720989358f, 8.82165261003093f, 8.82187267469875f,
    8.82016775424161f, 8.82038657061920f, 8.81675884387406f, 8.81697642252574f,
    8.81527182158995f, 8.81548817296149f, 8.81378328023903f, 8.81207870996388f,
    8.81229323974577f, 8.81058839169725f, 8.81080171980556f, 8.80909660340947f,
    8.80739182006356f, 8.80760336445859f, 8.80589832635901f, 8.80610869398147f,
    8.80440341027862f, 8.80461261089678f, 8.80290709063344f, 8.80120191782847f,
    8.80140938602084f, 8.79970398968595f, 8.79991031482745f, 8.78825930793902f,
    8.78846219390559f, 8.78675407755697f, 8.78504632874136f, 8.78524755053805f,
    8.78353962067476f, 8.78373974434793f, 8.78203164176273f, 8.78223067625635f,
    8.78052240917839f, 8.77881452261966f, 8.77901193990290f, 8.77730390068432f,
    8.77559624718409f, 8.77579206684317f, 8.77408427230694f, 8.77427903760903f,
    8.77257110989291f, 8.77276482931164f, 8.77105677618141f, 8.76934912094798f,
    8.76954128723245f, 8.76783351773637f, 8.76802465892895f, 8.76631678271580f,
    8.76650690697914f, 8.76479893150834f, 8.76498804691885f, 8.76327997956479f,
    8.76346809411380f, 8.76175994216697f, 8.76194706376180f, 8.76023883442988f,
    8.76042497089499f, 8.75871667130397f, 8.75890183038202f, 8.75719346757723f,
    8.75737765693012f, 8.75566923787733f, 8.75585246508727f, 8.75414399667368f,
    8.75432626924423f, 8.75261775827954f, 8.75090968234295f, 8.75109053685399f,
    8.75127092234682f, 8.74956234640392f, 8.74785421147492f, 8.74803320078602f,
    8.74632504621761f, 8.74461733673038f, 8.74291007344532f, 8.74308722620506f,
    8.74326392423471f, 8.74155620054932f, 8.74173199434523f, 8.74002427303862f,
    8.74019916951030f, 8.73849145680757f, 8.73866546279447f, 8.73695776485170f,
    8.73713088712407f, 8.73542321002923f, 8.73559545528903f, 8.73388780506281f,
    8.73405917994453f, 8.73235156254122f, 8.73252207361275f, 8.73081449492107f,
    8.73098414868451f, 8.72927661452841f, 8.72944541742091f, 8.72773793356043f,
    8.72790589195497f, 8.72619846408700f, 8.72636558429323f, 8.72653228775183f,
    8.72482450631728f, 8.72499038093493f, 8.72328266978537f, 8.72344772171672f,
    8.72174008633388f, 8.72003293816951f, 8.72019676747887f, 8.71848970287163f,
    8.71865272461753f, 8.71881534481493f, 8.71710796902965f, 8.71726979056008f,
    8.71556251187906f, 8.71572354060119f, 8.71401636421506f, 8.71417660593049f,
    8.71246953697379f, 8.71262899742776f, 8.71278806938804f, 8.71108072586171f,
    8.71123902507422f, 8.70953180189128f, 8.70968933396333f, 8.70798223606704f,
    8.70813900655191f, 8.70829539830896f, 8.70658805323002f, 8.70674369162996f,
    8.70503648428256f, 8.70519137474645f, 8.70348430989052f, 8.70363845778761f,
    8.70379223639875f, 8.70208495078059f, 8.70223799478307f, 8.70053086365168f,
    8.70068317828796f, 8.69897620622755f, 8.69912779669041f, 8.68932884670592f,
    8.68762000302773f, 8.68776884448380f, 8.68606017937344f, 8.68620831732905f,
    8.68449983514613f, 8.68464727456896f, 8.68479436657368f, 8.68308572543799f,
    8.68323212627427f, 8.68152367907888f, 8.68166939359894f, 8.68181476676632f,
    8.68010617755452f, 8.67839813079483f, 8.67854248705877f, 8.67868650672127f,
    8.67697833094137f, 8.67712168091770f, 8.67541371794501f, 8.67555640288794f,
    8.67384865672637f, 8.67399068124574f, 8.68390062160107f, 8.68219432235443f,
    8.68233691470609f, 8.68247917838292f, 8.68077277138796f, 8.68091438110988f,
    8.67920820006687f, 8.67934916032569f, 8.67948979752457f, 8.67778352432750f,
    8.67792351872432f, 8.67621748133180f, 8.67635683731680f, 8.67649587572979f,
    8.67478976139895f, 8.67492816791096f, 8.67506626008425f, 8.67336007789188f,
    8.67349754458695f, 8.67179161357488f, 8.67192845902535f, 8.67206499542924f,
    8.67035901118758f, 8.67049493262608f, 8.67063054813644f, 8.66892451939922f,
    8.66905952613920f, 8.66735376335083f, 8.66748816540435f, 8.66762226663473f,
    8.66591647342483f, 8.66421125283483f, 8.66250660524407f, 8.66263951321076f,
    8.66093514316382f, 8.65923134784848f, 8.65752812762827f, 8.65582548286296f,
    8.65412341390855f, 8.65242192111739f, 8.65072100483809f, 8.64902066541564f,
    8.64732090319137f, 8.64745089666342f, 8.64575142580246f, 8.64405253375412f,
    8.64235422084484f, 8.64065648739748f, 8.63895933373144f, 8.63726276016262f,
    8.63556676700344f, 8.63387135456289f, 8.63399880607199f, 8.63230369681987f,
    8.63060916980038f, 8.62891522530837f, 8.62722186363537f, 8.62552908506957f,
    8.62383688989589f, 8.62214527839595f, 8.62045425084813f, 8.62057923413287f,
    8.61888852099306f, 8.61719839322406f, 8.61550885109083f, 8.61381989485520f,
    8.61213152477584f, 8.61225490214502f, 8.61056685383226f, 8.60887939303327f,
    8.60719251999378f, 8.60369847555976f, 8.60382053816116f, 8.60394234032854f,
    8.60225697240927f, 8.59876697957841f, 8.59708363808768f, 8.59720440679893f,
    8.59552139839225f, 8.59383898041714f, 8.59215715308918f, 8.59047591662109f,
    8.58879527122275f, 8.58891451420684f, 8.58903350571243f, 8.58735294942756f,
    8.58567298620775f, 8.58399361624780f, 8.58231483973975f, 8.58063665687294f,
    8.57895906783397f, 8.57728207280678f, 8.57560567197261f, 8.57392986551003f,
    8.57404638533917f, 8.57237093049576f, 8.57248696522609f, 8.56902180741298f,
    8.56734813950615f, 8.56746345224170f, 8.56579014113883f, 8.56411742691198f,
    8.56244530971544f, 8.56077378970094f, 8.56088791398092f, 8.55921675580945f,
    8.55933040984012f, 8.55765961583721f, 8.55598942090227f, 8.55431982517126f,
    8.55443254737061f, 8.55276331980773f, 8.55109469242306f, 8.54942666534252f,
    8.54775923868966f, 8.54076218161341f, 8.53909843766319f, 8.53743529276498f,
    8.53754618531528f, 8.53588341401169f, 8.53422124267654f, 8.53255967141845f,
    8.53266966578484f, 8.53100847186734f, 8.52934787891323f, 8.52768788702179f,
    8.52779699406003f, 8.52613738317202f, 8.52447837420358f, 8.52281996724478f,
    8.52292819763660f, 8.52127017526276f, 8.51961275572668f, 8.51795593910948f,
    8.51806330336606f, 8.51640687483569f, 8.51475105002415f, 8.51485777147656f,
    8.50313962983211f, 8.50324475785744f, 8.49982824152465f, 8.49993295139312f,
    8.49827795948697f, 8.49838225368573f, 8.49496979858900f, 8.49507367958556f,
    8.49342030501101f, 8.49176753881319f, 8.49187080459044f, 8.49021844295144f,
    8.48856669037454f, 8.48691554689607f, 8.48701800083704f, 8.48536726502665f,
    8.48371713897484f, 8.48206762271016f, 8.48216927434053f, 8.48052016878301f,
    8.47887167364807f, 8.47897272970743f, 8.47732464765384f, 8.47567717663991f,
    8.47577764234150f, 8.47413058674447f, 8.47248414278601f, 8.47083831047568f,
    8.47093799703846f, 8.46929258301410f, 8.46764778121340f, 8.46774688934850f,
    8.46610250808181f, 8.46445873959686f, 8.46455727431892f, 8.46291392857930f,
    8.46127119616231f, 8.46136916242822f, 8.45972685493173f, 8.45808516128181f,
    8.45818256399157f, 8.45654129740170f, 8.45490064516572f, 8.45499748916322f,
    8.45335726609174f, 8.45171765786506f, 8.45007866445932f, 8.45017477094663f,
    8.44853620927392f, 8.44863195015821f, 8.44699382175575f, 8.44535630913176f,
    8.44371941225248f, 8.44381442815541f, 8.44217796702569f, 8.44054212207913f,
    8.44063659963106f, 8.43900119239275f, 8.43909531357936f, 8.43746034551955f,
    8.43582599452207f, 8.43591958492567f, 8.43428567501132f, 8.43265238255709f,
    8.43274544665058f, 8.43111259715250f, 8.42948036549767f, 8.42957290770386f,
    8.42794112084706f, 8.42630995220218f, 8.42640197689452f, 8.42477125485887f,
    8.42486293694762f, 8.42323266289285f, 8.42160300781375f, 8.42169417956989f,
    8.42006497423241f, 8.41843638820147f, 8.41852705387014f, 8.41689891931203f,
    8.41527140437771f, 8.41536156815713f, 8.41373450639779f, 8.41210806456602f,
    8.41219773060816f, 8.41057174362487f, 8.40894637685971f, 8.40903554927091f,
    8.40741063899950f, 8.40749948458245f, 8.40587503206534f, 8.40425120040078f,
    8.40433955909700f, 8.40271618678552f, 8.40109343558253f, 8.40118131137970f,
    8.39955902110142f, 8.39964657715982f, 8.39802474901748f, 8.39640354256068f,
    8.39649062227007f, 8.39486987947612f, 8.39324975859040f, 8.39163025950670f,
    8.39171671010196f, 8.39180300448298f, 8.39018381508821f, 8.38856524812366f,
    8.38694730347691f, 8.38703297721227f, 8.38711849683149f, 8.38550086657194f,
    8.38388385923108f, 8.38226747469047f, 8.38235238198133f, 8.38073646806615f,
    8.38082107165390f, 8.37920562946549f, 8.37759081052727f, 8.37767496160383f,
    8.37606061575931f, 8.37614446717391f, 8.37453059549704f, 8.37461414884609f,
    8.37300075239995f, 8.37138797988615f, 8.37147108910541f, 8.36985879313347f,
    8.36994160822893f, 8.36832978983540f, 8.36671859574903f, 8.36680097257521f,
    8.36519025733977f, 8.36527234391476f, 8.36366210854089f, 8.36205249781910f,
    8.36213415188276f, 8.36052502225735f, 8.36060638987481f, 8.35899774233109f,
    8.36906328366319f, 8.36745599896731f, 8.36584933792354f, 8.36593087269790f,
    8.36432469282948f, 8.36440594331694f, 8.36280024559060f, 8.36288121327418f,
    8.36127599864676f, 8.35967140818186f, 8.35975195441777f, 8.35814784820884f,
    8.35822811530357f, 8.35662449428343f, 8.35670448368437f, 8.35510134877630f,
    8.35518106192085f, 8.35357841403867f, 8.35365785235434f, 8.35205569240248f,
    8.35045415745305f, 8.35053318618056f, 8.34893214024213f, 8.34901089766681f,
    8.34741034162095f, 8.34581041077519f, 8.34588876385553f, 8.34428932394983f,
    8.34269050924751f, 8.34276846108694f, 8.34284627986358f, 8.34124782440677f,
    8.34132537807708f, 8.33972741611141f, 8.33813007984327f, 8.33820723838498f,
    8.33828426588748f, 8.33668729339379f, 8.33676405981657f, 8.33349522624397f,
    8.33190013122409f, 8.33197650850716f, 8.33205275673382f, 8.33045802938747f,
    8.32886392796943f, 8.32893979098194f, 8.32734618754917f, 8.32742179537051f,
    8.32582869070157f, 8.32590404461620f, 8.32431143948147f, 8.32438654076529f,
    8.32279443592703f, 8.32286928584740f, 8.32127768205985f, 8.32135228187567f,
    8.31976117988511f, 8.31983553084688f, 8.31824493139170f, 8.31831903474163f,
    8.31672893855239f, 8.31680279552442f, 8.31521320332393f, 8.31362423794623f,
    8.31369772764720f, 8.31210926707506f, 8.31218251344738f, 8.31059455838471f,
    8.31066756263400f, 8.30908011377724f, 8.30915287710116f, 8.30756593513926f,
    8.30763845872760f, 8.30605202434212f, 8.30612430937688f, 8.30453838324206f,
    8.30461043089745f, 8.30302501368025f, 8.30309682512281f, 8.30151191748300f,
    8.30158349387164f, 8.29999909646184f, 8.30007043894792f, 8.29848655241367f,
    8.29855766214103f, 8.29697428712085f, 8.29704516522590f, 8.29546230235136f,
    8.29553294996313f, 8.29395059985889f, 8.29402101809908f, 8.29243918138296f,
    8.29250937136602f, 8.29092804864905f, 8.29099801148223f, 8.28941720336873f,
    8.28948694015212f, 8.28790664723974f, 8.28797615906635f, 8.28639638194612f,
    8.28646566990194f, 8.28488640915835f, 8.28495547432238f, 8.28337673053339f,
    8.28344557397773f, 8.28186734771489f, 8.28193597050476f, 8.27017819449824f,
    8.27024585417908f, 8.26866792233384f, 8.26873536584578f, 8.26715795557297f,
    8.26722518394961f, 8.26564829578422f, 8.26571531005260f, 8.26413894452349f,
    8.26420574570412f, 8.26262990333412f, 8.26269649244104f, 8.26276297588936f,
    8.26118755178779f, 8.25961275728051f, 8.25967892525651f, 8.25810465544098f,
    8.25817061434708f, 8.25823646908936f, 8.25666262054728f, 8.25672826770033f,
    8.25515494533288f, 8.25522038587603f, 8.25364759016769f, 8.25371282507417f,
    8.25214055650372f, 8.25220558674063f, 8.25063384578123f, 8.25069867230963f,
    8.24912745942885f, 8.24919208320380f, 8.24925660595830f, 8.24768582083426f,
    8.24775014225673f, 8.24617988660073f, 8.24624400763143f, 8.24467428189162f,
    8.24473820346495f, 8.24316900808411f, 8.24323273112868f, 8.24166406654423f,
    8.24172759198290f, 8.24179101896291f, 8.24022278737693f, 8.24028601812464f,
    8.23871831864912f, 8.23878135407322f, 8.23721418712705f, 8.23727702813061f,
    8.23733977226080f, 8.23577304160806f, 8.23583559266193f, 8.23426939580623f,
    8.23433175467303f, 8.23276609201523f, 8.23282825957879f, 8.23126313151474f,
    8.23132510865348f, 8.22976051557408f, 8.22982230316103f, 8.22988399629783f,
    8.22831984435517f, 8.22838134923978f, 8.22681773347944f, 8.22687905097167f,
    8.22694027508851f, 8.22537710272175f, 8.22543814072614f, 8.22387550570830f,
    8.22393635844707f, 8.22237426113974f, 8.22243492945458f, 8.22087337021466f,
    8.22093385494218f, 8.22099424818818f, 8.21943313609370f, 8.21787265404081f,
    8.21793277408334f, 8.21799280346968f, 8.21643277007580f, 8.21649261876342f,
    8.21493312522628f, 8.21499279402937f, 8.21505237319400f, 8.21349333040847f,
    8.21355273090070f, 8.21199422903237f, 8.21205345165425f, 8.21211258564023f,
    8.21055453657232f, 8.20899711749525f, 8.20905598676346f, 8.20911476818710f,
    8.20755780332721f, 8.20761640965019f, 8.20767492871591f, 8.20611841935722f,
    8.20456253992591f, 8.20462079838524f, 8.20467897036231f, 8.20473705604943f,
    8.20318154739142f, 8.20323946107331f, 8.20168449663402f, 8.20174223907312f,
    8.20018781913647f, 8.20024539109073f, 8.19869151593658f, 8.19874891815950f,
    8.19719558806369f, 8.19725282130430f, 8.19730997033329f, 8.19736703533627f,
    8.19581408270369f, 8.19265325254094f, 8.19271006657415f, 8.19115900249672f,
    8.19121565013490f, 8.18966513207149f, 8.18650958272842f, 8.18656598214056f,
    8.18501734969057f, 8.18346934519575f, 8.18192196836458f, 8.17717170379845f,
    8.17562699749445f, 8.17408291704024f, 8.17253946214471f, 8.17099663251667f,
    8.16945442786483f, 8.16791284789780f, 8.16796835096100f, 8.16642731496180f,
    8.16488690323892f, 8.16334711550016f, 8.16180795145325f, 8.16026941080583f,
    8.15873149326542f, 8.15878643699898f, 8.15724906305914f, 8.15571231181225f,
    8.15417618296508f, 8.15264067622436f, 8.15269522559432f, 8.15116026230331f,
    8.14962592070057f, 8.14809220049210f, 8.14655910138382f, 8.14502662308157f,
    8.14508070479135f, 8.14354876964219f, 8.14201745487630f, 8.14048676019884f,
    8.14054053346655f, 8.13901038188417f, 8.13748084996421f, 8.13595193741119f,
    8.13442364392953f, 8.13289596922361f, 8.13294928474971f, 8.13142215280177f,
    8.12989563919925f, 8.12836974364590f, 8.12842275724862f, 8.12689740436780f,
    8.12537266910277f, 8.12384855115666f, 8.12390126539209f, 8.12237769002218f,
    8.12085473153483f, 8.11933238963259f, 8.11938480702804f, 8.11786300759498f,
    8.11634182430774f, 8.11482125686830f, 8.11487337992274f, 8.11335335483497f,
    8.11183394515285f, 8.11031515057783f, 8.10879697081129f, 8.10727940555463f,
    8.10733109172207f, 8.10581406839611f, 8.10429765913410f, 8.10278186363689f,
    8.10283326158928f, 8.10131800788042f, 8.09980336748775f, 8.09828934011163f,
    8.09834045225178f, 8.09682696651178f, 8.09531409333708f, 8.09380183242756f,
    8.09229018348308f, 8.09234094169636f, 8.09082983407241f, 8.09088045174303f,
    8.08780945305625f, 8.08630017906285f, 8.08635058701088f, 8.08484185400836f,
    8.08489212286426f, 8.08338393098375f, 8.08187634939786f, 8.08192641069197f,
    8.08041937019749f, 8.07891293953684f, 8.07740711840818f, 8.07745690494605f,
    8.07595162471175f, 8.07444695354630f, 8.07294289114748f, 8.07143943721306f,
    8.07148888347334f, 8.06998597006411f, 8.06848366465346f, 8.06698196693880f,
    8.06548087661756f, 8.06552998607523f, 8.06402943589713f, 8.06252949264403f,
    8.06103015601301f, 8.06107899849888f, 8.05958020178404f, 8.05808201122071f,
    8.05658442650561f, 8.05663300418972f, 8.05513595915549f, 8.05363951949682f,
    8.05368789998958f, 8.05219199993907f, 8.05069670478969f, 8.04920201423756f,
    8.04925013366991f, 8.04775598247604f, 8.04626243540297f, 8.04476949214651f,
    8.04481735262478f, 8.04332494846872f, 8.04183314765086f, 8.04188081528225f,
    8.04038955347468f, 8.03889889452528f, 8.03740883812929f, 8.03745625043850f,
    8.03596673278105f, 8.03447781719510f, 8.03452503935586f, 8.03303666240761f,
    8.03154888704735f, 8.03006171296983f, 8.03010868337276f, 8.02862204764774f,
    8.02713601272015f, 8.02565057828453f, 8.02569729893665f, 8.02421240256096f,
    8.02272810619019f, 8.02124440951864f, 8.02129088240572f, 8.01980772349432f,
    8.01985407323696f, 8.01837145214721f, 8.01688943008388f, 8.01540800674083f,
    8.01392718181196f, 8.01397322582548f, 8.01401920905608f, 8.01253886078973f,
    8.01105911056564f, 8.00957995807722f, 8.00810140301796f, 8.00814708413104f,
    8.00819270517784f, 8.00519164505985f, 8.00523714632877f, 8.00376026217617f,
    8.00380564413810f, 8.00232929694652f, 8.00085354600357f, 7.99937839100213f,
    7.99942353575290f, 7.99946862149134f, 7.99647489443730f, 7.99651986249756f,
    7.99504637603060f, 7.99357348450695f, 7.99210118761919f, 7.99214592170044f,
    7.99067416093170f, 7.98920299429772f, 7.98924755409054f, 7.98777692341388f,
    7.98782136757941f, 7.98635127289017f, 7.98488177163773f, 7.98492604320402f,
    7.98345707777071f, 7.98198870526966f, 7.98052092539286f, 7.98056496838578f,
    7.97909772395840f, 7.97914165332759f, 7.97767494437153f, 7.97620882733528f,
    7.97625258709119f, 7.97478700534684f, 7.97332201501439f, 7.97185761578544f,
    7.97039380735165f, 7.97043728659283f, 7.96897401286660f, 7.96751132942632f,
    7.96755464165088f, 7.96609249272969f, 7.96463093358420f, 7.96467407975223f,
    7.96321305493386f, 7.96325609092729f, 7.96179560044416f, 7.96033569902192f,
    7.96037857054268f, 7.95891920325775f, 7.95746042452055f, 7.95750313250930f,
    7.95604488770810f, 7.95458723094048f, 7.95462977633074f, 7.95317265329447f,
    7.95171611777669f, 7.95175850149487f, 7.95030249950052f, 7.95034477594852f,
    7.94888930747411f, 7.94743442579445f, 7.94747654209885f, 7.94602219372557f,
    7.94456843162926f, 7.94461038869789f, 7.94315715969122f, 7.94319901110364f,
    7.94174631517672f, 7.94029420479812f, 7.94033589847266f, 7.93888432095161f,
    7.93743332845856f, 7.93747486528495f, 7.93602440542419f, 7.93457453007028f,
    7.93461591093157f, 7.93316656798164f, 7.93320784535352f, 7.93175903478923f,
    7.93031080799686f, 7.93035193086073f, 7.92890423622049f, 7.92745712482865f,
    7.92749809405007f, 7.92605151457391f, 7.92609238184470f, 7.92464633425995f,
    7.92320086918425f, 7.92324158424087f, 7.92179665081508f, 7.92035229937259f,
    7.92039286306353f, 7.91894904302663f, 7.91898950627539f, 7.91754621761402f,
    7.91610351019241f, 7.91614382347568f, 7.91470164718040f, 7.91474186094918f,
    7.91330021574662f, 7.91185915103762f, 7.91041866651197f, 7.91045868235363f,
    7.91049864894173f, 7.90905864587311f, 7.90761922254927f, 7.90618037865992f,
    7.90622014914004f, 7.90625987081860f, 7.90334410091085f, 7.90338372525529f,
    7.90194652065353f, 7.90198604802103f, 7.90054937343709f, 7.89911327700454f,
    7.89915265957254f, 7.89771709289137f, 7.89775637936779f, 7.89632134239166f,
    7.89488688281196f, 7.89492602580883f, 7.89349209566332f, 7.89353114344322f,
    7.89209774268232f, 7.89213669559212f, 7.89070382416489f, 7.88927152915260f,
    7.88931034040396f, 7.88787857444832f, 7.88791729168944f, 7.88648605473636f,
    7.88652467830826f, 7.88509397030232f, 7.87347922114931f, 7.87351729104996f,
    7.87208727383296f, 7.87212525190789f, 7.87069576429284f, 7.86926685183945f,
    7.86930469279725f, 7.86787630965334f, 7.86791405961169f, 7.86648620571214f,
    7.86505892620407f, 7.86509654027728f, 7.86366978971717f, 7.86370731360733f,
    7.86228109192717f, 7.86231852595811f, 7.86089283308866f, 7.86093017758265f,
    7.85950501345345f, 7.85808042248116f, 7.85811763327064f, 7.85669357073507f,
    7.85673069278658f, 7.85530715861446f, 7.85388419682346f, 7.85392118636180f,
    7.85249875262700f, 7.85253565421687f, 7.85111374846223f, 7.84969241431120f,
    7.84972918456513f, 7.84830837808339f, 7.84834506116903f, 7.84692478227781f,
    7.84696137850464f, 7.84554162712403f, 7.84412244633333f, 7.84415891284905f,
    7.84274025925302f, 7.85141288690937f, 7.85144961512784f, 7.85003244871711f,
    7.85006909037938f, 7.84865244984218f, 7.84868900525374f, 7.84727289050818f,
    7.84730935997309f, 7.84589377093614f, 7.84593015475706f, 7.84451509134460f,
    7.84310059603029f, 7.84313685194958f, 7.84172288193889f, 7.84175905296461f,
    7.84179518165529f, 7.84038169460074f, 7.83896877493622f, 7.83900477706657f,
    7.83759238244247f, 7.83618055466107f, 7.83621643089471f, 7.83480512782671f,
    7.83484092049560f, 7.83343014204975f, 7.83346585144543f, 7.83205559752943f,
    7.83209122394205f, 7.83068149446259f, 7.83071703818099f, 7.82930783304372f,
    7.82789919324926f, 7.82793461346494f, 7.82652649767837f, 7.82656183591605f,
    7.82515424404068f, 7.82518950058452f, 7.82378243252265f, 7.82381760765553f,
    7.82241106330847f, 7.82244615731193f, 7.81815980032046f, 7.81963467968251f,
    7.81822978630920f, 7.81682545615005f, 7.81542168889519f, 7.81545654118684f,
    7.81405329648404f, 7.81408806875962f, 7.81268534650601f, 7.81272003904066f,
    7.81131783913241f, 7.81135245220004f, 7.80995077453237f, 7.80998530840561f,
    7.80858415287283f, 7.80718355872843f, 7.80721797431856f, 7.80581790196027f,
    7.80585223903216f, 7.80445268835214f, 7.80448694717412f, 7.80308791806361f,
    7.80312209890276f, 7.80172359125211f, 7.80175769437431f, 7.80035970807297f,
    7.80039373374288f, 7.79899626867941f, 7.79903021716051f, 7.79763327322259f,
    7.79766714477715f, 7.79627072185158f, 7.79630451674069f, 7.79490861471341f,
    7.79494233319698f, 7.79354695195305f, 7.79215212935342f, 7.79218573371345f,
    7.79221930016098f, 7.79082496013553f, 7.78943117826415f, 7.78946463135824f,
    7.78807136967477f, 7.78810474751849f, 7.78671200590435f, 7.78674530875124f,
    7.78677857419443f, 7.78538631518947f, 7.78399461335494f, 7.78402776696423f,
    7.78263658483832f, 7.78266966420460f, 7.78270270654283f, 7.78131200703774f,
    7.77992186396267f, 7.77995479558889f, 7.77856517185444f, 7.77859802998139f,
    7.77863085145093f, 7.77724171033669f, 7.77585312490987f, 7.77588583677433f,
    7.77449777031297f, 7.77453040941203f, 7.77456301221936f, 7.77317542836560f,
    7.77178839945454f, 7.77182089374904f, 7.77043438342129f, 7.77046680567449f,
    7.77049919199683f, 7.76911316425229f, 7.76772769070368f, 7.76775996959096f,
    7.76637501423668f, 7.76640722179721f, 7.76502278450610f, 7.76505492097597f,
    7.76367100161614f, 7.76370306723039f, 7.76373509750465f, 7.76235166066185f,
    7.76096877676598f, 7.76100070136998f, 7.75961833500535f, 7.75965018945267f,
    7.75968200890831f, 7.75830012500606f, 7.75691879329882f, 7.75695050812457f,
    7.75698218818832f, 7.75560133890093f, 7.75563294961187f, 7.75425261742615f,
    7.75287283668173f, 7.75290434378954f, 7.75152507975708f, 7.75155651807875f,
    7.75158792209177f, 7.75020914037975f, 7.74883090935359f, 7.74745322870535f,
    7.74748449604369f, 7.74610733143631f, 7.74613853077156f, 7.74476188206386f,
    7.74479301361756f, 7.74341688066770f, 7.74344794466043f, 7.74347897495509f,
    7.74210332397728f, 7.74072822211493f, 7.74075915164364f, 7.74079004769272f,
    7.73941542773344f, 7.73944625698609f, 7.73807215231902f, 7.73810291499155f,
    7.73672932547114f, 7.73676002177895f, 7.73538694725902f, 7.73541757741657f,
    7.73404501775030f, 7.73407558197112f, 7.73270353701110f, 7.73273403550780f,
    7.73136250510597f, 7.73139293809028f, 7.73142333839748f, 7.73005228978068f,
    7.72868178804865f, 7.72871209063963f, 7.72874236076320f, 7.72737234072627f,
    7.72740254607128f, 7.72603304009822f, 7.72606318087234f, 7.72469418881164f,
    7.72472426522164f, 7.72335578692120f, 7.72338579917296f, 7.72341577942255f,
    7.72204778277866f, 7.72068033154013f, 7.72071021608960f, 7.72074006884081f,
    7.71937309915523f, 7.71940288846195f, 7.71803643202356f, 7.71806615808777f,
    7.71670021474117f, 7.71672987776400f, 7.71536444735324f, 7.71399956106752f,
    7.71402912990351f, 7.71405866744353f, 7.71269426243436f, 7.71272373753127f,
    7.71135984498676f, 7.71138925783831f, 7.71002587760030f, 7.71005522840340f,
    7.70869236031320f, 7.70872164926393f, 7.70875090736174f, 7.70738852045593f,
    7.70602667618316f, 7.70605584201409f, 7.70608497718635f, 7.70472361397175f,
    7.70475268797144f, 7.70339183636089f, 7.70342084938044f, 7.70344983198180f,
    7.70208946144320f, 7.70211838335150f, 7.70075852418828f, 7.70078738559392f,
    7.69942803764294f, 7.69945683873552f, 7.69809800183312f, 7.69812674280145f,
    7.69815545377820f, 7.69679709781559f, 7.69682574894983f, 7.69546790380054f,
    7.69549649527912f, 7.69552505699968f, 7.69416769278661f, 7.69419619513033f,
    7.69283934149161f, 7.69286778464339f, 7.69151144141215f, 7.69015563785611f,
    7.69018399256504f, 7.69021231788464f, 7.68885699496583f, 7.68888526164366f,
    7.68891349906885f, 7.68755865684680f, 7.68620435356727f, 7.68623250350642f,
    7.68626062437385f, 7.68490680163368f, 7.68493486449261f, 7.68358155123853f,
    7.68360955626822f, 7.68225675232974f, 7.68228469970871f, 7.68231261832904f,
    7.68096029482092f, 7.68098815605693f, 7.67963634161052f, 7.67966414563891f,
    7.67831284008200f, 7.67834058707873f, 7.67836830562541f, 7.67701748037903f,
    7.67704514215666f, 7.67569482554126f, 7.67572243072402f, 7.67437262256575f,
    7.67440017132709f, 7.67442769194256f, 7.67307836396437f, 7.67172957219709f,
    7.67175700863318f, 7.67040872479895f, 7.67043610532978f, 7.67046345797203f,
    7.66911565404928f, 7.66914295104199f, 7.66917022027359f, 7.66782289629849f,
    7.66785011013471f, 7.66650329373374f, 7.66653045234317f, 7.66518414333889f,
    7.66521124688943f, 7.66523832297326f, 7.66389249376287f, 7.66254719901806f,
    7.66257419295179f, 7.66260115958519f, 7.66262809895966f, 7.66128325660027f,
    7.66131014158067f, 7.65996580606912f, 7.65999263682000f, 7.65864880797621f,
    7.65867558466144f, 7.65870233437509f, 7.65735898508777f, 7.65738568098033f,
    7.65604283808071f, 7.65606948031442f, 7.65472714362097f, 7.65475373235741f,
    7.65478029440560f, 7.65343843708835f, 7.65346494588041f, 7.65349142810445f,
    7.65215005018116f, 7.65080920452514f, 7.65083560728422f, 7.65086198363444f,
    7.64952161716499f, 7.64954794081617f, 7.64957423817702f, 7.64823435090775f,
    7.64826059580594f, 7.64692121388197f, 7.64694740647428f, 7.64560852971063f,
    7.64563467015326f, 7.64566078457940f, 7.64432238681307f, 7.64434844932262f,
    7.64301055642301f, 7.64303656717059f, 7.64306255209497f, 7.64172513809099f,
    7.64175107148420f, 7.64041416205060f, 7.64044004406572f, 7.64046590044903f,
    7.63912946980476f, 7.63915527503849f, 7.63918105475433f, 7.63784510290134f,
    7.63787083169477f, 7.63653538400034f, 7.63656106202205f, 7.63522611829733f,
    7.63525174569743f, 7.63527734784282f, 7.63394288268123f, 7.63396843442906f,
    7.63263447293292f, 7.63265997443180f, 7.63268545086176f, 7.63135196780901f,
    7.63137739421188f, 7.63004441451777f, 7.63006979104072f, 7.63009514267872f,
    7.62876264130389f, 7.62878794328165f, 7.62745594495614f, 7.62748119741940f,
    7.62750642517989f, 7.62617490504528f, 7.62620008350880f, 7.62622523737801f,
    7.62489419542260f, 7.62491930021111f, 7.62358876087232f, 7.62361381672361f,
    7.62228377980812f, 7.62230878686512f, 7.62233376957839f, 7.62100421058442f,
    7.62102914471672f, 7.61970008782946f, 7.61972497352234f, 7.61974983504840f,
    7.61842125594179f, 7.61844606923969f, 7.61847085847612f, 7.61714275713074f,
    7.61716749834905f, 7.61583989866594f, 7.61586459200549f, 7.61588926145774f,
    7.61456213938845f, 7.61458676116997f, 7.61326014044020f, 7.61328471468899f,
    7.61330926522300f, 7.61198312195571f, 7.61200762516301f, 7.61203210475825f,
    7.61070643892753f, 7.61073087140089f, 7.60940570645537f, 7.60943009194275f,
    7.60945445398794f, 7.60812976632134f, 7.60815408158342f, 7.60817837350445f,
    7.60685416308698f, 7.60687840842660f, 7.60555469843407f, 7.60557889732612f,
    7.60560307304439f, 7.60427984013719f, 7.60430396960766f, 7.60432807600398f,
    7.60300532014849f, 7.60302938049563f, 7.60170712459905f, 7.60173113902882f,
    7.60175513054917f, 7.60043335153450f, 7.60045729733421f, 7.60048122032260f,
    7.59915991815251f, 7.59918379561589f, 7.59786299293304f, 7.59788682500121f,
    7.59791063442030f, 7.59659030840635f, 7.59661407262401f, 7.59663781428919f,
    7.59531796490316f, 7.59534166155958f, 7.59402231118364f, 7.59404596295913f,
    7.59406959234195f, 7.59275071841253f, 7.59277430310528f, 7.59279786550050f,
    7.59147946797307f, 7.59150298586800f, 7.59018508686814f, 7.59020856038869f,
    7.59023201176912f, 7.58891458898425f, 7.58893799617829f, 7.58896138132594f,
    7.58764443470803f, 7.58766777585621f, 7.58635132727817f, 7.58637462455093f,
    7.58639789993235f, 7.58508192732879f, 7.58510515901996f, 7.58512836891210f,
    7.58515155703590f, 7.58383603861766f, 7.58385918329649f, 7.58254416248636f,
    7.58256726384219f, 7.58259034358211f, 7.58127579858836f, 7.58129883518735f,
    7.58132185026121f, 7.58000778102858f, 7.58003075314258f, 7.57871718101943f,
    7.57874011029371f, 7.57876301819304f, 7.57744992162713f, 7.57747278686613f,
    7.57749563081961f, 7.57618300975245f, 7.57620581122397f, 7.57622859149893f,
    7.57491644587078f, 7.57493918384116f, 7.57496190070343f, 7.57365023045334f,
    7.57367290518743f, 7.57369555890135f, 7.57238436396716f, 7.57240697572833f,
    7.57109627695034f, 7.57111884687516f, 7.57114139592532f, 7.56983117224850f,
    7.56985367963612f, 7.56987616623575f, 7.56856641759483f, 7.56858886270476f,
    7.56861128711287f, 7.56730201344145f, 7.56732439653175f, 7.56734675900595f,
    7.56603796023641f, 7.56606028156374f, 7.56608258236019f, 7.56477425842381f,
    7.56479651824339f, 7.56348868948821f, 7.56351090844373f, 7.56353310700940f,
    7.56355528521325f, 7.56224791073235f, 7.56227004829653f, 7.56229216558275f,
    7.56098526572191f, 7.56100734253566f, 7.55970093738852f, 7.55972297384077f,
    7.55974499015377f, 7.55843905939626f, 7.55846103551346f, 7.55848299157400f,
    7.55850492760535f, 7.55719945116063f, 7.55722134721568f, 7.55724322332355f,
    7.55593822119919f, 7.55596005749435f, 7.55465554945120f, 7.55467734604222f,
    7.55469912282176f, 7.55339508886160f, 7.54282966651951f, 7.55343854360594f,
    7.55346024140898f, 7.55215666177545f, 7.55217832025123f, 7.55087523421793f,
    7.54031819934936f, 7.54033979903436f, 7.53904143572760f, 7.53906299635162f,
    7.53908453748486f, 7.54306747979114f, 7.54837830652259f, 7.54839978934178f,
    7.54709812458524f, 7.53655365351434f, 7.53657507825590f, 7.53527812830351f,
    7.53529951445738f, 7.54588255528895f, 7.54590390295925f, 7.54460316609662f,
    7.54462447538711f, 7.53408833025382f, 7.53279230299647f, 7.53281355491099f,
    7.53151801756132f, 7.53153923135380f, 7.53156042612371f, 7.53158160189662f,
    7.53028651627133f, 7.53030765412677f, 7.53032877306164f, 7.52903415812099f,
    7.52905523929085f, 7.52776111376522f, 7.52778215727126f, 7.52780318198321f,
    7.52782418792625f, 7.52653051408721f, 7.52655148256772f, 7.52657243235452f,
    7.52527922893053f, 7.52530014140483f, 7.52532103526024f, 7.52402830215954f,
    7.52404915885167f, 7.52406999699933f, 7.52277773412920f, 7.52279853526202f,
    7.52281931792438f, 7.52284008214087f, 7.52154827098634f, 7.52156899838468f,
    7.52027767569381f, 7.52029836637182f, 7.52031903872623f, 7.52033969278137f,
    7.51904882176237f, 7.51906943929178f, 7.51909003859465f, 7.51779963749868f,
    7.51782020042086f, 7.51784074518883f, 7.51655081391820f, 7.51657132244977f,
    7.51659181289909f, 7.51530235135519f, 7.51532280571164f, 7.51534324205739f,
    7.51536366041622f, 7.51407465053640f, 7.51409503299255f, 7.51280651060282f,
    7.51282685725088f, 7.51284718603027f, 7.51286749696450f, 7.51157942617877f,
    7.51159970149310f, 7.51161995903261f, 7.51033235764065f, 7.51035257970054f,
    7.51037278405555f, 7.50908565195404f, 7.50910582096898f, 7.50912597234864f,
    7.50783930943340f, 7.50785942561180f, 7.50787952422413f, 7.50789960529340f,
    7.50661339393931f, 7.50663343999128f, 7.50534771513269f, 7.50536772625885f,
    7.50538771995631f, 7.50540769624784f, 7.50412242287476f, 7.50414236442252f,
    7.50416228863237f, 7.50418219552691f, 7.50289737369020f, 7.50291724602175f,
    7.50163291019824f, 7.50165274805672f, 7.50167256871229f, 7.50169237218731f,
    7.50040848781655f, 7.50042825699740f, 7.50044800906457f, 7.49916459326124f,
    7.49918431116761f, 7.49920401202679f, 7.49792106467948f, 7.49794073151054f,
    7.49796038136060f, 7.49667790235710f, 7.49669751831102f, 7.49671711734975f,
    7.49673669949516f, 7.49545467185099f, 7.49547422027520f, 7.49549375187145f,
    7.49421219240963f, 7.49423169041509f, 7.49425117165763f, 7.49297008026330f,
    7.49298952804478f, 7.49300895912805f, 7.49302837353459f, 7.49174773343687f,
    7.49176711455430f, 7.49178647905926f, 7.49050630686120f, 7.49052563820524f,
    7.49054495300073f, 7.48926524858490f, 7.48928453034700f, 7.48930379562416f,
    7.48932304443751f, 7.48804379124302f, 7.48806300719200f, 7.48808220674033f,
    7.48680342115406f, 7.48682258796403f, 7.48684173843620f, 7.48556342033819f,
    7.48558253819737f, 7.48560163978127f, 7.48562072511066f, 7.48434285814650f,
    7.48436191102906f, 7.48438094771922f, 7.48310354806328f, 7.48312255243049f,
    7.48314154066708f, 7.48316051279358f, 7.48188356423386f, 7.48190250420160f,
    7.48192142812062f, 7.48064494668490f, 7.48066383856759f, 7.48068271446260f,
    7.47940670002678f, 7.47942554400726f, 7.47944437206081f, 7.47946318420761f,
    7.47818762076032f, 7.47820640115404f, 7.47822516570133f, 7.47824391442227f,
    7.47696880197852f, 7.47698751910672f, 7.47571288915281f, 7.47573157476804f,
    7.47575024465661f, 7.47576889883838f, 7.47449471975386f, 7.47451334258139f,
    7.47453194976145f, 7.47455054131374f, 7.47327681310897f, 7.47329537346471f,
    7.47202212722719f, 7.47204065646482f, 7.47205917017273f, 7.47207766837044f,
    7.47080487287198f, 7.47082334010767f, 7.47084179189150f, 7.47086022824284f,
    7.46958788348929f, 7.46960628903368f, 7.46962467920354f, 7.46835280053496f,
    7.46837116001347f, 7.46838950417511f, 7.46840783303904f, 7.46713640504500f,
    7.46715470337084f, 7.46717298645624f, 7.46719125432022f, 7.46592027700163f,
    7.46593851447977f, 7.46595673679339f, 7.46597494396140f, 7.46470441731737f,
    7.46472259425075f, 7.46345254840048f, 7.46347069517447f, 7.46348882689689f,
    7.46350694358646f, 7.46223734825421f, 7.46225543493411f, 7.46227350663713f,
    7.46229156338184f, 7.46102241856245f, 7.46104044544623f, 7.46105845742730f,
    7.45978977797966f, 7.45980776021071f, 7.45982572759437f, 7.45855751338129f,
    7.45857545112528f, 7.45859337407692f, 7.45861128225450f, 7.45734351838243f,
    7.45736139706664f, 7.45737926103147f, 7.45611196217244f, 7.45612979675298f,
    7.45614761666857f, 7.45616542193726f, 7.45489857332325f, 7.45491634935229f,
    7.45493411078849f, 7.45495185764981f, 7.45368545926693f, 7.45370317703227f,
    7.45372088027644f, 7.45245494659468f, 7.45247262084999f, 7.45249028063757f,
    7.45250792597517f, 7.45124244242089f, 7.45126005891167f, 7.45127766100555f,
    7.45129524872019f, 7.45003021527506f, 7.45004777428398f, 7.44878321952857f,
    7.44880074990199f, 7.44881826598392f, 7.44883576779183f, 7.43727548639956f,
    7.43729277382540f, 7.43731004718073f, 7.43732730648274f, 7.43606346612857f,
    7.43608069737530f, 7.43609791462004f, 7.43483453871790f, 7.43485172800980f,
    7.43486890335078f, 7.43488606475783f, 7.43362313924374f, 7.43364027283374f,
    7.43365739254057f, 7.43367449838109f, 7.43241202322571f, 7.43242910138409f,
    7.43244616572660f, 7.43118415468149f, 7.43120119144248f, 7.43121821443777f,
    7.43123522368404f, 7.42997366286784f, 7.42999064466599f, 7.43000761276498f,
    7.43002456718139f, 7.42876345656070f, 7.42878038366152f, 7.42879729712929f,
    7.42753665026940f, 7.42755353652043f, 7.42757040918772f, 7.42758726828765f,
    7.42632707148660f, 7.42634390350084f, 7.42636072199670f, 7.42637752699047f,
    7.42511778021092f, 7.42513455824921f, 7.42387528831073f, 7.42389203945833f,
    7.42390877718483f, 7.42392550150634f, 7.42266668138160f, 7.42268337894139f,
    7.42270006314441f, 7.42271673400666f, 7.42145836365485f, 7.42147500788352f,
    7.42149163881933f, 7.42023373145931f, 7.42025033585711f, 7.42026692700973f,
    7.42028350493299f, 7.41902604719678f, 7.41904259870875f, 7.41905913703874f,
    7.41907566220248f, 7.41781865404551f, 7.41783515292387f, 7.41785163868306f,
    7.41659509314730f, 7.41661155271502f, 7.41662799921040f, 7.41664443264901f,
    7.41538833653703f, 7.41540474390869f, 7.41542113827012f, 7.41543751963679f,
    7.41418187290033f, 7.41419822832378f, 7.41421457079872f, 7.41295938630599f,
    7.41297570292999f, 7.41299200665152f, 7.41300829748587f, 7.41175356220701f,
    7.41176982731279f, 7.41178607957713f, 7.41180231901523f, 7.41054803289840f,
    7.41056424672953f, 7.40931043523787f, 7.40932662352253f, 7.40934279905659f,
    7.40935896185510f, 7.40937511193310f, 7.40812173667013f, 7.40813786135211f,
    7.40815397335853f, 7.40817007270433f, 7.40691714638039f, 7.40693322044968f,
    7.40568076819554f, 7.40569681704778f, 7.40571285331374f, 7.40572887700822f,
    7.40574488814600f};

// Set of scalers, one per reverberation time, by which the
// |kHighCorrectionCurve| and |kLowCorrectionCurve| curves are multiplied in
// order to generate the correct onset compensation curves.
static const float kCurveCorrectionMultipliers[kNumFeedbackValues] = {
    0.000249027612622225f, 0.0475827382701585f, 0.0982856973675534f,
    0.144077780284872f,    0.187710331328442f,  0.229214552437128f,
    0.268761703298196f,    0.306384317687497f,  0.342116205684862f,
    0.376122154314109f,    0.408019265118341f,  0.438826303187204f,
    0.468213758592444f,    0.496269855121223f,  0.523103160344453f,
    0.548734562484123f,    0.573191744554386f,  0.596776538466997f,
    0.619556090502133f,    0.646894092669191f,  0.661870435136273f,
    0.675494733319630f,    0.701821797958321f,  0.713358543560036f,
    0.731743894313465f,    0.749694785147017f,  0.758967096040535f,
    0.775801399270244f,    0.792218438223182f,  0.808206900102085f,
    0.815142557606016f,    0.830420164791760f,  0.845380788487056f,
    0.859865023229817f,    0.865296767216337f,  0.879331838452952f,
    0.893145291831122f,    0.897272345317607f,  0.910373690471610f,
    0.923470208927849f,    0.936268613920194f,  0.940724814169454f,
    0.943721434222406f,    0.956465808252697f,  0.959628673835810f,
    0.971718784707243f,    0.974554123797995f,  0.977943937294408f,
    0.997154666216203f,    0.999999999999996f,  0.00847079214371323f,
    0.0104396896429595f,   0.0383165227008017f, 0.0405943838116957f,
    0.0685361482242258f,   0.0701917817593536f, 0.0713216738657547f,
    0.0986119509733789f,   0.0995868953874107f, 0.127034424762083f,
    0.127777689282131f,    0.128246886910136f,  0.155790763336654f,
    0.156203946276228f,    0.183007628720406f,  0.182664049029335f,
    0.182636700276451f,    0.209980982953741f,  0.209930104556972f,
    0.209208261790760f,    0.236016550641923f,  0.235475840865507f,
    0.234825009787230f,    0.262017587951252f,  0.260795540834379f,
    0.259744215394585f,    0.286998748503651f,  0.286344090244485f,
    0.285483640041615f,    0.284539872612409f,  0.311891500333310f,
    0.310660815240075f,    0.309428379806926f,  0.336752370009492f,
    0.335612040225831f,    0.334119868753728f,  0.332881268861172f,
    0.360256878080152f,    0.358861080534800f,  0.357458025447185f,
    0.356097715473274f,    0.383869923377680f,  0.382590514632971f,
    0.381266229326427f,    0.379680871429225f,  0.407164010760711f,
    0.405597486292448f,    0.404000115485411f,  0.402266215035868f,
    0.429891973669762f,    0.428453570844335f,  0.426999237668900f,
    0.425435248817597f,    0.450978617489268f,  0.449350785137532f,
    0.447825393407652f,    0.446242285152424f,  0.444699143006987f,
    0.471928084570479f,    0.471460865818911f,  0.469975899073161f,
    0.468110718285786f,    0.466267910972978f,  0.492953377248067f,
    0.491503646910692f,    0.490004860015654f,  0.488499316392171f,
    0.516893492513121f,    0.515545795838069f,  0.514220820105042f,
    0.512893659480599f,    0.511085398658644f,  0.509264930798830f,
    0.537621048957386f,    0.536165057418577f,  0.534661930692635f,
    0.533210489850429f,    0.531756722686728f,  0.530384047322116f,
    0.559038131029643f,    0.557822105594252f,  0.556569707547800f,
    0.555271214486959f,    0.553432792863170f,  0.551315743703168f,
    0.549392825464220f,    0.577982280854402f,  0.576562976619477f,
    0.575372635620591f,    0.574072847452013f,  0.572872688648164f,
    0.571573384448578f,    0.570329067593286f,  0.599518572118246f,
    0.598348121604011f,    0.597160846836955f,  0.595942913014291f,
    0.592262360278945f,    0.590949984551833f,  0.589596537782439f,
    0.618986568059361f,    0.617798845452536f,  0.616549708449253f,
    0.615384701753942f,    0.614192476100642f,  0.612892560298438f,
    0.611692826552139f,    0.610490163023757f,  0.609174362863638f,
    0.638463194928117f,    0.637028017578980f,  0.635559187798163f,
    0.633766413298784f,    0.632208978019593f,  0.630423678935153f,
    0.629100361498165f,    0.627683147303628f,  0.626307825147069f,
    0.656136973617732f,    0.655009501850360f,  0.653877493459184f,
    0.652665127999537f,    0.651558827307289f,  0.650386889118506f,
    0.649111048361781f,    0.648028207288007f,  0.646759703278198f,
    0.645519124438692f,    0.675436924850795f,  0.674280803735362f,
    0.673132340969860f,    0.671373346843144f,  0.670134946644868f,
    0.668731343028307f,    0.667428990512785f,  0.666412032897529f,
    0.665280139756495f,    0.664193037044164f,  0.694188573172643f,
    0.693133764391982f,    0.692048704636038f,  0.690982103678122f,
    0.689946461409433f,    0.688925040506954f,  0.687918657972205f,
    0.686814035317773f,    0.685752672736224f,  0.684629857976692f,
    0.683538212068733f,    0.682502742914018f,  0.712650390189218f,
    0.711488074001633f,    0.710454507990280f,  0.709369576447197f,
    0.708293103236850f,    0.707161341261432f,  0.706142174509451f,
    0.705109933856033f,    0.703938122923563f,  0.702882741466471f,
    0.701795805729764f,    0.700970588654440f,  0.699907893809526f,
    0.698848748944422f,    0.729459722297675f,  0.728559828909388f,
    0.727267491809098f,    0.726368916168934f,  0.725549425012621f,
    0.724701162719866f,    0.723729724234480f,  0.722805513390802f,
    0.721877511846754f,    0.720923464905931f,  0.720063488137338f,
    0.719192489800377f,    0.718234018896418f,  0.748886813504199f,
    0.748055970602122f,    0.747192096825651f,  0.746255128515039f,
    0.745432542499951f,    0.744527975028176f,  0.743657128539979f,
    0.742815568594195f,    0.742016966380996f,  0.741081662145480f,
    0.740212516669136f,    0.738752388065193f,  0.737838959848034f,
    0.737079784380805f,    0.736207833150401f,  0.735328920674480f,
    0.766147300368082f,    0.765383964612946f,  0.764598877501888f,
    0.763830319512426f,    0.763071341941492f,  0.762261754967287f,
    0.761421505873705f,    0.760683170206014f,  0.759876633831137f,
    0.759111799483625f,    0.758326333824362f,  0.757541464414185f,
    0.756805357349899f,    0.755956399690542f,  0.755084876598806f,
    0.754190942656102f,    0.753440741268157f,  0.752571082952576f,
    0.751825304285737f,    0.782969421663051f,  0.782195656629464f,
    0.781491207489953f,    0.780796925868243f,  0.780038270784649f,
    0.779327959705502f,    0.778571499532576f,  0.777927257784694f,
    0.777170483964829f,    0.776461606951210f,  0.775702452890459f,
    0.774964094372046f,    0.774261007484278f,  0.773530800147826f,
    0.772829253854413f,    0.772012428008542f,  0.771256939594450f,
    0.770263007078531f,    0.769608778166788f,  0.768868370726016f,
    0.800132675567284f,    0.799414457498671f,  0.798801140955651f,
    0.798201875508813f,    0.797562927111111f,  0.796973690523352f,
    0.796325305652599f,    0.795751914017312f,  0.795062104562951f,
    0.794392825660217f,    0.793833580415710f,  0.793228467372150f,
    0.792632556794021f,    0.791987407625815f,  0.791349198936522f,
    0.790758300136560f,    0.790155406100658f,  0.789502309522568f,
    0.788850663559873f,    0.788244230667833f,  0.787690453856242f,
    0.787119319515071f,    0.786556345654314f,  0.785871663518153f,
    0.817012925306181f,    0.816430511458010f,  0.815893193830343f,
    0.815310874396162f,    0.814695628192059f,  0.814349340761576f,
    0.813807195954924f,    0.812960708290203f,  0.812353982324263f,
    0.811760341393159f,    0.811129283770446f,  0.810570029805507f,
    0.810041997821215f,    0.809539025954020f,  0.808890057582255f,
    0.808376532093617f,    0.807788126711313f,  0.807229044495192f,
    0.806691599106826f,    0.806133405950557f,  0.805529337345578f,
    0.804943724604441f,    0.804359069634119f,  0.803604313367394f,
    0.803039009317611f,    0.802427016830483f,  0.801897188869670f,
    0.833460875990872f,    0.832877329385786f,  0.832411116216460f,
    0.831855344205032f,    0.831306648690096f,  0.830826737217004f,
    0.830306337776770f,    0.829837395963940f,  0.829322442611281f,
    0.828828529233458f,    0.828227895618978f,  0.827688095121686f,
    0.827224825179024f,    0.826744576227558f,  0.826190089131040f,
    0.825644348776801f,    0.825111095394709f,  0.824576204498035f,
    0.824095489895994f,    0.823551761115446f,  0.823018027544573f,
    0.822476448619841f,    0.821809962673621f,  0.821291765423256f,
    0.820650961031543f,    0.819911412926238f,  0.819376250438516f,
    0.818827507074517f,    0.818298634930539f,  0.817837741058990f,
    0.817377454006327f,    0.849050771870725f,  0.848598343034601f,
    0.848137761126842f,    0.847720880735919f,  0.847272612414349f,
    0.846839043139818f,    0.846310781171764f,  0.845917411128721f,
    0.845432276174650f,    0.845016330807497f,  0.844560412891491f,
    0.844113268000604f,    0.843717326097535f,  0.843254715180455f,
    0.842771923047896f,    0.842317456952523f,  0.841894102789524f,
    0.841478900234186f,    0.841073526044248f,  0.840678089199553f,
    0.840282881990563f,    0.839851728795932f,  0.839437044180938f,
    0.839032992924171f,    0.838674041436160f,  0.838208899506653f,
    0.837815592964711f,    0.837413586062667f,  0.836945051441415f,
    0.836579931119682f,    0.836103168868015f,  0.835691507888582f,
    0.835190824315056f,    0.834796921260984f,  0.834397547217396f,
    0.833770693471246f,    0.833382919978899f,  0.865249714600488f,
    0.864778611194893f,    0.864359035626832f,  0.863974109667504f,
    0.863599275330096f,    0.863175308582704f,  0.862750738409356f,
    0.862410936451282f,    0.862060660613492f,  0.861703069468698f,
    0.861338903148747f,    0.860962071126888f,  0.860613235617068f,
    0.860255115049830f,    0.859941728589139f,  0.859556867553160f,
    0.859129368056343f,    0.858719511777174f,  0.858375007251166f,
    0.857991574276656f,    0.857077372527853f,  0.856709203488606f,
    0.856338576783691f,    0.855985971077822f,  0.855629124525839f,
    0.855290191537388f,    0.854966165735379f,  0.854612291162095f,
    0.854327456524765f,    0.854015841793238f,  0.853649356402945f,
    0.853317984411307f,    0.852952982901721f,  0.852579010983093f,
    0.852262745823692f,    0.851907995774951f,  0.851574167335807f,
    0.851264191083115f,    0.850902019876735f,  0.850580011598579f,
    0.850225307767087f,    0.849838013922043f,  0.881877389068535f,
    0.881425634756149f,    0.881222935599140f,  0.880862887645329f,
    0.880558209099982f,    0.880114949740768f,  0.879806159434422f,
    0.879538932468558f,    0.879173454249472f,  0.878803642529878f,
    0.878488774313953f,    0.878152689099870f,  0.877787934867571f,
    0.877472544637065f,    0.877200689447213f,  0.876929820734263f,
    0.876616544942982f,    0.876339352367848f,  0.876037900545338f,
    0.875737826612020f,    0.875450927592556f,  0.875146075620635f,
    0.874847876858510f,    0.874531696275083f,  0.874232625014850f,
    0.873956654705092f,    0.873657406655665f,  0.873376193316924f,
    0.873085430435485f,    0.872758743496618f,  0.872421095892080f,
    0.872098708018361f,    0.871788288227254f,  0.871530138982730f,
    0.871236056832414f,    0.870910572694844f,  0.870635091498100f,
    0.870312602324108f,    0.870054759273025f,  0.869714810151487f,
    0.869366798437594f,    0.869122644903574f,  0.868812734247120f,
    0.868513864159224f,    0.868223756623966f,  0.867982715181480f,
    0.867671997828998f,    0.867407549259445f,  0.867121915940559f,
    0.866777786882641f,    0.866494871692684f,  0.866169558022502f,
    0.865824390903947f,    0.865583948922912f,  0.865260577587179f,
    0.897442560230671f,    0.897197975041446f,  0.896936601622887f,
    0.896639930442358f,    0.896358496055820f,  0.896149517619677f,
    0.895865666553406f,    0.895617470802557f,  0.895302338320577f,
    0.895008903956437f,    0.894766131546458f,  0.894431674742994f,
    0.894121880084823f,    0.893833250070113f,  0.893486559142867f,
    0.893208534201067f,    0.892971102121326f,  0.892730883606376f,
    0.892486495783999f,    0.892266606766457f,  0.891912616713162f,
    0.891668012816153f,    0.891389344583694f,  0.891127210236977f,
    0.890877014364885f,    0.890647919776319f,  0.890418186238544f,
    0.890123493822626f,    0.889882930209191f,  0.889638857058379f,
    0.889365113843686f,    0.889063011963076f,  0.888904806210479f,
    0.888595224427484f,    0.888348740136566f,  0.888084256999739f,
    0.887840636004391f,    0.887568183405399f,  0.887244627634201f,
    0.886968902204163f,    0.886813404979471f,  0.886525300589349f,
    0.886292070226496f,    0.886001016727455f,  0.885775223543453f,
    0.885522319161176f,    0.885169241885031f,  0.884891814369998f,
    0.884613124661923f,    0.884274453041876f,  0.884000697538040f,
    0.883779270240537f,    0.883552044352765f,  0.883321450052730f,
    0.883113458915741f,    0.882876096712848f,  0.882646681017793f,
    0.882443778020906f,    0.882173341068617f,  0.881932486077329f,
    0.881703097548580f,    0.881512430676287f,  0.881324988614956f,
    0.881078415830145f,    0.880836019375906f,  0.880592380876914f,
    0.880399045447751f,    0.880144055756519f,  0.879916264512867f,
    0.912262761203549f,    0.912050025877916f,  0.911837032776423f,
    0.911637002893336f,    0.911410664361064f,  0.911128179710946f,
    0.910915857381980f,    0.910713468187444f,  0.910514321228273f,
    0.910306658353875f,    0.910045217950504f,  0.909852480850540f,
    0.909661981811356f,    0.909400554409887f,  0.909267222071404f,
    0.909082328173087f,    0.908854885017709f,  0.908645654905872f,
    0.908429066044714f,    0.908203229057788f,  0.907975524348559f,
    0.907844076324775f,    0.907589558754535f,  0.907386017765108f,
    0.907146503676051f,    0.906950671939951f,  0.906734850193100f,
    0.906517924929938f,    0.906308309667034f,  0.906112492983346f,
    0.905983410335819f,    0.905742671195132f,  0.905504049165944f,
    0.905260974598286f,    0.905102165719060f,  0.904889117286156f,
    0.904704031490403f,    0.904530549352657f,  0.904335967396413f,
    0.904167284044056f,    0.903943525425265f,  0.903781177741016f,
    0.903561239085713f,    0.903395399945175f,  0.903235371390208f,
    0.903016653613898f,    0.902845104975041f,  0.902610542350492f,
    0.902398193708679f,    0.902225960341570f,  0.902021465390646f,
    0.901838424668229f,    0.901655758835068f,  0.901444820937381f,
    0.901245949833824f,    0.901069151471710f,  0.900915074109771f,
    0.900760359450201f,    0.900578847724700f,  0.900395965994494f,
    0.900228168805928f,    0.900066616036273f,  0.899902868316372f,
    0.899691672138911f,    0.899502275011599f,  0.899326630297481f,
    0.899147517398502f,    0.898891311929557f,  0.898737589588815f,
    0.898516468975794f,    0.898279435824810f,  0.898133379766176f,
    0.897940625694500f,    0.897769012071762f,  0.897584094034260f,
    0.897386399090749f,    0.897204596940482f,  0.897029990887014f,
    0.896843984043430f,    0.896659433456950f,  0.896499592362130f,
    0.896326359805770f,    0.896161841889783f,  0.896002962811382f,
    0.895858985647756f,    0.895679053342135f,  0.895472630838065f,
    0.895298117862088f,    0.927754563825053f,  0.927595758579098f,
    0.927423098901489f,    0.927274066423043f,  0.927064318376423f,
    0.926889227267958f,    0.926684097147831f,  0.926538641708822f,
    0.926383924569828f,    0.926221023888201f,  0.926069942094448f,
    0.925936867669842f,    0.925764476234829f,  0.925474258843906f,
    0.925341480568837f,    0.925184458584946f,  0.924997556397188f,
    0.924807778651479f,    0.924662367744565f,  0.924518831561345f,
    0.924374320422518f,    0.924239452756272f,  0.924035083031803f,
    0.923866605961103f,    0.923708308696909f,  0.923561861399422f,
    0.923342111379313f,    0.923181861833928f,  0.923022655494797f,
    0.922830553518427f,    0.922680580975269f,  0.922530343438704f,
    0.922379253084900f,    0.922203220902534f,  0.922017473475193f,
    0.921852509706996f,    0.921632506820494f,  0.921478764462909f,
    0.921328663166398f,    0.921188703978730f,  0.920977965994368f,
    0.920847607574230f,    0.920703577888137f,  0.920525889242200f,
    0.920380588508078f,    0.920253615844872f,  0.920139601102595f,
    0.920016687689548f,    0.919852868556918f,  0.919622503761834f,
    0.919458278730055f,    0.919295852055926f,  0.919134591570401f,
    0.918979062183651f,    0.918841010886602f,  0.918716051757939f,
    0.918569984788460f,    0.918369062451782f,  0.918215421152890f,
    0.918088828939449f,    0.917919780942217f,  0.917733211893808f,
    0.917574574952106f,    0.917438548196200f,  0.917323625618981f,
    0.917182002664427f,    0.917002289446623f,  0.916827583253820f,
    0.916714472811547f,    0.916594418971258f,  0.916460972969900f,
    0.916335756364354f,    0.916196649287878f,  0.916065572511741f,
    0.915947599776994f,    0.915818586940659f,  0.915632981571024f,
    0.915459212086808f,    0.915265459093725f,  0.915128178161501f,
    0.914998176624468f,    0.914860825831538f,  0.914625484964234f,
    0.914493843678021f,    0.914371779571637f,  0.914248620563387f,
    0.914100275148066f,    0.913965738470841f,  0.913826791405259f,
    0.913703250466401f,    0.913546026990571f,  0.913439303707573f,
    0.913292218567478f,    0.913182541953684f,  0.913036917749445f,
    0.912916054634268f,    0.912755642259817f,  0.912614205240910f,
    0.912468319200128f,    0.912352343404882f,  0.912236216864747f,
    0.912140734213469f,    0.911996828465493f,  0.911863354404087f,
    0.911709187715872f,    0.911569284462607f,  0.911451616353229f,
    0.911303512456831f,    0.911177218626202f,  0.911072557873086f,
    0.910948265059310f,    0.910791791835805f,  0.910670891976214f,
    0.910537919420196f,    0.910401546439265f,  0.910274713004068f,
    0.942972886116868f,    0.942873255216079f,  0.942736199328047f,
    0.942643396902211f,    0.942519604430019f,  0.942405794729767f,
    0.942276761652276f,    0.942183483677777f,  0.942062466352059f,
    0.941936635001252f,    0.941827790721750f,  0.941701752257506f,
    0.941569181480861f,    0.941428171184760f,  0.941266420007758f,
    0.941177983355581f,    0.941068131173925f,  0.940966459861221f,
    0.940802771917656f,    0.940699319454695f,  0.940546307666082f,
    0.940436629963075f,    0.940335105964667f,  0.940233168267479f,
    0.940070294791264f,    0.939946864568674f,  0.939806207003117f,
    0.939699917303889f,    0.939575570410864f,  0.939449267180147f,
    0.939335059915008f,    0.939221634003170f,  0.939100130210696f,
    0.938992079038466f,    0.938888993643325f,  0.938794272868458f,
    0.938679778519673f,    0.938565198624842f,  0.938447781743272f,
    0.938348887694662f,    0.938231674973109f,  0.938100618555889f,
    0.937929151200885f,    0.937821433643558f,  0.937688230231913f,
    0.937566420334848f,    0.937429830160073f,  0.937311443695009f,
    0.937190423215931f,    0.937063169325198f,  0.936979087920893f,
    0.936811325718775f,    0.936731671218813f,  0.936634805076269f,
    0.936532298789202f,    0.936416186739817f,  0.936286492122839f,
    0.936179817873514f,    0.936053037111203f,  0.935955439730240f,
    0.935819042190242f,    0.935683118325825f,  0.935582469083303f,
    0.935446994096024f,    0.935322108430966f,  0.935220912945627f,
    0.935131513738277f,    0.934997955601702f,  0.934903645484091f,
    0.934768972260724f,    0.934650110711938f,  0.934559208948897f,
    0.934458767641773f,    0.934334472974010f,  0.934235823798025f,
    0.934104732808613f,    0.934019917031945f,  0.933918957071512f,
    0.933828574280968f,    0.933739424110845f,  0.933624421615613f,
    0.933536789816350f,    0.933419733757098f,  0.933286841283564f,
    0.933217203121627f,    0.933126139166788f,  0.933030450650661f,
    0.932884936176351f,    0.932779450273416f,  0.932651017715157f,
    0.932545945362592f,    0.932455958346926f,  0.932337686419334f,
    0.932227058302166f,    0.932025539224666f,  0.931920795784066f,
    0.931852577738301f,    0.931756049103193f,  0.931621152063621f,
    0.931541102847978f,    0.931464901339151f,  0.931333487828105f,
    0.931222317831001f,    0.931145274335559f,  0.931048712065999f,
    0.930944549228741f,    0.930833966893561f,  0.930767031403833f,
    0.930661219955564f,    0.930573682517608f,  0.930490587789090f,
    0.930399861447101f,    0.930260043354527f,  0.930174757179721f,
    0.930082735120729f,    0.929980062024817f,  0.929881070592492f,
    0.929815269359710f,    0.929685646724316f,  0.929601626627662f,
    0.929475080077155f,    0.929384083536765f,  0.929293843820907f,
    0.929197517689133f,    0.929071496407846f,  0.928928767504129f,
    0.928850812213896f,    0.928755858560461f,  0.928683595051250f,
    0.928619097034910f,    0.928515473276713f,  0.928415904127483f,
    0.928296589209896f,    0.928156514230767f,  0.928075740622194f,
    0.927976281024834f,    0.927893758602397f,  0.927793223751259f,
    0.927533278933111f,    0.927443413613116f,  0.927330462282895f,
    0.927224518557079f,    0.927121263013897f,  0.927032513483020f,
    0.926946653005310f,    0.926852163943676f,  0.926782799964173f,
    0.926704377754193f,    0.926593212214682f,  0.926514271256028f,
    0.926434387396852f,    0.926338455745648f,  0.926241793129996f,
    0.926151243722815f,    0.926062446007553f,  0.925975437993879f,
    0.925900994878278f,    0.925769198338033f,  0.925673561554096f,
    0.925575663106531f,    0.925500150132689f,  0.958275427245807f,
    0.958217720942176f,    0.958088351980738f,  0.958006794628661f,
    0.957895379103873f,    0.957838037581917f,  0.957720979374284f,
    0.957627497340097f,    0.957500947647853f,  0.957418120264750f,
    0.957316257911455f,    0.957217318469398f,  0.957134253993810f,
    0.957060018503037f,    0.956986278198766f,  0.956895348950843f,
    0.956777435571824f,    0.956705582605570f,  0.956594026023218f,
    0.956516152335888f,    0.956357855496398f,  0.956298018292689f,
    0.956199349632202f,    0.956126323885033f,  0.956032137481781f,
    0.955946413187796f,    0.955847188872627f,  0.955754430213996f,
    0.955687353291728f,    0.955609174338888f,  0.955480819432032f,
    0.955422957826911f,    0.955337878180787f,  0.955277719747449f,
    0.955209090297961f,    0.955115833029069f,  0.955013029263148f,
    0.954929052626773f,    0.954858410226777f,  0.954780241057584f,
    0.954681910461255f,    0.954620468882399f,  0.954550468034354f,
    0.954461164292287f,    0.954381926562481f,  0.954303937678699f,
    0.954176727439442f,    0.954102874300900f,  0.953998321230092f,
    0.953890423951280f,    0.953784655913618f,  0.953722369794478f,
    0.953623105806388f,    0.953565738290012f,  0.953492683988373f,
    0.953413141506086f,    0.953326969854586f,  0.953262954351555f,
    0.953184013301923f,    0.953101894037007f,  0.953041902190311f,
    0.952969534053552f,    0.952869749867014f,  0.952789835366490f,
    0.952716435862726f,    0.952645254649320f,  0.952542492593948f,
    0.952463994814448f,    0.952349666425407f,  0.952275493737817f,
    0.952196002674587f,    0.952116683488068f,  0.952054020589248f,
    0.951971212262823f,    0.951862394841917f,  0.951768820782406f,
    0.951688868658540f,    0.951600614570797f,  0.951492713205816f,
    0.951425735970941f,    0.951361524070373f,  0.951238590334566f,
    0.951128486638037f,    0.951065909028615f,  0.951010270116159f,
    0.950902781595030f,    0.950831870621427f,  0.950771893006073f,
    0.950692220598345f,    0.950620597115025f,  0.950534510887917f,
    0.950403380584764f,    0.950334478890738f,  0.950272098021135f,
    0.950196260665432f,    0.950113227599961f,  0.950054926159529f,
    0.949969340742870f,    0.949881762321030f,  0.949768876353368f,
    0.949694926630816f,    0.949573394239407f,  0.949506256443609f,
    0.949443701324821f,    0.949394489146048f,  0.949300730229581f,
    0.949198332808624f,    0.949133287613908f,  0.949025351436334f,
    0.948942100841359f,    0.948893326812499f,  0.948797799093608f,
    0.948739418726981f,    0.948668659827771f,  0.948605293473731f,
    0.948508263335470f,    0.948422283102023f,  0.948334646705041f,
    0.948229838411611f,    0.948135977369838f,  0.948064621226811f,
    0.947975836227614f,    0.947869887504853f,  0.947787963676601f,
    0.947725593707634f,    0.947662889604619f,  0.947606582024487f,
    0.947529016389140f,    0.947467718279620f,  0.947393788461035f,
    0.947318309358938f,    0.947265244768804f,  0.947190494072727f,
    0.947120952789888f,    0.947051021658373f,  0.946947790362777f,
    0.946867304156026f,    0.946794285196121f,  0.946721140964183f,
    0.946641154869108f,    0.946546018566680f,  0.946455030201867f,
    0.946381856581210f,    0.946325274031613f,  0.946255337478520f,
    0.946177270118115f,    0.946111078958173f,  0.946023514156008f,
    0.945959212905784f,    0.945849080079382f,  0.945786239425211f,
    0.945714124061532f,    0.945658743681213f,  0.945601619947753f,
    0.945519484258529f,    0.945455400385708f,  0.945357119411477f,
    0.945264635751721f,    0.945201727920235f,  0.945141917936100f,
    0.945079684332379f,    0.945015928228411f,  0.944946280980823f,
    0.944877156498061f,    0.944804451411428f,  0.944739649858619f,
    0.944673589496270f,    0.944611136357352f,  0.944511223725923f,
    0.944424321938049f,    0.944379812730986f,  0.944289367399254f,
    0.944211374628178f,    0.944134509881182f,  0.944064570023833f,
    0.944020390576029f,    0.943935471478227f,  0.943859733313642f,
    0.943783658792479f,    0.943611497969536f,  0.943543421957604f,
    0.943499628090061f,    0.943413995369032f,  0.943307061234208f,
    0.943209211047420f,    0.943139391056753f,  0.943036473656225f,
    0.942967493742562f,    0.942890220197419f,  0.942826382100022f,
    0.942762325628337f,    0.942694266534490f,  0.942624647563169f,
    0.942553430443271f,    0.942488591032619f,  0.942370472457731f,
    0.942305639341609f,    0.942234519293505f,  0.942182960322720f,
    0.942120517409552f,    0.942027480395684f,  0.941952932339245f,
    0.941891750910681f,    0.941835281530227f,  0.941772951246597f,
    0.941725887717769f,    0.941683506530303f,  0.941593394184890f,
    0.941515512558213f,    0.941473311245300f,  0.941404233632176f,
    0.941325255534943f,    0.941251396941407f,  0.941202439199741f,
    0.941105194862420f,    0.941038074691643f,  0.940974096891109f,
    0.940919561213800f,    0.940827436688845f,  0.940746282858963f,
    0.940672810072720f,    0.940626178969142f,  0.940581240903949f,
    0.940525846543147f,    0.940475217956959f,  0.940413763147880f,
    0.940361426762319f,    0.940301758975821f,  0.940257761802955f,
    0.940172669790372f,    0.940105121519804f,  0.940055180311003f,
    0.940010615028732f,    0.939935007349598f,  0.939881036050097f,
    0.939814506904525f,    0.939749695798634f,  0.939696784747100f,
    0.939625052341671f,    0.939556008686751f,  0.939512250891858f,
    0.939433258280986f,    0.939307887545587f,  0.939263971661818f,
    0.939215889164479f,    0.939167376618607f,  0.939121752120582f,
    0.971967121890630f,    0.971911994887031f,  0.971870201384295f,
    0.971825989837210f,    0.971772117969243f,  0.971732534447172f,
    0.971671680117055f,    0.971613103848265f,  0.971568277310130f,
    0.971497005410360f,    0.971437965611784f,  0.971374177119527f,
    0.971311337696788f,    0.971265675503488f,  0.971211728334830f,
    0.971173654659489f,    0.971097115191358f,  0.971055546830150f,
    0.970984778394942f,    0.970938216168554f,  0.970854927036070f,
    0.970811611972044f,    0.970768836659056f,  0.970693057804116f,
    0.970619816488717f,    0.970549808560914f,  0.970503576286217f,
    0.970432624714843f,    0.970388770439317f,  0.970349084551768f,
    0.970298841533354f,    0.970239776951519f,  0.970194884492194f,
    0.970152831029440f,    0.970090756146688f,  0.970000755530803f,
    0.969956773446540f,    0.969911684742526f,  0.969822649120895f,
    0.969768380795225f,    0.969704307081910f,  0.969664862298864f,
    0.969522047689548f,    0.969481769085865f,  0.969436588593615f,
    0.969381675858440f,    0.969343203466823f,  0.969266922648030f,
    0.969211596747585f,    0.969146961317084f,  0.969103250668287f,
    0.969062089905576f,    0.968999137424737f,  0.968951901823687f,
    0.968911916700945f,    0.968839412499583f,  0.968786327157115f,
    0.968751440609257f,    0.968695995382998f,  0.968656550332251f,
    0.968616734717366f,    0.968567111115919f,  0.968522709698953f,
    0.968475903432075f,    0.968425439658996f,  0.968384501083956f,
    0.968344193764876f,    0.968302257085111f,  0.968251151994677f,
    0.968211894760266f,    0.968149794153557f,  0.968115330462226f,
    0.968048914429886f,    0.967991768898303f,  0.967957662757341f,
    0.967901867217052f,    0.967842068702658f,  0.967808096936220f,
    0.967746019972052f,    0.967668943461478f,  0.967635103661283f,
    0.967593146864882f,    0.967519676893017f,  0.967485969937633f,
    0.967444577221587f,    0.967394666397157f,  0.967359767192470f,
    0.967326238566736f,    0.967269924014565f,  0.967205488183220f,
    0.967150317626980f,    0.967112403335753f,  0.967079092591829f,
    0.967028848415111f,    0.966995625287745f,  0.966944244501088f,
    0.966911108413841f,    0.966837881450369f,  0.966804830861701f,
    0.966754806420632f,    0.966711974994014f,  0.966667104208814f,
    0.966634226055970f,    0.966584175955802f,  0.966508735173705f,
    0.966412422599927f,    0.966337465181747f,  0.966304793467667f,
    0.966245985853935f,    0.966195744077399f,  0.966162098544657f,
    0.966129595993283f,    0.966065142768099f,  0.966008381118610f,
    0.965958352967421f,    0.965907874942570f,  0.965860700300948f,
    0.965828446705695f,    0.965778827683499f,  0.965702907650509f,
    0.965666139199334f,    0.965634050335282f,  0.965562124832480f,
    0.965517675557660f,    0.965436730013674f,  0.965359948253595f,
    0.965303173457664f,    0.965250492014372f,  0.965212160348909f,
    0.965170781733245f,    0.965118350471165f,  0.965086664224710f,
    0.965045725545761f,    0.965008421977835f,  0.964951359793605f,
    0.964899196027095f,    0.964824463593891f,  0.964772498398080f,
    0.964741091666004f,    0.964670443460613f,  0.964607787072424f,
    0.964551286041044f,    0.964496863047423f,  0.964465651940309f,
    0.964429879490986f,    0.964398594355012f,  0.964351600338458f,
    0.964307369250915f,    0.964271608387851f,  0.964227417766565f,
    0.964171256711235f,    0.964081060807346f,  0.964045880993023f,
    0.963985242031280f,    0.963945647749731f,  0.963907826366924f,
    0.963868197486238f,    0.963799938452283f,  0.963765207634998f,
    0.963720978726527f,    0.963684133853149f,  0.963649040588658f,
    0.963607534831113f,    0.963551321580042f,  0.963497204430698f,
    0.963424714887615f,    0.963374725409701f,  0.963344439312775f,
    0.963310870684628f,    0.963276090213696f,  0.963222993617856f,
    0.963129610624736f,    0.963090851003960f,  0.963060656182045f,
    0.963002655525515f,    0.962963164663654f,  0.962922597291628f,
    0.962882895853952f,    0.962848459982529f,  0.962771647590578f,
    0.962711992610457f,    0.962654371383541f,  0.962616259829856f,
    0.962562805802114f,    0.962525034500491f,  0.962484652724640f,
    0.962439417576559f,    0.962379290577768f,  0.962332639787563f,
    0.962297429044519f,    0.962249888365194f,  0.962200268067524f,
    0.962140347398233f,    0.962107323902686f,  0.962041853572779f,
    0.962008176643193f,    0.961975646236443f,  0.961924711249088f,
    0.961879023273176f,    0.961831509403864f,  0.961775868784700f,
    0.961746822676295f,    0.961716324502158f,  0.961640369374947f,
    0.961611429245648f,    0.961547547285081f,  0.961509489301074f,
    0.961480654833700f,    0.961447331320540f,  0.961387391917649f,
    0.961341468430511f,    0.961304123251159f,  0.961249351920283f,
    0.961212476662826f,    0.961178912062857f,  0.961090639886684f,
    0.961045097435303f,    0.961003493534426f,  0.960967973701429f,
    0.960938077569799f,    0.960884289021088f,  0.960819067244493f,
    0.960780320069444f,    0.960743327089429f,  0.960698279588637f,
    0.960667663213218f,    0.960634721461931f,  0.960594821404335f,
    0.960541761992586f,    0.960508945187094f,  0.960469856554093f,
    0.960426570022417f,    0.960387981064035f,  0.960347680867052f,
    0.960307472352100f,    0.960253653207329f,  0.960218687435176f,
    0.960186390857838f,    0.960158615758153f,  0.960112507807049f,
    0.960071344004724f,    0.960020983326931f,  0.959987667027966f,
    0.959956805300620f,    0.959927767865339f,  0.959896946101205f,
    0.959833975596898f,    0.959804107476303f,  0.959775204077521f,
    0.959747793439038f,    0.959702341797309f,  0.959634363246830f,
    0.959569029679109f,    0.959533052569412f,  0.959501336485215f,
    0.959474119229892f,    0.959419839626233f,  0.959341023507977f,
    0.959295438847227f,    0.959260524216600f,  0.959227089969235f,
    0.959200063868628f,    0.959173070436206f,  0.959125094986595f,
    0.959090861914065f,    0.959045960373426f,  0.959019094445528f,
    0.958992260803902f,    0.958954605304605f,  0.958916769209807f,
    0.958886764667757f,    0.958841502703832f,  0.958814827566503f,
    0.958777559544421f,    0.958702397967575f,  0.958671995235790f,
    0.958641005586528f,    0.958614486425838f,  0.958577069623433f,
    0.958549172046238f,    0.958512135033094f,  0.958441537076639f,
    0.958415172644034f,    0.958365776019084f,  0.958303096769660f,
    0.958263716843560f,    0.958185028692631f,  0.958145375013881f,
    0.958119191626474f,    0.958068694182465f,  0.958018918920774f,
    0.957949774071843f,    0.957901198070889f,  0.957866533472984f,
    0.957836112980788f,    0.957808708199475f,  0.957782766016934f,
    0.957746388685411f,    0.957713730335926f,  0.957680192807017f,
    0.957652942114070f,    0.957612146941780f,  0.957579663093048f,
    0.957545516193684f,    0.957506903711571f,  0.957472054074152f,
    0.957436503578588f,    0.957395181689988f,  0.957359417445984f,
    0.957311450421340f,    0.957282679922514f,  0.957209067658546f,
    0.957175128902001f,    0.957146509039227f,  0.957112476412398f,
    0.957074501733838f,    0.957036984482748f,  0.956999759246114f,
    0.956970851060728f,    0.956927264285104f,  0.956871217307326f,
    0.956844594547104f,    0.956813890060049f,  0.956780501144467f,
    0.956744461342733f,    0.956714994383490f,  0.956665821298225f,
    0.956640789566225f,    0.956610849917976f,  0.956550012130147f,
    0.956496846132444f,    0.956456019909584f,  0.956407330143974f,
    0.956337082514063f,    0.956307879496556f,  0.956240737167033f,
    0.956198057147075f,    0.956160694192385f,  0.956121596304570f,
    0.956084736949496f,    0.956060067954893f,  0.956028844508730f,
    0.956004232084434f,    0.955958879345404f,  0.955924575071532f,
    0.955883193326281f,    0.955847442906133f,  0.955813069719572f,
    0.955772060592762f,    0.955716387710303f,  0.955685728196237f,
    0.955645039455566f,    0.955586534578155f,  0.955554652983814f,
    0.955528968538545f,    0.955494147326420f,  0.955433145575848f,
    0.955398247978298f,    0.955366951631532f,  0.955341405225951f,
    0.955313054458997f,    0.955274786773442f,  0.955242793028928f,
    0.955217357860814f,    0.955166187024662f,  0.955135160237832f,
    0.955097685925402f,    0.955066290889999f,  0.955040052475596f,
    0.955002923204769f,    0.954979072001785f,  0.954946002019177f,
    0.954922204505984f,    0.954894464435323f,  0.954869328998638f,
    0.954822936955329f,    0.954799245855306f,  0.954763225287706f,
    0.954739587182904f,    0.954702687836680f,  0.954650651504783f,
    0.954627091600106f,    0.954586735039616f,  0.954543509103832f,
    0.954487508586822f,    0.954455423137410f,  0.954431992380845f,
    0.954395246407735f,    0.954368158761341f,  0.954328519361469f,
    0.954305192349057f,    0.954278682066255f,  0.954244790518497f,
    0.954210949137707f,    0.954183768594435f,  0.954155136691491f,
    0.954118315432295f,    0.954088453549574f,  0.954056625115569f,
    0.954015559093910f,    0.953982138679812f,  0.953953458629932f,
    0.953924167157067f,    0.953876244476431f,  0.953843277247482f,
    0.953820332674190f,    0.953783435648809f,  0.986922760337215f,
    0.986893347132258f,    0.986853719027791f,  0.986802177140835f,
    0.986780595774923f,    0.986735948250994f,  0.986714413193494f,
    0.986691544524921f,    0.986650126059355f,  0.986617659263719f,
    0.986574768928646f,    0.986553349155405f,  0.986509490127068f,
    0.986488116171129f,    0.986459808203524f,  0.986425749277674f,
    0.986401282919961f,    0.986377715956588f,  0.986347680503076f,
    0.986320259258079f,    0.986271810607144f,  0.986245050428010f,
    0.986223881966244f,    0.986176681328462f,  0.986145917556990f,
    0.986110482684939f,    0.986085509969948f,  0.986055947339547f,
    0.986034913700289f,    0.986000776580319f,  0.985965492876594f,
    0.985919261677516f,    0.985892883092507f,  0.985830609056223f,
    0.985801336195789f,    0.985779112009499f,  0.985758255139701f,
    0.985733238386015f,    0.985712426258423f,  0.985655501978720f,
    0.985626353966535f,    0.985590903455782f,  0.985563563636873f,
    0.985523439019124f,    0.985500817867769f,  0.985474456307468f,
    0.985449609353800f,    0.985405907406883f,  0.985360059543588f,
    0.985310904032878f,    0.985270714901873f,  0.985226727495785f,
    0.985193263279922f,    0.985172811115538f,  0.985115330253415f,
    0.985063087425340f,    0.985008336715826f,  0.984978613966300f,
    0.984941852360105f,    0.984912509332049f,  0.984850180222146f,
    0.984824312074071f,    0.984799854405289f,  0.984753768215678f,
    0.984732561511123f,    0.984706135723891f,  0.984678185530603f,
    0.984655684360588f,    0.984631067011433f,  0.984587731589628f,
    0.984558318722489f,    0.984523715071264f,  0.984502265042700f,
    0.984481759317740f,    0.984445908996804f,  0.984416448023547f,
    0.984384996037713f,    0.984346921353555f,  0.984323353794763f,
    0.984302492135683f,    0.984234826486034f,  0.984229190516759f,
    0.984189170519958f,    0.984154292087286f,  0.984130608803616f,
    0.984094955038999f,    0.984074738744969f,  0.984037558142149f,
    0.983994580486844f,    0.983967885641745f,  0.983943737430130f,
    0.983915944574975f,    0.983884236637719f,  0.983864172195582f,
    0.983831831959887f,    0.983802938958836f,  0.983775833669071f,
    0.983753187871529f,    0.983730105132050f,  0.983703133254244f,
    0.983669828705679f,    0.983644384789753f,  0.983620626656529f,
    0.983600307731900f,    0.983567972476239f,  0.983543210140435f,
    0.983491238528487f,    0.983454493564921f,  0.983367632603668f,
    0.983347051720359f,    0.983327810408723f,  0.983301876305209f,
    0.983276092919806f,    0.983251755401310f,  0.983232135834041f,
    0.983208069222592f,    0.983172240353604f,  0.983147868868929f,
    0.983081868982781f,    0.983045201512550f,  0.983012533352780f,
    0.982993510710699f,    0.982961813650806f,  0.982942002878203f,
    0.982889024861584f,    0.982867015386914f,  0.982847649649163f,
    0.982824371463141f,    0.982805048997046f,  0.982773218775641f,
    0.982748418352122f,    0.982729158100405f,  0.982703997651039f,
    0.982682108753123f,    0.982655505774959f,  0.982623018430868f,
    0.982593842523021f,    0.982573406804049f,  0.982547353367972f,
    0.982519585224023f,    0.982459288284645f,  0.982436394370087f,
    0.982415125919954f,    0.982387582740862f,  0.982367292542386f,
    0.982334965662622f,    0.982312635851139f,  0.982294117369415f,
    0.982251715517260f,    0.982224369337035f,  0.982205907157262f,
    0.982182761751450f,    0.982159903255411f,  0.982126373095434f,
    0.982107184687307f,    0.982084954430489f,  0.982041099920316f,
    0.982022769212310f,    0.981977331062889f,  0.981939690609133f,
    0.981914923334885f,    0.981856360333239f,  0.981834102582585f,
    0.981809598272734f,    0.981770121636027f,  0.981739618531661f,
    0.981717245565463f,    0.981683230681818f,  0.981652689781140f,
    0.981634196393471f,    0.981613495190574f,  0.981564013372326f,
    0.981524277410168f,    0.981498926071219f,  0.981450560157412f,
    0.981425266998042f,    0.981402192708996f,  0.981384225827395f,
    0.981365906020084f,    0.981343753206944f,  0.981300572619224f,
    0.981282678369968f,    0.981264437359234f,  0.981228534383292f,
    0.981209049239098f,    0.981182762882197f,  0.981158372244174f,
    0.981134700930964f,    0.981112259373601f,  0.981094152386932f,
    0.981068500570816f,    0.981036844052188f,  0.980986685235874f,
    0.980968652491355f,    0.980913591726037f,  0.980889441073170f,
    0.980848240136851f,    0.980807605621403f,  0.980774106571861f,
    0.980755251245704f,    0.980729769330780f,  0.980680692805421f,
    0.980654718270742f,    0.980636884095367f,  0.980615965819145f,
    0.980595938595629f,    0.980561974203509f,  0.980496049774783f,
    0.980478290932049f,    0.980460550538442f,  0.980432150774978f,
    0.980410648593876f,    0.980393295011126f,  0.980357014272223f,
    0.980326248344638f,    0.980299534184272f,  0.980264913469273f,
    0.980247317913468f,    0.980230066117568f,  0.980208414358917f,
    0.980187260862336f,    0.980132312570413f,  0.980109049768295f,
    0.980075371162398f,    0.980051209784426f,  0.980031865718607f,
    0.979994042174202f,    0.979937198188725f,  0.979920111293897f,
    0.979898807860237f,    0.979877564940339f,  0.979856737310176f,
    0.979835492266172f,    0.979818488652277f,  0.979774221453093f,
    0.979745444849853f,    0.979727225403310f,  0.979700314470918f,
    0.979683392321718f,    0.979666182005815f,  0.979611629473783f,
    0.979590852554355f,    0.979573692364784f,  0.979556851511248f,
    0.979534389742842f,    0.979506478571303f,  0.979478322317171f,
    0.979455398954295f,    0.979438638890386f,  0.979412895074252f,
    0.979392385523425f,    0.979371783642596f,  0.979355088151969f,
    0.979333607767233f,    0.979313189453699f,  0.979292658573124f,
    0.979263179385256f,    0.979246564075690f,  0.979198755309747f,
    0.979174516662574f,    0.979146797165384f,  0.979123464872970f,
    0.979106928065550f,    0.979069685325203f,  0.979053179759749f,
    0.979030332693243f,    0.978986657751362f,  0.978970198965351f,
    0.978946974036735f,    0.978926682675456f,  0.978905244289410f,
    0.978862765525649f,    0.978846097879915f,  0.978821788954481f,
    0.978803246624047f,    0.978766048702116f,  0.978746831450756f,
    0.978725426164543f,    0.978709138346453f,  0.978685752931826f,
    0.978669496205975f,    0.978649410758398f,  0.978624780793713f,
    0.978608570268773f,    0.978592094654600f,  0.978550920367351f,
    0.978513470662572f,    0.978497042038316f,  0.978480907605864f,
    0.978464788541799f,    0.978434611643655f,  0.978418522841283f,
    0.978390826033207f,    0.978374767625467f,  0.978353654444658f,
    0.978337626389741f,    0.978305022211732f,  0.978282789753918f,
    0.978266806773291f,    0.978234606202049f,  0.978203779970373f,
    0.978184787014873f,    0.978165052701335f,  0.978112832751907f,
    0.978096668038331f,    0.978058855230544f,  0.978042990406293f,
    0.978026872130719f,    0.978008154881375f,  0.977992335006128f,
    0.977968977528089f,    0.977951947084624f,  0.977932854511879f,
    0.977913681988638f,    0.977897936222764f,  0.977878414356271f,
    0.977862698102290f,    0.977844552571458f,  0.977825081855045f,
    0.977804621661370f,    0.977787726784343f,  0.977768071434286f,
    0.977752443360056f,    0.977733052522146f,  0.977717453632544f,
    0.977698822040011f,    0.977673199611945f,  0.977619793686062f,
    0.977602075088056f,    0.977576669465553f,  0.977542954045465f,
    0.977527455035169f,    0.977504152387557f,  0.977488682089939f,
    0.977472970424132f,    0.977448260145925f,  0.977398011663338f,
    0.977372297882134f,    0.977355666246754f,  0.977330698924145f,
    0.977298426353436f,    0.977283068966841f,  0.977250107782567f,
    0.977207119451928f,    0.977191803723299f,  0.977176502215050f,
    0.977161214907559f,    0.977102300457946f,  0.977063703061534f,
    0.977040908723148f,    0.977025676110134f,  0.977004557438028f,
    0.976989105443786f,    0.976970178122162f,  0.976938120674094f,
    0.976922957790298f,    0.976906582534861f,  0.976891202359989f,
    0.976872365503571f,    0.976857258525261f,  0.976833112633702f,
    0.976815174516876f,    0.976791566318235f,  0.976756563193258f,
    0.976741524718733f,    0.976698317437522f,  0.976651346433524f,
    0.976632645407126f,    0.976617419614971f,  0.976595534775329f,
    0.976576881856942f,    0.976561937972295f,  0.976546768986203f,
    0.976528820169054f,    0.976508065016963f,  0.976482720813795f,
    0.976467844882108f,    0.976446124920447f,  0.976403186250199f,
    0.976388350299752f,    0.976349887902191f,  0.976319445660324f,
    0.976304414770861f,    0.976289632025076f,  0.976264145624284f,
    0.976240532650802f,    0.976209273788994f,  0.976194311547957f,
    0.976179595220112f,    0.976161231562614f,  0.976146541915870f,
    0.976123701179350f,    0.976105154089893f,  0.976082960591513f,
    0.976042090614826f,    0.976026251263904f,  0.976007993676071f,
    0.975989338013885f,    0.975962781592164f,  0.975944570379698f,
    0.975930011937128f,    0.975907027483125f,  0.975888636596245f,
    0.975874117446146f,    0.975858399012390f,  0.975814767533837f,
    0.975770098214803f,    0.975755629917150f,  0.975740950932038f,
    0.975700126458313f,    0.975679056973671f,  0.975664640074239f,
    0.975650236167557f,    0.975580150537865f,  0.975564561643136f,
    0.975550195260922f,    0.975511963306388f,  0.975491182846468f,
    0.975476854410694f,    0.975458937949595f,  0.975441037900350f,
    0.975403340889900f,    0.975378834602348f,  0.975354759608012f,
    0.975337489516777f,    0.975321825596625f,  0.975307598487264f,
    0.975283675623818f,    0.975269473727394f,  0.975250515677110f,
    0.975222503407069f,    0.975208339209176f,  0.975185839227530f,
    0.975164426313621f,    0.975150299633837f,  0.975132834025883f,
    0.975114957906684f,    0.975095028384530f,  0.975080951799970f,
    0.975066887752846f,    0.975041741905158f,  0.975027702605693f,
    0.975005031349011f,    0.974987469475237f,  0.974973467389524f,
    0.974937686203278f,    0.974920168212932f,  0.974902828972747f,
    0.974872881920399f,    0.974855407841305f,  0.974841479047993f,
    0.974824552649550f,    0.974807122649889f,  0.974793024340947f,
    0.974779144750431f,    0.974761758864481f,  0.974743009828295f,
    0.974702179104436f,    0.974688347653034f,  0.974654844155268f,
    0.974629896769268f,    0.974609196289333f,  0.974595412768757f,
    0.974578144647479f,    0.974554761535420f,  0.974512105515984f,
    0.974493684651349f,    0.974479960679643f,  0.974445019848412f,
    0.974420342456999f,    0.974406653869030f,  0.974389302704651f,
    0.974375638158094f,    0.974356380485032f,  0.974342739773739f,
    0.974329111024600f,    0.974308667973371f,  0.974295062936910f,
    0.974277268860387f,    0.974257232359766f,  0.974243662862119f,
    0.974226418880396f,    0.974209427476104f,  0.974160143649720f,
    0.974135024493451f,    0.974121512976261f,  0.974108013252395f,
    0.974080630570305f,    0.974065964331530f,  0.974015781527702f,
    0.974002134457994f,    0.973968181110526f,  0.973941430008605f,
    0.973900624962740f,    0.973887216439222f,  0.973870418549401f,
    0.973850724231017f,    0.973837350500092f,  0.973820594740841f,
    0.973798522327485f,    0.973777807158058f,  0.973762348240277f,
    0.973720684603163f,    0.973707190440952f,  0.973690525402443f,
    0.973642098553068f,    0.973616296682685f,  0.973596682908872f,
    0.973574106353138f,    0.973560681801021f,  0.973542922676264f,
    0.973514709802091f,    0.973501505700315f,  0.973484979448508f,
    0.973471798042410f,    0.973456500215393f,  0.973431510917770f,
    0.973406975669394f,    0.973387629418889f,  0.973367520795037f,
    0.973303609216807f,    0.973284719314881f,  0.973267149528169f,
    0.973250597568072f,    0.973237526949800f,  0.973221187129731f,
    0.973199716550181f,    0.973183406462760f,  0.973164990751466f,
    0.973148718186700f,    0.973132284540874f,  0.973116049604857f,
    0.973096165761680f,    0.973079968266462f,  0.973062611300583f,
    0.973011998962548f,    0.972975003152676f,  0.972962075047062f,
    0.972945970132927f,    0.972923360946523f,  0.972900115256482f,
    0.972870276402172f,    0.972833571656061f,  0.972811347406848f,
    0.972777650857546f,    0.972760603364386f,  0.972744749946391f,
    0.972728915316704f,    0.972491425539914f,  0.972700300574877f,
    0.972657493357715f,    0.972632608746347f,  0.972616859264461f,
    0.972598619191692f,    0.972345186679530f,  0.972332446611726f,
    0.972312645617935f,    0.972296953045242f,  0.972283079206423f,
    0.972354259937865f,    0.972459209202242f,  0.972446537807611f,
    0.972430934483078f,    0.972176086196692f,  0.972163441757064f,
    0.972150807994720f,    0.972135323233078f,  0.972332090264623f,
    0.972296656512621f,    0.972281150355395f,  0.972261199146408f,
    0.972040937679384f,    0.972020853805726f,  0.971986352971951f,
    0.971970171296574f,    0.971957642162397f,  0.971945123560269f,
    0.971925366207946f,    0.971903023702926f,  0.971890536316386f,
    0.971869250638017f,    0.971845667162464f,  0.971833210739844f,
    0.971804277862056f,    0.971791841948099f,  0.971779416449271f,
    0.971764022143080f,    0.971747980164594f,  0.971735585761423f,
    0.971717421387968f,    0.971705047573062f,  0.971692684095817f,
    0.971677569616385f,    0.971654562083561f,  0.971639423267713f,
    0.971627100732641f,    0.971586709457336f,  0.971574406851170f,
    0.971562114493840f,    0.971547034241681f,  0.971528565510735f,
    0.971512518293816f,    0.971500266599958f,  0.971464929056654f,
    0.971452697226202f,    0.971418394914989f,  0.971406182921065f,
    0.971393981063291f,    0.971376907284318f,  0.971343341781312f,
    0.971331169765085f,    0.971319007835423f,  0.971298578641970f,
    0.971283662437030f,    0.971271530457587f,  0.971252454892243f,
    0.971240342827092f,    0.971225487479047f,  0.971204546317460f,
    0.971192464074439f,    0.971180391807126f,  0.971165370878734f,
    0.971153318463628f,    0.971140124638148f,  0.971123237224655f,
    0.971106480118568f,    0.971083157368327f,  0.971071154129043f,
    0.971034062159235f,    0.971018455159871f,  0.971006480918128f,
    0.970994516519120f,    0.970977092320320f,  0.970965003529432f,
    0.970950326173765f,    0.970937252551404f,  0.970925337052152f,
    0.970913431323699f,    0.970898580575337f,  0.970880595387861f,
    0.970865984305819f,    0.970854117296896f,  0.970827109552462f,
    0.970787121116513f,    0.970771664135147f,  0.970734868566354f,
    0.970720322134539f,    0.970708511741481f,  0.970696710990446f,
    0.970670091597675f,    0.970638391535586f,  0.970594506910305f,
    0.970571972858635f,    0.970560218611173f,  0.970545523090248f,
    0.970524813815583f,    0.970506896143813f,  0.970470838904634f,
    0.970456283769786f,    0.970444585924069f,  0.970432897582902f,
    0.970421218734771f,    0.970408406937207f,  0.970396747017460f,
    0.970385096556351f,    0.970370753011372f,  0.970353198100957f,
    0.970341440710966f,    0.970329827852595f,  0.970315487202049f,
    0.970303893080486f,    0.970269092080499f,  0.970251821409422f,
    0.970240254808422f,    0.970228697553167f,  0.970204163935976f,
    0.970191352781056f,    0.970179823267678f,  0.970165616505559f,
    0.970154105529172f,    0.970125646226645f,  0.970114153457064f,
    0.970102669944216f,    0.970087197011956f,  0.970075731899408f,
    0.970061466817810f,    0.970050020142580f,  0.970019988110384f,
    0.969972599795979f,    0.969961179607361f,  0.969948632115153f,
    0.969908746684612f,    0.969894682809931f,  0.969883170119172f,
    0.969871795082014f,    0.969860429159958f,  0.969846343221063f,
    0.969834995442697f,    0.969823656746862f,  0.969812327122728f,
    0.969798343939773f,    0.969787032374211f,  0.969772004681985f,
    0.969760711166196f,    0.969749426668237f,  0.969688426500697f,
    0.969677159032194f,    0.969663245949753f,  0.969651996391660f,
    0.969625257076576f,    0.969605977523474f,  0.969594754369765f,
    0.969583416110931f,    0.969572210865374f,  0.969561014532258f,
    0.969547180480629f,    0.969513246925795f,  0.969502076773797f,
    0.969485003257335f,    0.969472720232916f,  0.969461576525169f,
    0.969450319795174f,    0.969439193805961f,  0.969420426798349f,
    0.969406597238909f,    0.969395497473808f,  0.969376382415080f,
    0.969365300059954f,    0.969354226471582f,  0.969343161639623f,
    0.969325363726951f,    0.969314316299151f,  0.969299688455677f,
    0.969287530932357f,    0.969276509560222f,  0.969265496882931f,
    0.969254492890265f,    0.969243497572002f,  0.969229888366585f,
    0.969216720237517f,    0.969205750840223f,  0.969191858846575f,
    0.969172668560500f,    0.969153335935209f,  0.969142400641392f,
    0.969126015512418f,    0.969115097292469f,  0.969077096987749f,
    0.969066195393082f,    0.969054177875027f,  0.969035207922252f,
    0.969016820085756f,    0.969005952381187f,  0.968992380215632f,
    0.968981529478561f,    0.968968080712556f,  0.968955177833115f,
    0.968944352461800f,    0.968926930585352f,  0.968913081597044f,
    0.968902281458119f,    0.968891489754488f,  0.968877127176801f,
    0.968865230815253f,    0.968851865757681f,  0.968841107561196f,
    0.968827062840517f,    0.968816321345894f,  0.968790935192768f,
    0.968768188742240f,    0.968757471871783f,  0.968732819132344f,
    0.968722118670559f,    0.968708836038462f,  0.968698152157404f,
    0.968657086013096f,    0.968646418161465f,  0.968624158304073f,
    0.968613506791441f,    0.968596572545570f,  0.968579061738822f,
    0.968561014979841f,    0.968550396126364f,  0.968537202998056f,
    0.968526600536290f,    0.968516006279852f,  0.968505420219268f,
    0.968494842345072f,    0.968484165957274f,  0.968448598566619f,
    0.968438044781854f,    0.968427499146142f,  0.968390635470169f,
    0.968375420846665f,    0.968364899038218f,  0.968354385341824f,
    0.968332577679763f,    0.968322079974605f,  0.968311590353718f,
    0.968301004306507f,    0.968290530868138f,  1.00143961392577f,
    1.00142968724369f,     1.00141976816660f,   1.00140446209241f,
    1.00137317954622f,     1.00136328275028f,   1.00135014137470f,
    1.00133902200513f,     1.00131146903538f,   1.00130160207297f,
    1.00128450674836f,     1.00127465472096f,   1.00126481021308f,
    1.00125497321619f,     1.00124225940996f,   1.00122500973736f,
    1.00121519503829f,     1.00120538781624f,   1.00119545326034f,
    1.00118313784023f,     1.00117335295109f,   1.00116357550511f,
    1.00115380549385f,     1.00113942387171f,   1.00112966862280f,
    1.00109314412405f,     1.00107946844994f,   1.00106973486180f,
    1.00105418133174f,     1.00104229664803f,   1.00103006985615f,
    1.00102036557219f,     1.00100778834931f,   1.00096616031686f,
    1.00095647745121f,     1.00092507998694f,   1.00091541138654f,
    1.00090575009718f,     1.00089487173027f,   1.00087920118928f,
    1.00086956165688f,     1.00084041349058f,   1.00082467685478f,
    1.00080693845908f,     1.00079732740243f,   1.00078092556175f,
    1.00074906862448f,     1.00073947877630f,   1.00072989615011f,
    1.00072019315343f,     1.00066830628723f,   1.00065874454301f,
    1.00064918998899f,     1.00063246831712f,   1.00061832499982f,
    1.00060879176472f,     1.00059926568779f,   1.00058974676105f,
    1.00058023497646f,     1.00055457310189f,   1.00054507529979f,
    1.00049419893387f,     1.00048471465008f,   1.00044437316419f,
    1.00043490254958f,     1.00042341268056f,   1.00039435986397f,
    1.00038491011887f,     1.00037546743804f,   1.00036303684998f,
    1.00034636152703f,     1.00033693981721f,   1.00032504191268f,
    1.00031563421870f,     1.00030623354207f,   1.00029683987498f,
    1.00028352575811f,     1.00026603619147f,   1.00025666330226f,
    1.00024729739164f,     1.00023128713687f,   1.00022193504773f,
    1.00020066879491f,     1.00019133040773f,   1.00017618304649f,
    1.00016577323552f,     1.00015645552966f,   1.00013420200033f,
    1.00009861649226f,     1.00008931885617f,   1.00008002811490f,
    1.00007062639745f,     1.00005669782209f,   1.00004496048148f,
    1.00003569712223f,     1.00002644061987f,   1.00000924294396f,
    0.999999999999995f};

// Curve which is scaled by |kCurveCorrectionMultipliers| and added to the curve
// generated by the |kHighReverberationCorrectionCurve| polynomial, for
// reverberation times which result in a feedback factor index greater than
// |kCurveChangeoverIndex|.
static const float kHighCorrectionCurve[kCorrectionCurveLength] = {
    0.00905424704513036f,  0.00926599025691877f,  0.00947778313262726f,
    0.00968962562016418f,  0.00990151766744984f,  0.0101134592224174f,
    0.0103254502330116f,   0.0105374906471899f,   0.0107495804129221f,
    0.0109617194781898f,   0.0111739077909878f,   0.0113861452993219f,
    0.0115984319512115f,   0.0118107676946871f,   0.0120231524777923f,
    0.0122355862485828f,   0.0124480689551263f,   0.0126606005455030f,
    0.0128731809678055f,   0.0130858101701389f,   0.0132984881006193f,
    0.0135112147073766f,   0.0137239899385524f,   0.0139368137423006f,
    0.0141496860667869f,   0.0143626068601902f,   0.0145755760707011f,
    0.0147885936465225f,   0.0150016595358697f,   0.0152147736869704f,
    0.0154279360480643f,   0.0156411465674036f,   0.0158544051932527f,
    0.0160677118738880f,   0.0162810665575990f,   0.0164944691926864f,
    0.0167079197274641f,   0.0169214181102577f,   0.0171349642894055f,
    0.0173485582132578f,   0.0175621998301772f,   0.0177758890885386f,
    0.0179896259367293f,   0.0182034103231487f,   0.0184172421962087f,
    0.0186311215043333f,   0.0188450481959587f,   0.0190590222195339f,
    0.0192730435235194f,   0.0194871120563886f,   0.0197012277666270f,
    0.0199153906027323f,   0.0201296005132146f,   0.0203438574465959f,
    0.0205581613514112f,   0.0207725121762073f,   0.0209869098695431f,
    0.0212013543799902f,   0.0214158456561324f,   0.0216303836465656f,
    0.0218449682998980f,   0.0220595995647505f,   0.0222742773897556f,
    0.0224890017235585f,   0.0227037725148169f,   0.0229185897122002f,
    0.0231334532643903f,   0.0233483631200817f,   0.0235633192279807f,
    0.0237783215368064f,   0.0239933699952896f,   0.0242084645521738f,
    0.0244236051562147f,   0.0246387917561803f,   0.0248540243008508f,
    0.0250693027390185f,   0.0252846270194886f,   0.0254999970910778f,
    0.0257154129026155f,   0.0259308744029435f,   0.0261463815409154f,
    0.0263619342653978f,   0.0265775325252690f,   0.0267931762694196f,
    0.0270088654467529f,   0.0272246000061842f,   0.0274403798966409f,
    0.0276562050670630f,   0.0278720754664027f,   0.0280879910436245f,
    0.0283039517477049f,   0.0285199575276330f,   0.0287360083324103f,
    0.0289521041110502f,   0.0291682448125785f,   0.0293844303860333f,
    0.0296006607804654f,   0.0298169359449371f,   0.0300332558285236f,
    0.0302496203803120f,   0.0304660295494019f,   0.0306824832849052f,
    0.0308989815359457f,   0.0311155242516602f,   0.0313321113811973f,
    0.0315487428737179f,   0.0317654186783949f,   0.0319821387444144f,
    0.0321989030209739f,   0.0324157114572834f,   0.0326325640025653f,
    0.0328494606060544f,   0.0330664012169974f,   0.0332833857846536f,
    0.0335004142582945f,   0.0337174865872039f,   0.0339346027206777f,
    0.0341517626080246f,   0.0343689661985646f,   0.0345862134416310f,
    0.0348035042865692f,   0.0350208386827363f,   0.0352382165795020f,
    0.0354556379262483f,   0.0356731026723698f,   0.0358906107672728f,
    0.0361081621603763f,   0.0363257568011115f,   0.0365433946389216f,
    0.0367610756232627f,   0.0369787997036024f,   0.0371965668294212f,
    0.0374143769502114f,   0.0376322300154781f,   0.0378501259747384f,
    0.0380680647775213f,   0.0382860463733691f,   0.0385040707118353f,
    0.0387221377424862f,   0.0389402474149004f,   0.0391583996786689f,
    0.0393765944833944f,   0.0395948317786927f,   0.0398131115141910f,
    0.0400314336395295f,   0.0402497981043605f,   0.0404682048583482f,
    0.0406866538511695f,   0.0409051450325135f,   0.0411236783520815f,
    0.0413422537595872f,   0.0415608712047565f,   0.0417795306373275f,
    0.0419982320070504f,   0.0422169752636886f,   0.0424357603570167f,
    0.0426545872368219f,   0.0428734558529040f,   0.0430923661550748f,
    0.0433113180931585f,   0.0435303116169914f,   0.0437493466764225f,
    0.0439684232213125f,   0.0441875412015349f,   0.0444067005669753f,
    0.0446259012675311f,   0.0448451432531127f,   0.0450644264736426f,
    0.0452837508790555f,   0.0455031164192982f,   0.0457225230443301f,
    0.0459419707041227f,   0.0461614593486596f,   0.0463809889279372f,
    0.0466005593919636f,   0.0468201706907599f,   0.0470398227743584f,
    0.0472595155928048f,   0.0474792490961565f,   0.0476990232344832f,
    0.0479188379578670f,   0.0481386932164022f,   0.0483585889601954f,
    0.0485785251393657f,   0.0487985017040441f,   0.0490185186043742f,
    0.0492385757905118f,   0.0494586732126247f,   0.0496788108208932f,
    0.0498989885655103f,   0.0501192063966806f,   0.0401356533420085f,
    0.0403559511969490f,   0.0405762889891309f,   0.0407966666688085f,
    0.0410170841862478f,   0.0412375414917278f,   0.0414580385355386f,
    0.0416785752679840f,   0.0418991516393793f,   0.0421197676000519f,
    0.0423404231003423f,   0.0425611180906021f,   0.0427818525211963f,
    0.0430026263425014f,   0.0432234395049069f,   0.0434442919588138f,
    0.0436651836546360f,   0.0438861145427996f,   0.0441070845737422f,
    0.0443280936979149f,   0.0445491418657804f,   0.0447702290278135f,
    0.0449913551345018f,   0.0452125201363448f,   0.0454337239838546f,
    0.0456549666275551f,   0.0458762480179831f,   0.0460975681056872f,
    0.0463189268412284f,   0.0465403241751802f,   0.0467617600581280f,
    0.0469832344406698f,   0.0472047472734157f,   0.0474262985069884f,
    0.0476478880920221f,   0.0478695159791643f,   0.0480911821190742f,
    0.0483128864624232f,   0.0485346289598951f,   0.0487564095621865f,
    0.0489782282200054f,   0.0492000848840726f,   0.0494219795051212f,
    0.0496439120338962f,   0.0498658824211554f,   0.0500878906176685f,
    0.0503099365742177f,   0.0505320202415973f,   0.0507541415706141f,
    0.0509763005120871f,   0.0511984970168471f,   0.0514207310357381f,
    0.0516430025196157f,   0.0518653114193479f,   0.0520876576858154f,
    0.0523100412699105f,   0.0525324621225380f,   0.0527549201946156f,
    0.0529774154370724f,   0.0531999478008504f,   0.0534225172369033f,
    0.0536451236961979f,   0.0538677671297125f,   0.0540904474884381f,
    0.0543131647233778f,   0.0545359187855472f,   0.0547587096259737f,
    0.0549815371956978f,   0.0552044014457715f,   0.0554273023272596f,
    0.0556502397912386f,   0.0558732137887982f,   0.0560962242710392f,
    0.0563192711890757f,   0.0565423544940334f,   0.0567654741370510f,
    0.0569886300692788f,   0.0572118222418796f,   0.0574350506060288f,
    0.0576583151129133f,   0.0578816157137332f,   0.0581049523597000f,
    0.0583283250020387f,   0.0585517335919851f,   0.0485713671581760f,
    0.0487948474970971f,   0.0490183636374095f,   0.0492419155303987f,
    0.0494655031273628f,   0.0496891263796120f,   0.0499127852384687f,
    0.0501364796552677f,   0.0503602095813560f,   0.0505839749680932f,
    0.0508077757668506f,   0.0510316119290123f,   0.0512554834059744f,
    0.0514793901491452f,   0.0517033321099457f,   0.0519273092398090f,
    0.0521513214901801f,   0.0523753688125170f,   0.0525994511582891f,
    0.0528235684789790f,   0.0530477207260808f,   0.0532719078511013f,
    0.0534961298055596f,   0.0537203865409869f,   0.0539446780089268f,
    0.0541690041609351f,   0.0543933649485798f,   0.0546177603234417f,
    0.0548421902371131f,   0.0550666546411992f,   0.0552911534873171f,
    0.0555156867270965f,   0.0557402543121789f,   0.0559648561942188f,
    0.0561894923248822f,   0.0564141626558481f,   0.0566388671388072f,
    0.0568636057254629f,   0.0570883783675304f,   0.0573131850167378f,
    0.0575380256248254f,   0.0577629001435447f,   0.0579878085246610f,
    0.0582127507199510f,   0.0584377266812042f,   0.0586627363602216f,
    0.0588877797088170f,   0.0591128566788167f,   0.0593379672220590f,
    0.0595631112903944f,   0.0597882888356857f,   0.0600134998098081f,
    0.0602387441646493f,   0.0604640218521088f,   0.0606893328240985f,
    0.0609146770325428f,   0.0611400544293785f,   0.0613654649665540f,
    0.0615909085960309f,   0.0618163852697822f,   0.0620418949397940f,
    0.0622674375580639f,   0.0624930130766024f,   0.0627186214474318f,
    0.0629442626225872f,   0.0631699365541155f,   0.0633956431940764f,
    0.0534175715719286f,   0.0536433434849815f,   0.0538691479627182f,
    0.0540949849572480f,   0.0543208544206912f,   0.0545467563051810f,
    0.0547726905628627f,   0.0549986571458940f,   0.0552246560064449f,
    0.0554506870966974f,   0.0556767503688461f,   0.0559028457750979f,
    0.0561289732676714f,   0.0563551327987983f,   0.0565813243207222f,
    0.0568075477856989f,   0.0570338031459965f,   0.0572600903538957f,
    0.0574864093616889f,   0.0577127601216814f,   0.0579391425861903f,
    0.0581655567075452f,   0.0583920024380880f,   0.0586184797301730f,
    0.0588449885361663f,   0.0590715288084468f,   0.0592981004994055f,
    0.0595247035614457f,   0.0597513379469827f,   0.0599780036084447f,
    0.0602047004982714f,   0.0604314285689156f,   0.0606581877728418f,
    0.0608849780625269f,   0.0611117993904601f,   0.0613386517091431f,
    0.0615655349710893f,   0.0617924491288254f,   0.0620193941348894f,
    0.0622463699418319f,   0.0624733765022160f,   0.0627004137686168f,
    0.0629274816936218f,   0.0631545802298307f,   0.0633817093298557f,
    0.0636088689463210f,   0.0638360590318633f,   0.0640632795391316f,
    0.0642905304207867f,   0.0645178116295023f,   0.0647451231179644f,
    0.0649724648388708f,   0.0651998367449316f,   0.0654272387888696f,
    0.0656546709234196f,   0.0658821331013287f,   0.0661096252753566f,
    0.0663371473982747f,   0.0665646994228671f,   0.0667922813019303f,
    0.0568160820656600f,   0.0570437235121022f,   0.0572713946714774f,
    0.0574990954966311f,   0.0577268259404211f,   0.0579545859557173f,
    0.0581823754954017f,   0.0584101945123692f,   0.0586380429595264f,
    0.0588659207897921f,   0.0590938279560984f,   0.0593217644113885f,
    0.0595497301086181f,   0.0597777250007558f,   0.0600057490407822f,
    0.0602338021816895f,   0.0604618843764834f,   0.0606899955781809f,
    0.0609181357398114f,   0.0611463048144174f,   0.0613745027550527f,
    0.0616027295147840f,   0.0618309850466897f,   0.0620592693038611f,
    0.0622875822394012f,   0.0625159238064261f,   0.0627442939580634f,
    0.0629726926474533f,   0.0632011198277480f,   0.0634295754521126f,
    0.0636580594737237f,   0.0638865718457710f,   0.0641151125214557f,
    0.0643436814539919f,   0.0645722785966058f,   0.0648009039025354f,
    0.0650295573250316f,   0.0652582388173575f,   0.0654869483327883f,
    0.0657156858246115f,   0.0659444512461268f,   0.0661732445506464f,
    0.0664020656914949f,   0.0666309146220087f,   0.0668597912955365f,
    0.0670886956654401f,   0.0673176276850925f,   0.0675465873078798f,
    0.0677755744872001f,   0.0680045891764636f,   0.0682336313290928f,
    0.0684627008985227f,   0.0686917978382008f,   0.0689209221015861f,
    0.0691500736421506f,   0.0693792524133784f,   0.0594046474461534f,
    0.0596338805392087f,   0.0598631407234532f,   0.0600924279524199f,
    0.0603217421796543f,   0.0605510833587138f,   0.0607804514431689f,
    0.0610098463866017f,   0.0612392681426066f,   0.0614687166647906f,
    0.0616981919067730f,   0.0619276938221849f,   0.0621572223646700f,
    0.0623867774878845f,   0.0626163591454967f,   0.0628459672911870f,
    0.0630756018786481f,   0.0633052628615854f,   0.0635349501937161f,
    0.0637646638287699f,   0.0639944037204887f,   0.0642241698226267f,
    0.0644539620889507f,   0.0646837804732392f,   0.0649136249292833f,
    0.0651434954108865f,   0.0653733918718643f,   0.0656033142660447f,
    0.0658332625472679f,   0.0660632366693863f,   0.0662932365862647f,
    0.0665232622517804f,   0.0667533136198224f,   0.0669833906442924f,
    0.0672134932791041f,   0.0674436214781841f,   0.0676737751954706f,
    0.0679039543849144f,   0.0681341590004783f,   0.0683643889961381f,
    0.0685946443258808f,   0.0688249249437064f,   0.0690552308036274f,
    0.0692855618596678f,   0.0695159180658648f,   0.0697462993762669f,
    0.0699767057449354f,   0.0702071371259442f,   0.0704375934733789f,
    0.0706680747413375f,   0.0708985808839306f,   0.0711291118552808f,
    0.0713596676095232f,   0.0715902481008047f,   0.0616170423606727f,
    0.0618476721885236f,   0.0620783266159294f,   0.0623090055970863f,
    0.0625397090862030f,   0.0627704370375001f,   0.0630011894052112f,
    0.0632319661435816f,   0.0634627672068692f,   0.0636935925493440f,
    0.0639244421252883f,   0.0641553158889967f,   0.0643862137947761f,
    0.0646171357969457f,   0.0648480818498370f,   0.0650790519077937f,
    0.0653100459251719f,   0.0655410638563397f,   0.0657721056556779f,
    0.0660031712775794f,   0.0662342606764491f,   0.0664653738067047f,
    0.0666965106227758f,   0.0669276710791043f,   0.0671588551301446f,
    0.0673900627303632f,   0.0676212938342389f,   0.0678525483962630f,
    0.0680838263709389f,   0.0683151277127820f,   0.0685464523763204f,
    0.0687778003160946f,   0.0690091714866568f,   0.0692405658425719f,
    0.0694719833384170f,   0.0697034239287815f,   0.0699348875682669f,
    0.0701663742114875f,   0.0703978838130692f,   0.0706294163276505f,
    0.0708609717098825f,   0.0710925499144278f,   0.0713241508959620f,
    0.0715557746091728f,   0.0717874210087600f,   0.0720190900494359f,
    0.0722507816859246f,   0.0724824958729632f,   0.0727142325653007f,
    0.0729459917176984f,   0.0731777732849299f,   0.0734095772217809f,
    0.0736414034830499f,   0.0738732520235469f,   0.0741051227980950f,
    0.0641332048389163f,   0.0643651199460837f,   0.0645970571518437f,
    0.0648290164110684f,   0.0650609976786417f,   0.0652930009094604f,
    0.0655250260584330f,   0.0657570730804803f,   0.0659891419305358f,
    0.0662212325635450f,   0.0664533449344658f,   0.0666854789982679f,
    0.0669176347099340f,   0.0671498120244587f,   0.0673820108968489f,
    0.0676142312821242f,   0.0678464731353154f,   0.0680787364114669f,
    0.0683110210656344f,   0.0685433270528865f,   0.0687756543283037f,
    0.0690080028469791f,   0.0692403725640174f,   0.0694727634345366f,
    0.0697051754136662f,   0.0699376084565483f,   0.0701700625183374f,
    0.0704025375541997f,   0.0706350335193146f,   0.0708675503688727f,
    0.0711000880580778f,   0.0713326465421456f,   0.0715652257763042f,
    0.0717978257157935f,   0.0720304463158665f,   0.0722630875317879f,
    0.0724957493188349f,   0.0727284316322967f,   0.0729611344274751f,
    0.0731938576596843f,   0.0734266012842503f,   0.0736593652565118f,
    0.0738921495318196f,   0.0741249540655367f,   0.0743577788130385f,
    0.0745906237297128f,   0.0748234887709595f,   0.0750563738921907f,
    0.0752892790488311f,   0.0755222041963173f,   0.0757551492900986f,
    0.0759881142856362f,   0.0762210991384039f,   0.0764541038038875f,
    0.0766871282375853f,   0.0769201723950076f,   0.0669494253090649f,
    0.0671825087805169f,   0.0674156118422988f,   0.0676487344499700f,
    0.0678818765591026f,   0.0681150381252804f,   0.0683482191041003f,
    0.0685814194511707f,   0.0688146391221127f,   0.0690478780725596f,
    0.0692811362581574f,   0.0695144136345632f,   0.0697477101574477f,
    0.0699810257824931f,   0.0702143604653941f,   0.0704477141618579f,
    0.0706810868276036f,   0.0709144784183627f,   0.0711478888898790f,
    0.0713813181979088f,   0.0716147662982205f,   0.0718482331465947f,
    0.0720817186988244f,   0.0723152229107147f,   0.0725487457380832f,
    0.0727822871367598f,   0.0730158470625864f,   0.0732494254714176f,
    0.0734830223191197f,   0.0737166375615722f,   0.0739502711546658f,
    0.0741839230543041f,   0.0744175932164030f,   0.0746512815968905f,
    0.0748849881517070f,   0.0751187128368048f,   0.0753524556081493f,
    0.0755862164217173f,   0.0758199952334984f,   0.0760537919994943f,
    0.0762876066757191f,   0.0765214392181990f,   0.0767552895829728f,
    0.0769891577260911f,   0.0772230436036172f,   0.0774569471716267f,
    0.0776908683862069f,   0.0779248072034582f,   0.0781587635794926f,
    0.0783927374704347f,   0.0786267288324215f,   0.0788607376216020f,
    0.0790947637941377f,   0.0793288073062022f,   0.0795628681139814f,
    0.0797969461736736f,   0.0800310414414898f,   0.0802651538736520f,
    0.0804992834263956f,   0.0807334300559682f,   0.0707637827960168f,
    0.0709979634480382f,   0.0712321610457045f,   0.0714663755453118f,
    0.0717006069031690f,   0.0719348550755973f,   0.0721691200189298f,
    0.0724034016895124f,   0.0726377000437030f,   0.0728720150378716f,
    0.0731063466284005f,   0.0733406947716849f,   0.0735750594241317f,
    0.0738094405421600f,   0.0740438380822017f,   0.0742782520007005f,
    0.0745126822541126f,   0.0747471287989061f,   0.0749815915915625f,
    0.0752160705885742f,   0.0754505657464466f,   0.0756850770216972f,
    0.0759196043708562f,   0.0761541477504655f,   0.0763887071170792f,
    0.0766232824272645f,   0.0768578736376001f,   0.0770924807046775f,
    0.0773271035850998f,   0.0775617422354831f,   0.0777963966124555f,
    0.0780310666726576f,   0.0782657523727417f,   0.0785004536693729f,
    0.0787351705192282f,   0.0789699028789975f,   0.0792046507053824f,
    0.0794394139550970f,   0.0796741925848675f,   0.0799089865514328f,
    0.0801437958115436f,   0.0803786203219632f,   0.0806134600394671f,
    0.0808483149208430f,   0.0810831849228909f,   0.0813180700024232f,
    0.0815529701162645f,   0.0817878852212516f,   0.0820228152742338f,
    0.0822577602320724f,   0.0824927200516413f,   0.0827276946898264f,
    0.0829626841035263f,   0.0831976882496509f,   0.0834327070851236f,
    0.0836677405668795f,   0.0839027886518659f,   0.0841378512970423f,
    0.0843729284593812f,   0.0846080200958663f,   0.0848431261634945f,
    0.0850782466192747f,   0.0853133814202278f,   0.0855485305233871f,
    0.0857836938857984f,   0.0860188714645199f,   0.0862540632166216f,
    0.0762854581765733f,   0.0765206781466951f,   0.0767559121614814f,
    0.0769911601780519f,   0.0772264221535377f,   0.0774616980450831f,
    0.0776969878098444f,   0.0779322914049897f,   0.0781676087877001f,
    0.0784029399151685f,   0.0786382847446002f,   0.0788736432332130f,
    0.0791090153382364f,   0.0793444010169132f,   0.0795798002264975f,
    0.0798152129242558f,   0.0800506390674675f,   0.0802860786134237f,
    0.0805215315194281f,   0.0807569977427964f,   0.0809924772408571f,
    0.0812279699709503f,   0.0814634758904288f,   0.0816989949566574f,
    0.0819345271270137f,   0.0821700723588872f,   0.0824056306096794f,
    0.0826412018368049f,   0.0828767859976898f,   0.0831123830497728f,
    0.0833479929505050f,   0.0835836156573494f,   0.0838192511277818f,
    0.0840548993192899f,   0.0842905601893735f,   0.0845262336955456f,
    0.0847619197953302f,   0.0849976184462647f,   0.0852333296058980f,
    0.0854690532317919f,   0.0857047892815197f,   0.0859405377126680f,
    0.0861762984828349f,   0.0864120715496310f,   0.0866478568706792f,
    0.0868836544036147f,   0.0871194641060850f,   0.0873552859357498f,
    0.0875911198502811f,   0.0878269658073632f,   0.0880628237646929f,
    0.0882986936799788f,   0.0885345755109422f,   0.0887704692153166f,
    0.0890063747508475f,   0.0892422920752930f,   0.0894782211464236f,
    0.0897141619220215f,   0.0899501143598818f,   0.0901860784178115f,
    0.0904220540536300f,   0.0906580412251691f,   0.0908940398902726f,
    0.0911300500067969f,   0.0913660715326105f,   0.0916021044255941f,
    0.0918381486436412f,   0.0920742041446566f,   0.0923102708865582f,
    0.0925463488272760f,   0.0927824379247522f,   0.0930185381369413f,
    0.0932546494218102f,   0.0934907717373377f,   0.0937269050415152f,
    0.0939630492923465f,   0.0941992044478474f,   0.0944353704660461f,
    0.0946715473049831f,   0.0949077349227112f,   0.0951439332772953f,
    0.0953801423268129f,   0.0956163620293533f,   0.0856487814204062f,
    0.0858850223033106f,   0.0861212737135806f,   0.0863575356093548f,
    0.0865938079487840f,   0.0868300906900319f,   0.0870663837912740f,
    0.0873026872106978f,   0.0875390009065038f,   0.0877753248369042f,
    0.0880116589601236f,   0.0882480032343993f,   0.0884843576179805f,
    0.0887207220691284f,   0.0889570965461172f,   0.0891934810072328f,
    0.0894298754107737f,   0.0896662797150505f,   0.0899026938783861f,
    0.0901391178591160f,   0.0903755516155873f,   0.0906119951061600f,
    0.0908484482892064f,   0.0910849111231104f,   0.0913213835662688f,
    0.0915578655770907f,   0.0917943571139971f,   0.0920308581354216f,
    0.0922673685998101f,   0.0925038884656203f,   0.0927404176913226f,
    0.0929769562353998f,   0.0932135040563467f,   0.0934500611126704f,
    0.0936866273628904f,   0.0939232027655385f,   0.0941597872791585f,
    0.0943963808623070f,   0.0946329834735524f,   0.0948695950714755f,
    0.0951062156146697f,   0.0953428450617401f,   0.0955794833713046f,
    0.0958161305019931f,   0.0960527864124481f,   0.0962894510613237f,
    0.0965261244072870f,   0.0967628064090171f,   0.0969994970252055f,
    0.0972361962145558f,   0.0974729039357838f,   0.0977096201476179f,
    0.0979463448087987f,   0.0981830778780787f,   0.0984198193142229f,
    0.0986565690760093f,   0.0988933271222270f,   0.0991300934116780f,
    0.0993668679031768f,   0.0996036505555495f,   0.0998404413276351f,
    0.100077240178285f,    0.100314047066361f,    0.100550861950741f,
    0.100787684790311f,    0.101024515543973f,    0.101261354170637f,
    0.101498200629230f,    0.101735054878688f,    0.101971916877960f,
    0.102208786586009f,    0.102445663961807f,    0.102682548964341f,
    0.102919441552610f,    0.103156341685624f,    0.103393249322406f,
    0.103630164421992f,    0.103867086943429f,    0.104104016845776f,
    0.104340954088107f,    0.104577898629505f,    0.104814850429067f,
    0.105051809445902f,    0.105288775639131f,    0.105525748967888f,
    0.105762729391318f,    0.105999716868580f,    0.106236711358845f,
    0.106473712821294f,    0.106710721215122f,    0.106947736499538f,
    0.107184758633760f,    0.107421787577020f,    0.107658823288562f,
    0.107895865727643f,    0.108132914853530f,    0.108369970625506f,
    0.108607033002864f,    0.108844101944907f,    0.109081177410956f,
    0.109318259360339f,    0.109555347752398f,    0.109792442546489f,
    0.110029543701978f,    0.110266651178245f,    0.110503764934681f,
    0.110740884930689f,    0.110978011125686f,    0.111215143479099f,
    0.111452281950370f,    0.111689426498952f,    0.111926577084308f,
    0.112163733665917f,    0.112400896203270f,    0.112638064655866f,
    0.112875238983221f,    0.113112419144862f,    0.113349605100326f,
    0.113586796809166f,    0.113823994230944f,    0.114061197325236f,
    0.114298406051631f,    0.114535620369728f,    0.114772840239140f,
    0.115010065619491f,    0.115247296470419f,    0.115484532751574f,
    0.115721774422616f,    0.115959021443219f,    0.116196273773071f,
    0.116433531371868f,    0.116670794199323f,    0.116908062215158f,
    0.117145335379108f,    0.117382613650922f,    0.117619896990358f,
    0.117857185357189f,    0.118094478711200f,    0.118331777012186f,
    0.118569080219959f,    0.118806388294337f,    0.119043701195156f,
    0.119281018882261f,    0.119518341315510f,    0.119755668454773f,
    0.119993000259933f,    0.120230336690886f,    0.120467677707538f,
    0.120705023269809f,    0.120942373337631f,    0.121179727870948f,
    0.121417086829716f,    0.121654450173904f,    0.121891817863493f,
    0.122129189858476f,    0.122366566118858f,    0.122603946604659f,
    0.122841331275906f,    0.123078720092644f,    0.123316113014926f,
    0.123553510002819f,    0.123790911016403f,    0.124028316015769f,
    0.124265724961022f,    0.124503137812276f,    0.124740554529660f,
    0.124977975073316f,    0.125215399403394f,    0.125452827480062f,
    0.125690259263497f,    0.125927694713887f,    0.126165133791435f,
    0.126402576456356f,    0.126640022668876f,    0.126877472389234f,
    0.127114925577681f,    0.127352382194480f,    0.127589842199908f,
    0.127827305554251f,    0.128064772217811f,    0.128302242150900f,
    0.128539715313843f,    0.118573380744364f,    0.118810860248038f,
    0.119048342862613f,    0.119285828548465f,    0.119523317265979f,
    0.119760808975554f,    0.119998303637600f,    0.120235801212540f,
    0.120473301660811f,    0.120710804942859f,    0.120948311019145f,
    0.121185819850140f,    0.121423331396330f,    0.121660845618210f,
    0.121898362476291f,    0.122135881931093f,    0.122373403943149f,
    0.122610928473007f,    0.122848455481224f,    0.123085984928370f,
    0.123323516775029f,    0.123561050981795f,    0.123798587509275f,
    0.124036126318090f,    0.124273667368871f,    0.124511210622262f,
    0.124748756038919f,    0.124986303579512f,    0.125223853204721f,
    0.125461404875240f,    0.125698958551774f,    0.125936514195041f,
    0.126174071765771f,    0.126411631224707f,    0.126649192532603f,
    0.126886755650226f,    0.127124320538356f,    0.127361887157783f,
    0.127599455469312f,    0.127837025433760f,    0.128074597011953f,
    0.128312170164733f,    0.128549744852953f,    0.128787321037478f,
    0.129024898679186f,    0.129262477738966f,    0.129500058177720f,
    0.129737639956363f,    0.129975223035822f,    0.130212807377034f,
    0.130450392940952f,    0.130687979688539f,    0.130925567580771f,
    0.131163156578635f,    0.131400746643133f,    0.131638337735276f,
    0.131875929816089f,    0.132113522846610f,    0.132351116787888f,
    0.132588711600985f,    0.132826307246975f,    0.133063903686944f,
    0.133301500881990f,    0.133539098793225f,    0.133776697381772f,
    0.134014296608765f,    0.134251896435354f,    0.134489496822697f,
    0.134727097731967f,    0.134964699124348f,    0.135202300961038f,
    0.135439903203245f,    0.135677505812191f,    0.135915108749109f,
    0.136152711975246f,    0.136390315451860f,    0.136627919140220f,
    0.136865523001611f,    0.137103126997326f,    0.137340731088674f,
    0.137578335236974f,    0.137815939403557f,    0.138053543549769f,
    0.138291147636965f,    0.138528751626515f,    0.138766355479799f,
    0.139003959158210f,    0.139241562623155f,    0.139479165836051f,
    0.139716768758329f,    0.139954371351430f,    0.140191973576809f,
    0.140429575395934f,    0.140667176770284f,    0.140904777661350f,
    0.141142378030637f,    0.141379977839660f,    0.141617577049948f,
    0.141855175623041f,    0.142092773520492f,    0.142330370703868f,
    0.142567967134744f,    0.142805562774712f,    0.143043157585373f,
    0.143280751528341f,    0.143518344565244f,    0.143755936657720f,
    0.143993527767421f,    0.144231117856009f,    0.144468706885161f,
    0.144706294816565f,    0.144943881611921f,    0.145181467232942f,
    0.145419051641353f,    0.145656634798891f,    0.145894216667305f,
    0.146131797208357f,    0.146369376383822f,    0.146606954155485f,
    0.146844530485146f,    0.147082105334614f,    0.147319678665714f,
    0.147557250440280f,    0.147794820620161f,    0.148032389167217f,
    0.148269956043319f,    0.148507521210353f,    0.148745084630214f,
    0.148982646264813f,    0.149220206076071f,    0.149457764025922f,
    0.149695320076311f,    0.149932874189196f,    0.150170426326549f,
    0.150407976450352f,    0.150645524522601f,    0.150883070505302f,
    0.151120614360475f,    0.151358156050152f,    0.151595695536379f,
    0.151833232781210f,    0.152070767746715f,    0.152308300394975f,
    0.152545830688083f,    0.152783358588146f,    0.153020884057280f,
    0.153258407057616f,    0.153495927551297f,    0.153733445500477f,
    0.153970960867323f,    0.154208473614016f,    0.154445983702745f,
    0.154683491095716f,    0.154920995755145f,    0.155158497643259f,
    0.155395996722300f,    0.155633492954521f,    0.155870986302186f,
    0.156108476727575f,    0.156345964192975f,    0.156583448660691f,
    0.156820930093035f,    0.157058408452335f,    0.157295883700930f,
    0.157533355801171f,    0.147567013792809f,    0.147804479483445f,
    0.148041941912855f,    0.148279401043438f,    0.148516856837608f,
    0.148754309257790f,    0.148991758266421f,    0.149229203825950f,
    0.149466645898839f,    0.149704084447562f,    0.149941519434606f,
    0.150178950822470f,    0.150416378573664f,    0.150653802650711f,
    0.150891223016147f,    0.151128639632520f,    0.151366052462390f,
    0.151603461468329f,    0.151840866612921f,    0.152078267858765f,
    0.152315665168468f,    0.152553058504653f,    0.152790447829953f,
    0.153027833107014f,    0.153265214298494f,    0.153502591367065f,
    0.153739964275408f,    0.153977332986220f,    0.154214697462207f,
    0.154452057666090f,    0.154689413560599f,    0.154926765108481f,
    0.155164112272490f,    0.155401455015396f,    0.155638793299981f,
    0.155876127089037f,    0.156113456345370f,    0.156350781031798f,
    0.156588101111152f,    0.156825416546274f,    0.157062727300019f,
    0.157300033335253f,    0.157537334614857f,    0.157774631101722f,
    0.158011922758752f,    0.158249209548862f,    0.158486491434983f,
    0.158723768380054f,    0.158961040347028f,    0.159198307298871f,
    0.159435569198561f,    0.159672826009087f,    0.159910077693452f,
    0.160147324214669f,    0.160384565535767f,    0.160621801619783f,
    0.160859032429770f,    0.161096257928790f,    0.161333478079920f,
    0.161570692846248f,    0.161807902190875f,    0.162045106076912f,
    0.162282304467485f,    0.162519497325732f,    0.162756684614802f,
    0.162993866297856f,    0.153027231415457f,    0.153264401776015f,
    0.153501566420118f,    0.153738725310976f,    0.153975878411812f,
    0.154213025685861f,    0.154450167096373f,    0.154687302606606f,
    0.154924432179834f,    0.155161555779340f,    0.155398673368422f,
    0.155635784910389f,    0.155872890368562f,    0.156109989706276f,
    0.156347082886876f,    0.156584169873721f,    0.156821250630182f,
    0.157058325119641f,    0.157295393305493f,    0.157532455151146f,
    0.157769510620020f,    0.158006559675547f,    0.158243602281171f,
    0.158480638400349f,    0.158717667996550f,    0.158954691033254f,
    0.159191707473956f,    0.159428717282161f,    0.159665720421387f,
    0.159902716855164f,    0.160139706547035f,    0.160376689460556f,
    0.160613665559292f,    0.160850634806823f,    0.161087597166742f,
    0.161324552602651f,    0.161561501078168f,    0.161798442556921f,
    0.162035377002551f,    0.162272304378710f,    0.162509224649065f,
    0.162746137777293f,    0.162983043727084f,    0.163219942462139f,
    0.163456833946175f,    0.163693718142917f,    0.163930595016105f,
    0.164167464529489f,    0.164404326646834f,    0.164641181331916f,
    0.164878028548522f,    0.165114868260454f,    0.165351700431524f,
    0.155384714102944f,    0.155621531083777f,    0.155858340415260f,
    0.156095142061255f,    0.156331935985636f,    0.156568722152290f,
    0.156805500525115f,    0.157042271068022f,    0.157279033744935f,
    0.157515788519790f,    0.157752535356533f,    0.157989274219126f,
    0.158226005071540f,    0.158462727877761f,    0.158699442601785f,
    0.158936149207623f,    0.159172847659294f,    0.159409537920833f,
    0.159646219956287f,    0.159882893729713f,    0.160119559205183f,
    0.160356216346779f,    0.160592865118596f,    0.160829505484743f,
    0.161066137409338f,    0.161302760856514f,    0.161539375790416f,
    0.161775982175200f,    0.162012579975035f,    0.162249169154102f,
    0.162485749676594f,    0.162722321506719f,    0.162958884608692f,
    0.163195438946746f,    0.163431984485122f,    0.163668521188075f,
    0.163905049019873f,    0.164141567944796f,    0.164378077927134f,
    0.164614578931192f,    0.164851070921286f,    0.165087553861745f,
    0.165324027716909f,    0.165560492451133f,    0.165796948028780f,
    0.166033394414230f,    0.156066020649259f,    0.156302448543495f,
    0.156538867138740f,    0.156775276399421f,    0.157011676289976f,
    0.157248066774858f,    0.157484447818530f,    0.157720819385467f,
    0.157957181440159f,    0.158193533947106f,    0.158429876870820f,
    0.158666210175827f,    0.158902533826663f,    0.159138847787880f,
    0.159375152024038f,    0.159611446499712f,    0.159847731179489f,
    0.160084006027966f,    0.160320271009756f,    0.160556526089482f,
    0.160792771231779f,    0.161029006401295f,    0.161265231562690f,
    0.161501446680637f,    0.161737651719821f,    0.161973846644939f,
    0.162210031420699f,    0.162446206011824f,    0.162682370383048f,
    0.162918524499116f,    0.163154668324787f,    0.163390801824832f,
    0.163626924964034f,    0.163863037707188f,    0.164099140019102f,
    0.164335231864596f,    0.164571313208501f,    0.164807384015662f,
    0.165043444250936f,    0.165279493879192f,    0.165515532865311f,
    0.165751561174187f,    0.155783767848112f,    0.156019774697231f,
    0.156255770763861f,    0.156491756012944f,    0.156727730409436f,
    0.156963693918304f,    0.157199646504527f,    0.157435588133098f,
    0.157671518769019f,    0.157907438377308f,    0.158143346922994f,
    0.158379244371116f,    0.158615130686729f,    0.158851005834898f,
    0.159086869780700f,    0.159322722489227f,    0.159558563925579f,
    0.159794394054872f,    0.160030212842232f,    0.160266020252799f,
    0.160501816251724f,    0.160737600804171f,    0.160973373875317f,
    0.161209135430348f,    0.161444885434466f,    0.161680623852884f,
    0.161916350650826f,    0.162152065793531f,    0.162387769246248f,
    0.162623460974239f,    0.162859140942778f,    0.163094809117152f,
    0.163330465462660f,    0.163566109944612f,    0.163801742528333f,
    0.164037363179157f,    0.164272971862433f,    0.164508568543521f,
    0.164744153187794f,    0.154775914838023f,    0.155011475304832f,
    0.155247023631016f,    0.155482559781998f,    0.155718083723212f,
    0.155953595420103f,    0.156189094838130f,    0.156424581942764f,
    0.156660056699488f,    0.156895519073797f,    0.157130969031199f,
    0.157366406537214f,    0.157601831557375f,    0.157837244057224f,
    0.158072644002320f,    0.158308031358231f,    0.158543406090539f,
    0.158778768164837f,    0.159014117546731f,    0.159249454201839f,
    0.159484778095791f,    0.159720089194232f,    0.159955387462814f,
    0.160190672867206f,    0.160425945373088f,    0.160661204946150f,
    0.160896451552098f,    0.161131685156647f,    0.161366905725526f,
    0.161602113224477f,    0.161837307619252f,    0.162072488875617f,
    0.162307656959350f,    0.162542811836242f,    0.162777953472093f,
    0.163013081832719f,    0.163248196883948f,    0.163483298591617f,
    0.153514575998966f,    0.153749650917085f,    0.153984712389236f,
    0.154219760381308f,    0.154454794859201f,    0.154689815788829f,
    0.154924823136117f,    0.155159816867001f,    0.155394796947432f,
    0.155629763343373f,    0.155864716020796f,    0.156099654945689f,
    0.156334580084050f,    0.156569491401892f,    0.156804388865237f,
    0.157039272440121f,    0.157274142092592f,    0.157508997788710f,
    0.157743839494548f,    0.157978667176190f,    0.158213480799735f,
    0.158448280331291f,    0.158683065736980f,    0.158917836982936f,
    0.159152594035305f,    0.159387336860246f,    0.159622065423929f,
    0.159856779692538f,    0.160091479632268f,    0.160326165209327f,
    0.160560836389934f,    0.160795493140323f,    0.161030135426737f,
    0.161264763215433f,    0.161499376472680f,    0.161733975164760f,
    0.161968559257967f,    0.151999317795992f,    0.152233872590381f,
    0.152468412684851f,    0.152702938045745f,    0.152937448639417f,
    0.153171944432235f,    0.153406425390579f,    0.153640891480840f,
    0.153875342669423f,    0.154109778922744f,    0.154344200207233f,
    0.154578606489328f,    0.154812997735486f,    0.155047373912170f,
    0.155281734985858f,    0.155516080923042f,    0.155750411690222f,
    0.155984727253915f,    0.156219027580646f,    0.156453312636956f,
    0.156687582389395f,    0.156921836804528f,    0.157156075848931f,
    0.157390299489191f,    0.157624507691911f,    0.157858700423702f,
    0.158092877651190f,    0.158327039341012f,    0.158561185459819f,
    0.158795315974272f,    0.159029430851045f,    0.159263530056826f,
    0.159497613558314f,    0.159731681322219f,    0.159965733315264f,
    0.160199769504187f,    0.150229978933122f,    0.150463983414054f,
    0.150697971991144f,    0.150931944631177f,    0.151165901300950f,
    0.151399841967272f,    0.151633766596964f,    0.151867675156863f,
    0.152101567613812f,    0.152335443934671f,    0.152569304086311f,
    0.152803148035615f,    0.153036975749479f,    0.153270787194809f,
    0.153504582338527f,    0.153738361147564f,    0.153972123588865f,
    0.154205869629387f,    0.154439599236099f,    0.154673312375982f,
    0.154907009016030f,    0.155140689123249f,    0.155374352664657f,
    0.155607999607285f,    0.155841629918175f,    0.156075243564383f,
    0.156308840512976f,    0.156542420731034f,    0.156775984185648f,
    0.157009530843923f,    0.157243060672975f,    0.157476573639934f,
    0.157710069711939f,    0.157943548856145f,    0.158177011039717f,
    0.158410456229833f,    0.158643884393684f,    0.148673484575858f,
    0.148906878588797f,    0.149140255477114f,    0.149373615208048f,
    0.149606957748852f,    0.149840283066790f,    0.150073591129136f,
    0.150306881903181f,    0.150540155356224f,    0.150773411455578f,
    0.151006650168570f,    0.151239871462536f,    0.151473075304826f,
    0.151706261662802f,    0.151939430503840f,    0.152172581795324f,
    0.152405715504656f,    0.152638831599245f,    0.152871930046515f,
    0.153105010813902f,    0.153338073868854f,    0.153571119178832f,
    0.153804146711309f,    0.154037156433768f,    0.154270148313708f,
    0.154503122318637f,    0.154736078416078f,    0.154969016573565f,
    0.155201936758644f,    0.155434838938874f,    0.155667723081826f,
    0.155900589155082f,    0.156133437126239f,    0.156366266962903f,
    0.156599078632696f,    0.156831872103249f,    0.157064647342207f,
    0.147093593394615f,    0.147326332073366f,    0.147559052423530f,
    0.147791754412800f,    0.148024438008882f,    0.148257103179495f,
    0.148489749892369f,    0.148722378115246f,    0.148954987815883f,
    0.149187578962047f,    0.149420151521517f,    0.149652705462085f,
    0.149885240751555f,    0.150117757357745f,    0.150350255248482f,
    0.150582734391608f,    0.150815194754977f,    0.151047636306454f,
    0.151280059013917f,    0.151512462845255f,    0.151744847768373f,
    0.151977213751184f,    0.152209560761616f,    0.152441888767607f,
    0.152674197737110f,    0.152906487638089f,    0.153138758438519f,
    0.153371010106390f,    0.153603242609701f,    0.153835455916467f,
    0.154067649994712f,    0.154299824812474f,    0.154531980337803f,
    0.154764116538760f,    0.154996233383421f,    0.155228330839873f,
    0.155460408876213f,    0.145488656537942f,    0.145720695638407f,
    0.145952715223132f,    0.146184715260266f,    0.146416695717967f,
    0.146648656564411f,    0.146880597767780f,    0.147112519296273f,
    0.147344421118099f,    0.147576303201479f,    0.147808165514649f,
    0.148040008025854f,    0.148271830703353f,    0.148503633515418f,
    0.148735416430330f,    0.148967179416387f,    0.149198922441895f,
    0.149430645475174f,    0.149662348484558f,    0.149894031438390f,
    0.150125694305028f,    0.150357337052841f,    0.150588959650210f,
    0.150820562065529f,    0.151052144267205f,    0.151283706223655f,
    0.151515247903310f,    0.151746769274613f,    0.151978270306019f,
    0.152209750965996f,    0.152441211223024f,    0.152672651045593f,
    0.152904070402210f,    0.153135469261390f,    0.153366847591662f,
    0.153598205361567f,    0.153829542539659f,    0.154060859094504f,
    0.154292154994679f,    0.154523430208775f,    0.144550873782782f,
    0.144782107530540f,    0.145013320498062f,    0.145244512653990f,
    0.145475683966974f,    0.145706834405678f,    0.145937963938779f,
    0.146169072534965f,    0.146400160162936f,    0.146631226791407f,
    0.146862272389103f,    0.147093296924760f,    0.147324300367130f,
    0.147555282684973f,    0.147786243847065f,    0.148017183822193f,
    0.148248102579156f,    0.148479000086764f,    0.148709876313842f,
    0.148940731229225f,    0.149171564801762f,    0.149402377000313f,
    0.149633167793751f,    0.149863937150961f,    0.150094685040840f,
    0.150325411432298f,    0.150556116294257f,    0.150786799595650f,
    0.151017461305425f,    0.151248101392539f,    0.151478719825965f,
    0.151709316574685f,    0.151939891607695f,    0.152170444894002f,
    0.152400976402628f,    0.152631486102603f,    0.152861973962974f,
    0.153092439952797f,    0.153322884041140f,    0.153553306197087f,
    0.153783706389730f,    0.154014084588175f,    0.144040629838929f,
    0.144270963956348f,    0.144501275986961f,    0.144731565899924f,
    0.144961833664404f,    0.145192079249581f,    0.145422302624648f,
    0.145652503758808f,    0.145882682621278f,    0.146112839181288f,
    0.146342973408078f,    0.146573085270901f,    0.146803174739024f,
    0.147033241781725f,    0.147263286368294f,    0.147493308468033f,
    0.147723308050258f,    0.147953285084295f,    0.148183239539485f,
    0.148413171385178f,    0.148643080590738f,    0.148872967125543f,
    0.149102830958981f,    0.149332672060452f,    0.149562490399370f,
    0.149792285945159f,    0.150022058667259f,    0.150251808535119f,
    0.150481535518200f,    0.150711239585978f,    0.150940920707939f,
    0.151170578853583f,    0.151400213992420f,    0.151629826093976f,
    0.151859415127784f,    0.152088981063395f,    0.152318523870367f,
    0.152548043518276f,    0.152777539976703f,    0.153007013215249f,
    0.153236463203521f,    0.153465889911143f,    0.153695293307748f,
    0.153924673362982f,    0.154154030046506f,    0.154383363327988f,
    0.144408862254502f,    0.144638148640966f,    0.144867411534476f,
    0.145096650904754f,    0.145325866721530f,    0.145555058954550f,
    0.145784227573570f,    0.146013372548361f,    0.146242493848704f,
    0.146471591444392f,    0.146700665305232f,    0.146929715401042f,
    0.147158741701652f,    0.147387744176907f,    0.147616722796661f,
    0.147845677530781f,    0.148074608349148f,    0.148303515221653f,
    0.148532398118202f,    0.148761257008710f,    0.148990091863106f,
    0.149218902651333f,    0.149447689343343f,    0.149676451909102f,
    0.149905190318589f,    0.150133904541793f,    0.150362594548718f,
    0.150591260309378f,    0.150819901793800f,    0.151048518972024f,
    0.151277111814102f,    0.151505680290097f,    0.151734224370087f,
    0.151962744024160f,    0.152191239222416f,    0.152419709934969f,
    0.152648156131945f,    0.152876577783480f,    0.153104974859726f,
    0.153333347330843f,    0.153561695167008f,    0.153790018338407f,
    0.154018316815238f,    0.154246590567715f,    0.154474839566059f,
    0.154703063780507f,    0.154931263181308f,    0.155159437738722f,
    0.155387587423022f,    0.155615712204492f,    0.155843812053431f,
    0.156071886940148f,    0.156299936834965f,    0.146324150785604f,
    0.146552150607636f,    0.146780125348808f,    0.147008074979490f,
    0.147235999470067f,    0.147463898790933f,    0.147691772912497f,
    0.147919621805179f,    0.148147445439411f,    0.148375243785639f,
    0.148603016814319f,    0.148830764495920f,    0.149058486800924f,
    0.149286183699826f,    0.149513855163131f,    0.149741501161358f,
    0.149969121665038f,    0.150196716644712f,    0.150424286070939f,
    0.150651829914283f,    0.150879348145326f,    0.151106840734659f,
    0.151334307652887f,    0.151561748870628f,    0.151789164358508f,
    0.152016554087171f,    0.152243918027269f,    0.152471256149469f,
    0.152698568424449f,    0.152925854822898f,    0.153153115315520f,
    0.153380349873030f,    0.153607558466155f,    0.153834741065634f,
    0.154061897642219f,    0.154289028166675f,    0.154516132609777f,
    0.154743210942315f,    0.154970263135090f,    0.155197289158914f,
    0.155424288984613f,    0.155651262583025f,    0.155878209925000f,
    0.156105130981400f,    0.156332025723100f,    0.156558894120987f,
    0.156785736145960f,    0.157012551768931f,    0.157239340960823f,
    0.157466103692572f,    0.157692839935127f,    0.157919549659449f,
    0.158146232836509f,    0.158372889437294f,    0.158599519432801f,
    0.158826122794039f,    0.159052699492031f,    0.159279249497811f,
    0.159505772782425f,    0.159732269316933f,    0.159958739072404f,
    0.160185182019924f,    0.160411598130588f,    0.160637987375502f,
    0.160864349725789f,    0.161090685152580f,    0.161316993627020f,
    0.161543275120267f,    0.151565718680876f,    0.151791946125256f,
    0.152018146501987f,    0.152244319782275f,    0.152470465937339f,
    0.152696584938411f,    0.152922676756732f,    0.153148741363559f,
    0.153374778730158f,    0.153600788827810f,    0.153826771627808f,
    0.154052727101455f,    0.154278655220068f,    0.154504555954977f,
    0.154730429277522f,    0.154956275159058f,    0.155182093570950f,
    0.155407884484577f,    0.155633647871328f,    0.155859383702607f,
    0.156085091949829f,    0.156310772584421f,    0.156536425577822f,
    0.156762050901484f,    0.156987648526872f,    0.157213218425462f,
    0.157438760568743f,    0.157664274928215f,    0.157889761475393f,
    0.158115220181800f,    0.158340651018977f,    0.158566053958471f,
    0.158791428971847f,    0.159016776030678f,    0.159242095106551f,
    0.159467386171067f,    0.159692649195835f,    0.159917884152481f,
    0.160143091012640f,    0.160368269747960f,    0.160593420330103f,
    0.160818542730740f,    0.161043636921558f,    0.161268702874254f,
    0.161493740560537f,    0.161718749952131f,    0.161943731020768f,
    0.162168683738195f,    0.162393608076173f,    0.162618504006471f,
    0.162843371500873f,    0.163068210531175f,    0.163293021069186f,
    0.163517803086724f,    0.163742556555623f,    0.163967281447728f,
    0.164191977734896f,    0.164416645388997f,    0.164641284381911f,
    0.164865894685533f,    0.165090476271770f,    0.165315029112540f,
    0.165539553179773f,    0.165764048445414f,    0.165988514881416f,
    0.166212952459749f,    0.166437361152392f,    0.166661740931336f,
    0.166886091768588f,    0.167110413636163f,    0.167334706506090f,
    0.167558970350412f,    0.167783205141181f,    0.168007410850464f,
    0.168231587450339f,    0.168455734912896f,    0.168679853210239f,
    0.168903942314481f,    0.169128002197751f,    0.169352032832189f,
    0.169576034189945f,    0.169800006243185f,    0.170023948964084f,
    0.170247862324832f,    0.170471746297630f,    0.170695600854690f,
    0.170919425968239f,    0.171143221610514f,    0.171366987753766f,
    0.171590724370256f,    0.171814431432261f,    0.172038108912066f,
    0.172261756781971f,    0.172485375014288f,    0.172708963581339f,
    0.172932522455463f,    0.173156051609007f,    0.173379551014331f,
    0.173603020643809f,    0.173826460469826f,    0.174049870464779f,
    0.174273250601078f,    0.174496600851146f,    0.174719921187417f,
    0.174943211582337f,    0.175166472008365f,    0.175389702437973f,
    0.175612902843644f,    0.175836073197874f,    0.176059213473171f,
    0.176282323642056f,    0.176505403677060f,    0.176728453550729f,
    0.176951473235621f,    0.177174462704304f,    0.177397421929360f,
    0.177620350883383f,    0.177843249538981f,    0.178066117868771f,
    0.178288955845384f,    0.178511763441463f,    0.178734540629664f,
    0.178957287382655f,    0.179180003673115f,    0.179402689473738f,
    0.179625344757226f,    0.179847969496298f,    0.180070563663683f,
    0.180293127232121f,    0.180515660174367f,    0.180738162463187f,
    0.180960634071358f,    0.181183074971672f,    0.181405485136931f,
    0.181627864539950f,    0.181850213153558f,    0.182072530950592f,
    0.182294817903906f,    0.182517073986364f,    0.182739299170841f,
    0.182961493430228f,    0.183183656737425f,    0.183405789065345f,
    0.183627890386914f,    0.183849960675070f,    0.184071999902763f,
    0.184294008042956f,    0.184515985068623f,    0.184737930952752f,
    0.184959845668342f,    0.185181729188404f,    0.185403581485962f,
    0.185625402534053f,    0.185847192305725f,    0.186068950774038f,
    0.186290677912066f,    0.186512373692895f,    0.186734038089621f,
    0.186955671075354f,    0.187177272623217f,    0.187398842706344f,
    0.187620381297882f,    0.187841888370990f,    0.188063363898839f,
    0.188284807854612f,    0.188506220211506f,    0.188727600942729f,
    0.188948950021500f,    0.189170267421053f,    0.189391553114633f,
    0.189612807075496f,    0.189834029276914f,    0.190055219692166f,
    0.190276378294547f,    0.190497505057364f,    0.190718599953936f,
    0.190939662957593f,    0.191160694041678f,    0.191381693179547f,
    0.191602660344568f,    0.181619784587509f,    0.181840687726985f,
    0.182061558813791f,    0.182282397821343f,    0.182503204723069f,
    0.182723979492412f,    0.182944722102825f,    0.183165432527773f,
    0.183386110740737f,    0.183606756715205f,    0.183827370424682f,
    0.184047951842681f,    0.184268500942731f,    0.184489017698371f,
    0.184709502083153f,    0.184929954070642f,    0.185150373634414f,
    0.185370760748058f,    0.185591115385175f,    0.185811437519379f,
    0.186031727124294f,    0.186251984173560f,    0.186472208640826f,
    0.186692400499755f,    0.186912559724022f,    0.187132686287313f,
    0.187352780163329f,    0.187572841325781f,    0.187792869748393f,
    0.188012865404900f,    0.188232828269052f,    0.188452758314610f,
    0.188672655515346f,    0.188892519845045f,    0.189112351277506f,
    0.189332149786539f,    0.189551915345965f,    0.189771647929618f,
    0.189991347511347f,    0.190211014065009f,    0.190430647564477f,
    0.190650247983633f,    0.190869815296374f,    0.191089349476607f,
    0.191308850498254f,    0.191528318335246f,    0.191747752961530f,
    0.191967154351062f,    0.192186522477811f,    0.192405857315761f,
    0.192625158838903f,    0.192844427021247f,    0.193063661836809f,
    0.193282863259621f,    0.193502031263726f,    0.193721165823180f,
    0.193940266912050f,    0.194159334504417f,    0.194378368574374f,
    0.194597369096023f,    0.194816336043484f,    0.195035269390884f,
    0.195254169112365f,    0.195473035182082f,    0.195691867574200f,
    0.195910666262898f,    0.196129431222366f,    0.196348162426807f,
    0.196566859850436f,    0.196785523467482f,    0.197004153252183f,
    0.197222749178791f,    0.197441311221572f,    0.197659839354801f,
    0.197878333552768f,    0.198096793789773f,    0.198315220040131f,
    0.198533612278166f,    0.198751970478218f,    0.198970294614635f,
    0.199188584661781f,    0.199406840594031f,    0.199625062385772f,
    0.199843250011403f,    0.200061403445335f,    0.200279522661994f,
    0.200497607635815f,    0.200715658341246f,    0.200933674752750f,
    0.201151656844797f,    0.201369604591875f,    0.201587517968481f,
    0.201805396949124f,    0.202023241508327f,    0.202241051620625f,
    0.202458827260564f,    0.202676568402703f,    0.202894275021614f,
    0.203111947091881f,    0.203329584588098f,    0.203547187484876f,
    0.203764755756833f,    0.203982289378603f,    0.204199788324830f,
    0.204417252570173f,    0.204634682089301f,    0.204852076856895f,
    0.205069436847651f,    0.205286762036273f,    0.195300241474869f,
    0.195517496983395f,    0.195734717613981f,    0.195951903341383f,
    0.196169054140369f,    0.196386169985719f,    0.196603250852226f,
    0.196820296714693f,    0.197037307547939f,    0.197254283326792f,
    0.197471224026094f,    0.197688129620699f,    0.197905000085473f,
    0.198121835395295f,    0.198338635525054f,    0.198555400449654f,
    0.198772130144011f,    0.198988824583052f,    0.199205483741716f,
    0.199422107594955f,    0.199638696117735f,    0.199855249285032f,
    0.200071767071834f,    0.200288249453143f,    0.200504696403972f,
    0.200721107899348f,    0.200937483914308f,    0.201153824423902f,
    0.201370129403194f,    0.201586398827258f,    0.201802632671180f,
    0.202018830910062f,    0.202234993519015f,    0.202451120473161f,
    0.202667211747639f,    0.202883267317596f,    0.203099287158194f,
    0.203315271244605f,    0.203531219552015f,    0.203747132055621f,
    0.203963008730634f,    0.204178849552277f,    0.204394654495782f,
    0.204610423536399f,    0.204826156649384f,    0.205041853810011f,
    0.205257514993562f,    0.205473140175333f,    0.205688729330634f,
    0.205904282434784f,    0.206119799463116f,    0.206335280390975f,
    0.206550725193719f,    0.206766133846718f,    0.206981506325353f,
    0.207196842605018f,    0.197208331738507f,    0.197423595546465f,
    0.197638823081710f,    0.197854014319685f,    0.198069169235847f,
    0.198284287805661f,    0.198499370004610f,    0.198714415808185f,
    0.198929425191892f,    0.199144398131246f,    0.199359334601778f,
    0.199574234579030f,    0.199789098038554f,    0.200003924955918f,
    0.200218715306700f,    0.200433469066490f,    0.200648186210891f,
    0.200862866715519f,    0.201077510556001f,    0.201292117707978f,
    0.201506688147100f,    0.201721221849034f,    0.201935718789455f,
    0.202150178944052f,    0.202364602288527f,    0.202578988798593f,
    0.202793338449976f,    0.203007651218415f,    0.203221927079658f,
    0.203436166009470f,    0.203650367983626f,    0.203864532977911f,
    0.204078660968127f,    0.204292751930084f,    0.204506805839607f,
    0.204720822672532f,    0.204934802404708f,    0.205148745011996f,
    0.205362650470268f,    0.205576518755411f,    0.205790349843322f,
    0.206004143709912f,    0.206217900331101f,    0.206431619682827f,
    0.196441490818421f,    0.196655135559069f,    0.196868742958131f,
    0.197082312991588f,    0.197295845635438f,    0.197509340865688f,
    0.197722798658360f,    0.197936218989487f,    0.198149601835112f,
    0.198362947171294f,    0.198576254974103f,    0.198789525219620f,
    0.199002757883940f,    0.199215952943169f,    0.199429110373427f,
    0.199642230150843f,    0.199855312251563f,    0.200068356651740f,
    0.200281363327544f,    0.200494332255154f,    0.200707263410764f,
    0.200920156770577f,    0.201133012310811f,    0.201345830007696f,
    0.201558609837472f,    0.201771351776394f,    0.201984055800729f,
    0.202196721886754f,    0.202409350010760f,    0.202621940149051f,
    0.202834492277942f,    0.203047006373760f,    0.203259482412845f,
    0.203471920371550f,    0.203684320226238f,    0.203896681953288f,
    0.204109005529087f,    0.204321290930037f,    0.204533538132551f,
    0.194541936190444f,    0.194754106925378f,    0.194966239391191f,
    0.195178333564346f,    0.195390389421317f,    0.195602406938593f,
    0.195814386092672f,    0.196026326860067f,    0.196238229217301f,
    0.196450093140911f,    0.196661918607446f,    0.196873705593467f,
    0.197085454075546f,    0.197297164030270f,    0.197508835434236f,
    0.197720468264055f,    0.197932062496348f,    0.198143618107750f,
    0.198355135074909f,    0.198566613374484f,    0.198778052983145f,
    0.198989453877578f,    0.199200816034477f,    0.199412139430552f,
    0.199623424042522f,    0.199834669847121f,    0.200045876821094f,
    0.200257044941199f,    0.200468174184204f,    0.200679264526893f,
    0.200890315946060f,    0.201101328418510f,    0.201312301921064f,
    0.201523236430552f,    0.201734131923817f,    0.191741177455104f,
    0.191951994846504f,    0.192162773152286f,    0.192373512349342f,
    0.192584212414577f,    0.192794873324908f,    0.193005495057265f,
    0.193216077588589f,    0.193426620895835f,    0.193637124955968f,
    0.193847589745967f,    0.194058015242823f,    0.194268401423539f,
    0.194478748265130f,    0.194689055744625f,    0.194899323839062f,
    0.195109552525495f,    0.195319741780988f,    0.195529891582617f,
    0.195740001907471f,    0.195950072732653f,    0.196160104035275f,
    0.196370095792464f,    0.196580047981357f,    0.196789960579106f,
    0.196999833562872f,    0.197209666909832f,    0.197419460597172f,
    0.197629214602091f,    0.197838928901803f,    0.198048603473531f,
    0.198258238294511f,    0.188264022419379f,    0.188473577670623f,
    0.188683093102902f,    0.188892568693502f,    0.189102004419720f,
    0.189311400258868f,    0.189520756188268f,    0.189730072185253f,
    0.189939348227172f,    0.190148584291383f,    0.190357780355258f,
    0.190566936396181f,    0.190776052391548f,    0.190985128318767f,
    0.191194164155259f,    0.191403159878457f,    0.191612115465806f,
    0.191821030894764f,    0.192029906142801f,    0.192238741187397f,
    0.192447536006049f,    0.192656290576263f,    0.192865004875557f,
    0.193073678881463f,    0.193282312571524f,    0.193490905923295f,
    0.193699458914346f,    0.193907971522257f,    0.194116443724619f,
    0.194324875499038f,    0.194533266823131f,    0.184537806751915f,
    0.184746117108257f,    0.184954386947198f,    0.185162616246405f,
    0.185370804983555f,    0.185578953136341f,    0.185787060682465f,
    0.185995127599643f,    0.186203153865602f,    0.186411139458082f,
    0.186619084354836f,    0.186826988533627f,    0.187034851972234f,
    0.187242674648445f,    0.187450456540062f,    0.187658197624897f,
    0.187865897880778f,    0.188073557285542f,    0.188281175817040f,
    0.188488753453134f,    0.188696290171701f,    0.188903785950626f,
    0.189111240767811f,    0.189318654601166f,    0.189526027428616f,
    0.189733359228097f,    0.189940649977559f,    0.190147899654962f,
    0.190355108238280f,    0.180358464782885f,    0.180565591112002f,
    0.180772676281026f,    0.180979720267981f,    0.181186723050902f,
    0.181393684607836f,    0.181600604916841f,    0.181807483955990f,
    0.182014321703366f,    0.182221118137065f,    0.182427873235197f,
    0.182634586975881f,    0.182841259337251f,    0.183047890297452f,
    0.183254479834642f,    0.183461027926990f,    0.183667534552678f,
    0.183873999689901f,    0.184080423316866f,    0.184286805411791f,
    0.184493145952909f,    0.184699444918461f,    0.184905702286705f,
    0.185111918035908f,    0.185318092144350f,    0.185524224590325f,
    0.185730315352137f,    0.185936364408104f,    0.175938560813942f,
    0.176144526393218f,    0.176350450201674f,    0.176556332217677f,
    0.176762172419604f,    0.176967970785847f,    0.177173727294808f,
    0.177379441924904f,    0.177585114654563f,    0.177790745462223f,
    0.177996334326338f,    0.178201881225372f,    0.178407386137802f,
    0.178612849042117f,    0.178818269916818f,    0.179023648740420f,
    0.179228985491448f,    0.179434280148441f,    0.179639532689949f,
    0.179844743094535f,    0.180049911340774f,    0.180255037407254f,
    0.180460121272574f,    0.180665162915346f,    0.180870162314195f,
    0.181075119447757f,    0.181280034294680f,    0.181484906833627f,
    0.171485926120658f,    0.171690713979683f,    0.171895459466788f,
    0.172100162560683f,    0.172304823240091f,    0.172509441483747f,
    0.172714017270397f,    0.172918550578801f,    0.173123041387730f,
    0.173327489675969f,    0.173531895422314f,    0.173736258605573f,
    0.173940579204567f,    0.174144857198129f,    0.174349092565105f,
    0.174553285284351f,    0.174757435334738f,    0.174961542695148f,
    0.175165607344476f,    0.175369629261627f,    0.175573608425522f,
    0.175777544815091f,    0.175981438409279f,    0.176185289187040f,
    0.176389097127344f,    0.176592862209170f,    0.176796584411511f,
    0.177000263713372f,    0.167000089171158f,    0.167203682609124f,
    0.167407233083699f,    0.167610740573936f,    0.167814205058903f,
    0.168017626517678f,    0.168221004929352f,    0.168424340273029f,
    0.168627632527822f,    0.168830881672862f,    0.169034087687287f,
    0.169237250550250f,    0.169440370240916f,    0.169643446738461f,
    0.169846480022075f,    0.170049470070959f,    0.170252416864327f,
    0.170455320381406f,    0.170658180601432f,    0.170860997503658f,
    0.171063771067346f,    0.171266501271771f,    0.171469188096220f,
    0.171671831519994f,    0.171874431522404f,    0.172076988082775f,
    0.172279501180443f,    0.162278159872144f,    0.162480585982465f,
    0.162682968568167f,    0.162885307608635f,    0.163087603083268f,
    0.163289854971476f,    0.163492063252680f,    0.163694227906317f,
    0.163896348911833f,    0.164098426248687f,    0.164300459896352f,
    0.164502449834311f,    0.164704396042060f,    0.164906298499108f,
    0.165108157184976f,    0.165309972079197f,    0.165511743161317f,
    0.165713470410892f,    0.165915153807493f,    0.166116793330702f,
    0.166318388960114f,    0.166519940675335f,    0.166721448455984f,
    0.166922912281694f,    0.167124332132106f,    0.167325707986878f,
    0.167527039825678f,    0.157524516705573f,    0.157725760451481f,
    0.157926960120494f,    0.158128115692331f,    0.158329227146721f,
    0.158530294463405f,    0.158731317622137f,    0.158932296602685f,
    0.159133231384826f,    0.159334121948353f,    0.159534968273067f,
    0.159735770338785f,    0.159936528125335f,    0.160137241612557f,
    0.160337910780302f,    0.160538535608436f,    0.160739116076836f,
    0.160939652165391f,    0.161140143854002f,    0.161340591122584f,
    0.161540993951062f,    0.161741352319375f,    0.161941666207473f,
    0.162141935595320f,    0.162342160462890f,    0.162542340790172f,
    0.162742476557164f,    0.162942567743880f,    0.152938803407730f,
    0.153138805373977f,    0.153338762700057f,    0.153538675366031f,
    0.153738543351972f,    0.153938366637967f,    0.154138145204113f,
    0.154337879030521f,    0.154537568097313f,    0.154737212384624f,
    0.154936811872602f,    0.155136366541405f,    0.155335876371206f,
    0.155535341342188f,    0.155734761434549f,    0.155934136628495f,
    0.156133466904249f,    0.156332752242043f,    0.156531992622123f,
    0.156731188024747f,    0.156930338430184f,    0.157129443818716f,
    0.157328504170639f,    0.157527519466260f,    0.157726489685896f,
    0.157925414809880f,    0.158124294818555f,    0.158323129692277f,
    0.148318108488803f,    0.148516853033737f,    0.148715552384859f,
    0.148914206522576f,    0.149112815427303f,    0.149311379079472f,
    0.149509897459523f,    0.149708370547911f,    0.149906798325103f,
    0.150105180771577f,    0.150303517867824f,    0.150501809594347f,
    0.150700055931663f,    0.150898256860299f,    0.151096412360795f,
    0.151294522413704f,    0.151492586999591f,    0.151690606099032f,
    0.151888579692616f,    0.152086507760946f,    0.152284390284635f,
    0.152482227244310f,    0.152680018620608f,    0.152877764394181f,
    0.153075464545691f,    0.153273119055814f,    0.153470727905238f,
    0.153668291074661f,    0.143661997622184f,    0.143859469373757f,
    0.144056895387502f,    0.144254275644170f,    0.144451610124522f,
    0.144648898809330f,    0.144846141679382f,    0.145043338715474f,
    0.145240489898417f,    0.145437595209035f,    0.145634654628161f,
    0.145831668136644f,    0.146028635715342f,    0.146225557345127f,
    0.146422433006884f,    0.146619262681509f,    0.146816046349909f,
    0.147012783993008f,    0.147209475591736f,    0.147406121127041f,
    0.147602720579879f,    0.147799273931220f,    0.147995781162048f,
    0.148192242253357f,    0.148388657186152f,    0.148585025941455f,
    0.148781348500295f,    0.148977624843717f,    0.149173854952777f,
    0.149370038808543f,    0.139362365469483f,    0.139558456761915f,
    0.139754501744331f,    0.139950500397849f,    0.140146452703599f,
    0.140342358642723f,    0.140538218196374f,    0.140734031345720f,
    0.140929798071938f,    0.141125518356222f,    0.141321192179772f,
    0.141516819523806f,    0.141712400369552f,    0.141907934698248f,
    0.142103422491149f,    0.142298863729519f,    0.142494258394634f,
    0.142689606467784f,    0.142884907930271f,    0.143080162763409f,
    0.143275370948524f,    0.143470532466954f,    0.143665647300050f,
    0.143860715429175f,    0.144055736835705f,    0.144250711501027f,
    0.144445639406540f,    0.144640520533658f,    0.144835354863804f,
    0.145030142378415f,    0.145224883058941f,    0.135215765964230f,
    0.135410412920980f,    0.135605012988065f,    0.135799566146984f,
    0.135994072379245f,    0.136188531666373f,    0.136382943989902f,
    0.136577309331380f,    0.136771627672365f,    0.136965898994431f,
    0.137160123279160f,    0.137354300508149f,    0.137548430663007f,
    0.137742513725355f,    0.137936549676826f,    0.138130538499066f,
    0.138324480173732f,    0.138518374682495f,    0.138712222007036f,
    0.138906022129051f,    0.139099775030246f,    0.139293480692340f,
    0.139487139097066f,    0.139680750226166f,    0.139874314061398f,
    0.140067830584528f,    0.140261299777338f,    0.140454721621621f,
    0.140648096099182f,    0.140841423191837f,    0.141034702881418f,
    0.141227935149766f,    0.141421119978735f,    0.131410446427580f,
    0.131603536323403f,    0.131796578725485f,    0.131989573615728f,
    0.132182520976047f,    0.132375420788372f,    0.132568273034642f,
    0.132761077696810f,    0.132953834756840f,    0.133146544196710f,
    0.133339205998410f,    0.133531820143939f,    0.133724386615314f,
    0.133916905394560f,    0.134109376463715f,    0.134301799804830f,
    0.134494175399969f,    0.134686503231206f,    0.134878783280630f,
    0.135071015530340f,    0.135263199962448f,    0.135455336559080f,
    0.135647425302371f,    0.135839466174471f,    0.136031459157541f,
    0.136223404233754f,    0.136415301385298f,    0.136607150594369f,
    0.136798951843178f,    0.136990705113949f,    0.137182410388915f,
    0.137374067650325f,    0.137565676880437f,    0.137757238061525f,
    0.137948751175871f,    0.127936405283160f,    0.128127822210926f,
    0.128319191018876f,    0.128510511689345f,    0.128701784204678f,
    0.128893008547232f,    0.129084184699378f,    0.129275312643499f,
    0.129466392361988f,    0.129657423837253f,    0.129848407051713f,
    0.130039341987800f,    0.130230228627958f,    0.130421066954641f,
    0.130611856950320f,    0.130802598597474f,    0.130993291878596f,
    0.131183936776193f,    0.131374533272780f,    0.131565081350888f,
    0.131755580993060f,    0.131946032181848f,    0.132136434899821f,
    0.132326789129556f,    0.132517094853645f,    0.132707352054692f,
    0.132897560715312f,    0.133087720818134f,    0.133277832345797f,
    0.133467895280954f,    0.133657909606270f,    0.133847875304422f,
    0.134037792358100f,    0.134227660750005f,    0.134417480462852f,
    0.134607251479366f,    0.134796973782287f,    0.134986647354364f,
    0.135176272178362f,    0.125162037314443f,    0.125351564590619f,
    0.125541043067079f,    0.125730472726634f,    0.125919853552109f,
    0.126109185526341f,    0.126298468632178f,    0.126487702852483f,
    0.126676888170129f,    0.126866024568002f,    0.127055112029000f,
    0.127244150536033f,    0.127433140072025f,    0.127622080619911f,
    0.127810972162637f,    0.127999814683163f,    0.128188608164462f,
    0.128377352589517f,    0.128566047941326f,    0.128754694202896f,
    0.128943291357249f,    0.129131839387418f,    0.129320338276448f,
    0.129508788007399f,    0.129697188563339f,    0.129885539927351f,
    0.130073842082530f,    0.130262095011983f,    0.130450298698829f,
    0.130638453126200f,    0.130826558277239f,    0.131014614135104f,
    0.131202620682961f,    0.131390577903992f,    0.131578485781390f,
    0.131766344298360f,    0.131954153438120f,    0.132141913183899f,
    0.132329623518940f,    0.132517284426497f,    0.132704895889836f,
    0.132892457892237f,    0.133079970416991f,    0.133267433447401f,
    0.133454846966783f,    0.133642210958465f,    0.123625714483175f,
    0.123812979369491f,    0.124000194678165f,    0.124187360392574f,
    0.124374476496108f,    0.124561542972168f,    0.124748559804169f,
    0.124935526975537f,    0.125122444469710f,    0.125309312270140f,
    0.125496130360289f,    0.125682898723633f,    0.125869617343660f,
    0.126056286203869f,    0.126242905287774f,    0.126429474578898f,
    0.126615994060779f,    0.126802463716965f,    0.126988883531018f,
    0.127175253486512f,    0.127361573567033f,    0.127547843756178f,
    0.127734064037559f,    0.127920234394799f,    0.128106354811532f,
    0.128292425271406f,    0.128478445758080f,    0.128664416255227f,
    0.128850336746530f,    0.129036207215687f,    0.129222027646406f,
    0.129407798022407f,    0.129593518327426f,    0.129779188545206f,
    0.129964808659507f,    0.130150378654098f,    0.130335898512761f,
    0.130521368219292f,    0.130706787757498f,    0.130892157111197f,
    0.131077476264222f,    0.131262745200416f,    0.131447963903636f,
    0.131633132357750f,    0.131818250546639f,    0.132003318454195f,
    0.132188336064325f,    0.132373303360945f,    0.132558220327986f,
    0.132743086949389f,    0.132927903209110f,    0.133112669091115f,
    0.133297384579382f,    0.133482049657903f,    0.133666664310682f,
    0.133851228521735f,    0.134035742275089f,    0.134220205554786f,
    0.134404618344876f,    0.134588980629427f,    0.124569481469902f,
    0.124753742695616f,    0.124937953368058f,    0.125122113471342f,
    0.125306222989596f,    0.125490281906956f,    0.125674290207575f,
    0.125858247875615f,    0.126042154895252f,    0.126226011250674f,
    0.126409816926081f,    0.126593571905685f,    0.126777276173711f,
    0.126960929714395f,    0.127144532511988f,    0.127328084550749f,
    0.127511585814954f,    0.127695036288888f,    0.127878435956849f,
    0.128061784803149f,    0.128245082812109f,    0.128428329968065f,
    0.128611526255365f,    0.128794671658368f,    0.128977766161446f,
    0.129160809748984f,    0.129343802405377f,    0.129526744115036f,
    0.129709634862381f,    0.129892474631845f,    0.130075263407874f,
    0.130258001174926f,    0.130440687917472f,    0.130623323619993f,
    0.130805908266985f,    0.130988441842955f,    0.131170924332422f,
    0.131353355719918f,    0.131535735989987f,    0.131718065127185f,
    0.131900343116080f,    0.132082569941253f,    0.132264745587298f,
    0.132446870038820f,    0.132628943280436f,    0.132810965296776f,
    0.132992936072482f,    0.133174855592210f,    0.133356723840624f,
    0.133538540802406f,    0.133720306462246f,    0.133902020804847f,
    0.134083683814926f,    0.134265295477211f,    0.134446855776441f,
    0.134628364697371f,    0.134809822224764f,    0.134991228343398f,
    0.135172583038063f,    0.135353886293561f,    0.135535138094705f,
    0.135716338426323f,    0.135897487273252f,    0.136078584620344f,
    0.136259630452462f,    0.136440624754481f,    0.136621567511290f,
    0.136802458707789f,    0.136983298328889f,    0.137164086359516f,
    0.137344822784607f,    0.137525507589110f,    0.137706140757987f,
    0.137886722276213f,    0.138067252128772f,    0.138247730300664f,
    0.138428156776899f,    0.138608531542500f,    0.138788854582503f,
    0.138969125881954f,    0.139149345425913f,    0.139329513199453f,
    0.139509629187658f,    0.139689693375625f,    0.139869705748462f,
    0.140049666291290f,    0.140229574989243f,    0.140409431827467f,
    0.140589236791119f,    0.140768989865371f,    0.140948691035404f,
    0.141128340286414f,    0.141307937603607f,    0.141487482972203f,
    0.141666976377434f,    0.141846417804544f,    0.142025807238789f,
    0.142205144665437f,    0.142384430069770f,    0.142563663437080f,
    0.142742844752673f,    0.142921974001866f,    0.143101051169990f,
    0.143280076242387f,    0.143459049204410f,    0.143637970041428f,
    0.143816838738820f,    0.143995655281975f,    0.144174419656299f,
    0.144353131847207f,    0.144531791840127f,    0.144710399620500f,
    0.144888955173778f,    0.145067458485427f,    0.145245909540924f,
    0.145424308325758f,    0.145602654825432f,    0.145780949025460f,
    0.145959190911367f,    0.146137380468694f,    0.146315517682990f,
    0.146493602539820f,    0.146671635024758f,    0.146849615123393f,
    0.147027542821324f,    0.147205418104165f,    0.147383240957540f,
    0.147561011367085f,    0.147738729318451f,    0.147916394797298f,
    0.148094007789301f,    0.148271568280146f,    0.148449076255531f,
    0.148626531701166f,    0.148803934602776f,    0.148981284946094f,
    0.149158582716869f,    0.149335827900860f,    0.149513020483839f,
    0.149690160451591f,    0.149867247789913f,    0.150044282484613f,
    0.150221264521513f,    0.150398193886446f,    0.150575070565257f,
    0.150751894543806f,    0.150928665807962f,    0.151105384343607f,
    0.151282050136638f,    0.151458663172960f,    0.151635223438494f,
    0.151811730919171f,    0.151988185600936f,    0.152164587469744f,
    0.152340936511564f,    0.152517232712377f,    0.152693476058176f,
    0.152869666534967f,    0.153045804128767f,    0.153221888825607f,
    0.153397920611528f,    0.153573899472585f,    0.153749825394845f,
    0.153925698364388f,    0.154101518367304f,    0.154277285389697f,
    0.154452999417684f,    0.154628660437392f,    0.154804268434962f,
    0.154979823396547f,    0.155155325308312f,    0.155330774156434f,
    0.155506169927103f,    0.155681512606521f,    0.155856802180902f,
    0.145828227713860f,    0.146003411036860f,    0.146178541213538f,
    0.146353618230160f,    0.146528642073000f,    0.146703612728346f,
    0.146878530182500f,    0.147053394421772f,    0.147228205432488f,
    0.147402963200985f,    0.147577667713612f,    0.147752318956730f,
    0.147926916916715f,    0.148101461579951f,    0.148275952932838f,
    0.148450390961786f,    0.148624775653217f,    0.148799106993568f,
    0.148973384969286f,    0.149147609566831f,    0.149321780772674f,
    0.149495898573301f,    0.149669962955208f,    0.149843973904904f,
    0.150017931408910f,    0.150191835453759f,    0.150365686025999f,
    0.150539483112186f,    0.150713226698891f,    0.150886916772697f,
    0.151060553320199f,    0.151234136328003f,    0.151407665782731f,
    0.151581141671012f,    0.151754563979492f,    0.151927932694826f,
    0.152101247803684f,    0.152274509292746f,    0.152447717148705f,
    0.152620871358268f,    0.152793971908151f,    0.152967018785084f,
    0.153140011975811f,    0.153312951467086f,    0.153485837245675f,
    0.153658669298358f,    0.153831447611926f,    0.154004172173183f,
    0.154176842968946f,    0.154349459986041f,    0.154522023211310f,
    0.154694532631606f,    0.154866988233794f,    0.155039390004751f,
    0.155211737931367f,    0.155384032000544f,    0.155556272199196f,
    0.155728458514250f,    0.155900590932645f,    0.156072669441331f,
    0.156244694027273f,    0.156416664677445f,    0.156588581378836f,
    0.156760444118446f,    0.156932252883288f,    0.157104007660386f,
    0.157275708436777f,    0.157447355199512f,    0.157618947935650f,
    0.157790486632267f,    0.157961971276449f,    0.158133401855293f,
    0.158304778355912f,    0.158476100765427f,    0.158647369070975f,
    0.158818583259702f,    0.158989743318769f,    0.159160849235348f,
    0.159331900996624f,    0.159502898589793f,    0.159673842002064f,
    0.159844731220658f,    0.160015566232809f,    0.160186347025763f,
    0.160357073586779f,    0.160527745903126f,    0.160698363962087f,
    0.160868927750957f,    0.161039437257044f,    0.161209892467666f,
    0.161380293370157f,    0.161550639951860f,    0.161720932200131f,
    0.161891170102339f,    0.151857542723253f,    0.152027671895491f,
    0.152197746683846f,    0.152367767075735f,    0.152537733058589f,
    0.152707644619850f,    0.152877501746973f,    0.153047304427424f,
    0.153217052648684f,    0.153386746398243f,    0.153556385663605f,
    0.153725970432286f,    0.153895500691816f,    0.154064976429733f,
    0.154234397633591f,    0.154403764290956f,    0.154573076389405f,
    0.154742333916527f,    0.154911536859925f,    0.155080685207213f,
    0.155249778946017f,    0.155418818063977f,    0.155587802548743f,
    0.155756732387980f,    0.155925607569361f,    0.156094428080577f,
    0.156263193909327f,    0.156431905043323f,    0.156600561470291f,
    0.156769163177967f,    0.156937710154101f,    0.157106202386455f,
    0.157274639862803f,    0.157443022570930f,    0.157611350498636f,
    0.157779623633731f,    0.157947841964038f,    0.158116005477393f,
    0.158284114161644f,    0.158452168004650f,    0.158620166994283f,
    0.158788111118428f,    0.158956000364983f,    0.159123834721855f,
    0.159291614176966f,    0.159459338718251f,    0.159627008333654f,
    0.159794623011134f,    0.159962182738662f,    0.160129687504220f,
    0.160297137295804f,    0.160464532101419f,    0.160631871909088f,
    0.160799156706840f,    0.160966386482721f,    0.150929750302173f,
    0.151096869998492f,    0.151263934637146f,    0.151430944206227f,
    0.151597898693841f,    0.151764798088106f,    0.151931642377153f,
    0.152098431549124f,    0.152265165592173f,    0.152431844494468f,
    0.152598468244187f,    0.152765036829522f,    0.152931550238678f,
    0.153098008459870f,    0.153264411481326f,    0.153430759291287f,
    0.153597051878007f,    0.153763289229750f,    0.153929471334795f,
    0.154095598181429f,    0.154261669757957f,    0.154427686052692f,
    0.154593647053961f,    0.154759552750102f,    0.154925403129467f,
    0.155091198180420f,    0.155256937891336f,    0.155422622250604f,
    0.155588251246622f,    0.155753824867805f,    0.155919343102577f,
    0.156084805939376f,    0.156250213366650f,    0.156415565372862f,
    0.156580861946485f,    0.156746103076005f,    0.156911288749923f,
    0.157076418956747f,    0.157241493685002f,    0.157406512923222f,
    0.157571476659956f,    0.157736384883764f,    0.157901237583217f,
    0.147862223824288f,    0.148026965440799f,    0.148191651498746f,
    0.148356281986751f,    0.148520856893447f,    0.148685376207480f,
    0.148849839917509f,    0.149014248012204f,    0.149178600480248f,
    0.149342897310336f,    0.149507138491175f,    0.149671324011486f,
    0.149835453859999f,    0.149999528025459f,    0.150163546496623f,
    0.150327509262259f,    0.150491416311149f,    0.150655267632085f,
    0.150819063213874f,    0.150982803045333f,    0.151146487115293f,
    0.151310115412596f,    0.151473687926096f,    0.151637204644661f,
    0.151800665557170f,    0.151964070652515f,    0.152127419919599f,
    0.152290713347339f,    0.152453950924663f,    0.152617132640512f,
    0.152780258483839f,    0.152943328443608f,    0.153106342508799f,
    0.153269300668400f,    0.153432202911413f,    0.153595049226854f,
    0.153757839603748f,    0.153920574031135f,    0.154083252498065f,
    0.144042064070991f,    0.144204630584213f,    0.144367141104205f,
    0.144529595620070f,    0.144691994120919f,    0.144854336595877f,
    0.145016623034082f,    0.145178853424683f,    0.145341027756842f,
    0.145503146019733f,    0.145665208202542f,    0.145827214294467f,
    0.145989164284721f,    0.146151058162526f,    0.146312895917117f,
    0.146474677537742f,    0.146636403013661f,    0.146798072334147f,
    0.146959685488484f,    0.147121242465970f,    0.147282743255912f,
    0.147444187847633f,    0.147605576230467f,    0.147766908393758f,
    0.147928184326867f,    0.148089404019162f,    0.148250567460028f,
    0.148411674638859f,    0.148572725545063f,    0.148733720168058f,
    0.148894658497279f,    0.149055540522167f,    0.149216366232181f,
    0.149377135616789f,    0.139334037742859f,    0.139494694445110f,
    0.139655294790435f,    0.139815838768352f,    0.139976326368392f,
    0.140136757580096f,    0.140297132393020f,    0.140457450796730f,
    0.140617712780806f,    0.140777918334840f,    0.140938067448435f,
    0.141098160111208f,    0.141258196312788f,    0.141418176042814f,
    0.141578099290940f,    0.141737966046832f,    0.141897776300166f,
    0.142057530040633f,    0.142217227257936f,    0.142376867941787f,
    0.142536452081915f,    0.142695979668058f,    0.142855450689968f,
    0.143014865137407f,    0.143174223000153f,    0.143333524267992f,
    0.143492768930726f,    0.143651956978168f,    0.143811088400141f,
    0.143970163186484f,    0.144129181327045f,    0.144288142811688f,
    0.144447047630285f,    0.134402084850112f,    0.134560876306290f,
    0.134719611066120f,    0.134878289119524f,    0.135036910456437f,
    0.135195475066808f,    0.135353982940597f,    0.135512434067775f,
    0.135670828438328f,    0.135829166042253f,    0.135987446869558f,
    0.136145670910266f,    0.136303838154409f,    0.136461948592034f,
    0.136620002213200f,    0.136777999007977f,    0.136935938966447f,
    0.137093822078707f,    0.137251648334863f,    0.137409417725035f,
    0.137567130239356f,    0.137724785867969f,    0.137882384601031f,
    0.138039926428711f,    0.138197411341190f,    0.138354839328662f,
    0.138512210381332f,    0.138669524489419f,    0.138826781643152f,
    0.138983981832774f,    0.128937314125928f,    0.129094400358105f,
    0.129251429596973f,    0.129408401832822f,    0.129565317055958f,
    0.129722175256696f,    0.129878976425365f,    0.130035720552306f,
    0.130192407627872f,    0.130349037642427f,    0.130505610586351f,
    0.130662126450032f,    0.130818585223873f,    0.130974986898289f,
    0.131131331463705f,    0.131287618910562f,    0.131443849229311f,
    0.131600022410415f,    0.131756138444350f,    0.131912197321604f,
    0.132068199032678f,    0.132224143568085f,    0.132380030918349f,
    0.132535861074008f,    0.132691634025612f,    0.132847349763722f,
    0.133003008278912f,    0.133158609561769f,    0.133314153602892f,
    0.133469640392892f,    0.123421258999779f,    0.123576631259414f,
    0.123731946239833f,    0.123887203931694f,    0.124042404325672f,
    0.124197547412451f,    0.124352633182726f,    0.124507661627209f,
    0.124662632736620f,    0.124817546501693f,    0.124972402913174f,
    0.125127201961822f,    0.125281943638407f,    0.125436627933713f,
    0.125591254838533f,    0.125745824343677f,    0.125900336439963f,
    0.126054791118225f,    0.126209188369305f,    0.126363528184061f,
    0.126517810553362f,    0.126672035468088f,    0.126826202919134f,
    0.126980312897405f,    0.127134365393820f,    0.127288360399307f,
    0.127442297904811f,    0.127596177901286f,    0.117546189457086f,
    0.117699954408416f,    0.117853661823656f,    0.118007311693808f,
    0.118160904009891f,    0.118314438762931f,    0.118467915943971f,
    0.118621335544062f,    0.118774697554272f,    0.118928001965676f,
    0.119081248769366f,    0.119234437956443f,    0.119387569518022f,
    0.119540643445230f,    0.119693659729205f,    0.119846618361101f,
    0.119999519332079f,    0.120152362633316f,    0.120305148256000f,
    0.120457876191332f,    0.120610546430524f,    0.120763158964802f,
    0.120915713785403f,    0.121068210883577f,    0.121220650250584f,
    0.121373031877701f,    0.121525355756213f,    0.121677621877418f,
    0.111626019310016f,    0.111778169890555f,    0.111930262687758f,
    0.112082297692972f,    0.112234274897559f,    0.112386194292890f,
    0.112538055870350f,    0.112689859621336f,    0.112841605537257f,
    0.112993293609536f,    0.113144923829605f,    0.113296496188912f,
    0.113448010678914f,    0.113599467291082f,    0.113750866016899f,
    0.113902206847861f,    0.114053489775474f,    0.114204714791260f,
    0.114355881886749f,    0.114506991053486f,    0.114658042283029f,
    0.114809035566945f,    0.114959970896817f,    0.115110848264237f,
    0.115261667660812f,    0.115412429078159f,    0.115563132507910f,
    0.115713777941706f,    0.105660554448589f,    0.105811083865453f,
    0.105961555261365f,    0.106111968628015f,    0.106262323957108f,
    0.106412621240361f,    0.106562860469501f,    0.106713041636271f,
    0.106863164732422f,    0.107013229749721f,    0.107163236679946f,
    0.107313185514885f,    0.107463076246342f,    0.107612908866132f,
    0.107762683366080f,    0.107912399738026f,    0.108062057973823f,
    0.108211658065332f,    0.108361200004431f,    0.108510683783007f,
    0.108660109392962f,    0.108809476826207f,    0.108958786074669f,
    0.109108037130284f,    0.109257229985002f,    0.109406364630786f,
    0.109555441059609f,    0.109704459263458f,    0.0996496083117188f,
    0.0997985100416282f,   0.0999473535225969f,   0.100096138746660f,
    0.100244865705867f,    0.100393534392276f,    0.100542144797961f,
    0.100690696915006f,    0.100839190735508f,    0.100987626251577f,
    0.101136003455335f,    0.101284322338914f,    0.101432582894463f,
    0.101580785114137f,    0.101728928990110f,    0.101877014514563f,
    0.102025041679692f,    0.102173010477705f,    0.102320920900821f,
    0.102468772941273f,    0.102616566591305f,    0.102764301843173f,
    0.102911978689148f,    0.103059597121509f,    0.103207157132551f,
    0.103354658714579f,    0.103502101859912f,    0.0934456756382679f,
    0.0935930018872135f,   0.0937402696764917f,   0.0938874789984700f,
    0.0940346298455276f,   0.0941817222100565f,   0.0943287560844607f,
    0.0944757314611567f,   0.0946226483325731f,   0.0947695066911507f,
    0.0949163065293431f,   0.0950630478396154f,   0.0952097306144455f,
    0.0953563548463236f,   0.0955029205277519f,   0.0956494276512452f,
    0.0957958762093304f,   0.0959422661945465f,   0.0960885975994451f,
    0.0962348704165899f,   0.0963810846385570f,   0.0965272402579347f,
    0.0966733372673236f,   0.0968193756593367f,   0.0969653554265990f,
    0.0971112765617480f,   0.0972571390574334f,   0.0974029429063172f,
    0.0873448771784613f,   0.0874905637117771f,   0.0876361915763511f,
    0.0877817607648946f,   0.0879272712701305f,   0.0880727230847950f,
    0.0882181162016359f,   0.0883634506134133f,   0.0885087263129001f,
    0.0886539432928808f,   0.0887991015461526f,   0.0889442010655249f,
    0.0890892418438194f,   0.0892342238738701f,   0.0893791471485232f,
    0.0895240116606371f,   0.0896688174030826f,   0.0898135643687430f,
    0.0899582525505135f,   0.0901028819413016f,   0.0902474525340274f,
    0.0903919643216231f,   0.0905364172970332f,   0.0906808114532143f,
    0.0908251467831358f,   0.0909694232797786f,   0.0911136409361365f,
    0.0912577997452155f,   0.0811980887774211f,   0.0813421298710088f,
    0.0814861120964089f,   0.0816300354466764f,   0.0817738999148785f,
    0.0819177054940949f,   0.0820614521774175f,   0.0822051399579502f,
    0.0823487688288098f,   0.0824923387831246f,   0.0826358498140358f,
    0.0827793019146968f,   0.0829226950782728f,   0.0830660292979419f,
    0.0832093045668942f,   0.0833525208783320f,   0.0834956782254701f,
    0.0836387766015353f,   0.0837818159997669f,   0.0839247964134165f,
    0.0840677178357478f,   0.0842105802600369f,   0.0843533836795723f,
    0.0844961280876545f,   0.0846388134775965f,   0.0847814398427233f,
    0.0849240071763727f,   0.0850665154718944f,   0.0852089647226503f,
    0.0751475439994024f,   0.0752898751407623f,   0.0754321472175162f,
    0.0755743602230756f,   0.0757165141508637f,   0.0758586089943163f,
    0.0760006447468815f,   0.0761426214020196f,   0.0762845389532032f,
    0.0764263973939170f,   0.0765681967176584f,   0.0767099369179366f,
    0.0768516179882736f,   0.0769932399222031f,   0.0771348027132715f,
    0.0772763063550374f,   0.0774177508410716f,   0.0775591361649572f,
    0.0777004623202896f,   0.0778417293006766f,   0.0779829370997380f,
    0.0781240857111062f,   0.0782651751284255f,   0.0784062053453531f,
    0.0785471763555577f,   0.0786880881527207f,   0.0788289407305362f,
    0.0789697340827096f,   0.0791104682029593f,   0.0792511430850161f,
    0.0691879478000098f,   0.0693285041869207f,   0.0694690013169037f,
    0.0696094391837383f,   0.0697498177812166f,   0.0698901371031425f,
    0.0700303971433326f,   0.0701705978956158f,   0.0703107393538329f,
    0.0704508215118373f,   0.0705908443634948f,   0.0707308079026829f,
    0.0708707121232921f,   0.0710105570192245f,   0.0711503425843951f,
    0.0712900688127309f,   0.0714297356981711f,   0.0715693432346672f,
    0.0717088914161831f,   0.0718483802366950f,   0.0719878096901912f,
    0.0721271797706725f,   0.0722664904721519f,   0.0724057417886545f,
    0.0725449337142179f,   0.0726840662428920f,   0.0728231393687387f,
    0.0729621530858327f,   0.0731011073882604f,   0.0732400022701208f,
    0.0733788377255252f,   0.0735176137485970f,   0.0634525194108596f,
    0.0635911765516861f,   0.0637297742426242f,   0.0638683124778465f,
    0.0640067912515382f,   0.0641452105578962f,   0.0642835703911301f,
    0.0644218707454617f,   0.0645601116151250f,   0.0646982929943664f,
    0.0648364148774443f,   0.0649744772586297f,   0.0651124801322058f,
    0.0652504234924680f,   0.0653883073337242f,   0.0655261316502941f,
    0.0656638964365102f,   0.0658016016867171f,   0.0659392473952715f,
    0.0660768335565426f,   0.0662143601649120f,   0.0663518272147731f,
    0.0664892347005321f,   0.0666265826166071f,   0.0667638709574289f,
    0.0669010997174400f,   0.0670382688910958f,   0.0671753784728635f,
    0.0673124284572228f,   0.0674494188386658f,   0.0675863496116965f,
    0.0677232207708315f,   0.0678600323105998f,   0.0679967842255422f,
    0.0579296655875996f,   0.0580662982365629f,   0.0582028712443972f,
    0.0583393846056928f,   0.0584758383150523f,   0.0586122323670904f,
    0.0587485667564341f,   0.0588848414777228f,   0.0590210565256080f,
    0.0591572118947538f,   0.0592933075798361f,   0.0594293435755436f,
    0.0595653198765770f,   0.0597012364776492f,   0.0598370933734855f,
    0.0599728905588236f,   0.0601086280284134f,   0.0602443057770169f,
    0.0603799237994086f,   0.0605154820903751f,   0.0606509806447155f,
    0.0607864194572412f,   0.0609217985227755f,   0.0610571178361543f,
    0.0611923773922258f,   0.0613275771858503f,   0.0614627172119006f,
    0.0615977974652616f,   0.0617328179408305f,   0.0618677786335169f,
    0.0620026795382425f,   0.0621375206499414f,   0.0622723019635602f,
    0.0624070234740572f,   0.0625416851764035f,   0.0626762870655824f,
    0.0526070182139767f,   0.0527415004618193f,   0.0528759228815178f,
    0.0530102854681045f,   0.0531445882166238f,   0.0532788311221330f,
    0.0534130141797010f,   0.0535471373844093f,   0.0536812007313518f,
    0.0538152042156344f,   0.0539491478323753f,   0.0540830315767053f,
    0.0542168554437671f,   0.0543506194287161f,   0.0544843235267195f,
    0.0546179677329570f,   0.0547515520426208f,   0.0548850764509150f,
    0.0550185409530562f,   0.0551519455442734f,   0.0552852902198074f,
    0.0554185749749120f,   0.0555517998048525f,   0.0556849647049072f,
    0.0558180696703662f,   0.0559511146965319f,   0.0560840997787193f,
    0.0562170249122554f,   0.0563498900924798f,   0.0564826953147438f,
    0.0566154405744115f,   0.0567481258668592f,   0.0568807511874753f,
    0.0570133165316607f,   0.0571458218948283f,   0.0572782672724036f,
    0.0574106526598240f,   0.0575429780525398f,   0.0576752434460127f,
    0.0578074488357176f,   0.0579395942171410f,   0.0478678686631696f,
    0.0479998940145396f,   0.0481318593441621f,   0.0482637646475730f,
    0.0483956099203204f,   0.0485273951579650f,   0.0486591203560793f,
    0.0487907855102481f,   0.0489223906160692f,   0.0490539356691518f,
    0.0491854206651178f,   0.0493168455996015f,   0.0494482104682491f,
    0.0495795152667194f,   0.0497107599906833f,   0.0498419446358243f,
    0.0499730691978377f,   0.0501041336724313f,   0.0502351380553253f,
    0.0503660823422521f,   0.0504969665289563f,   0.0506277906111949f,
    0.0507585545847371f,   0.0508892584453644f,   0.0510199021888707f,
    0.0511504858110618f,   0.0512810093077563f,   0.0514114726747848f,
    0.0515418759079902f,   0.0516722190032277f,   0.0518025019563648f,
    0.0519327247632811f,   0.0520628874198688f,   0.0521929899220323f,
    0.0523230322656880f,   0.0524530144467649f,   0.0525829364612041f,
    0.0527127983049592f,   0.0528425999739958f,   0.0529723414642920f,
    0.0531020227718380f,   0.0532316438926364f,   0.0533612048227022f,
    0.0534907055580622f,   0.0536201460947562f,   0.0537495264288357f,
    0.0538788465563647f,   0.0540081064734196f,   0.0541373061760887f,
    0.0542664456604730f,   0.0441917140000732f,   0.0443207330362394f,
    0.0444496918424971f,   0.0445785904149960f,   0.0447074287498985f,
    0.0448362068433790f,   0.0449649246916242f,   0.0450935822908335f,
    0.0452221796372181f,   0.0453507167270016f,   0.0454791935564199f,
    0.0456076101217214f,   0.0457359664191663f,   0.0458642624450276f,
    0.0459924981955904f,   0.0461206736671516f,   0.0462487888560214f,
    0.0463768437585213f,   0.0465048383709856f,   0.0466327726897607f,
    0.0467606467112054f,   0.0468884604316909f,   0.0470162138476001f,
    0.0471439069553288f,   0.0472715397512850f,   0.0473991122318885f,
    0.0475266243935721f,   0.0476540762327803f,   0.0477814677459702f,
    0.0479087989296110f,   0.0480360697801842f,   0.0481632802941839f,
    0.0482904304681158f,   0.0484175202984986f,   0.0485445497818628f,
    0.0486715189147517f,   0.0487984276937201f,   0.0489252761153358f,
    0.0490520641761785f,   0.0491787918728405f,   0.0493054592019259f,
    0.0494320661600513f,   0.0495586127438459f,   0.0496850989499509f,
    0.0498115247750195f,   0.0499378902157179f,   0.0500641952687237f,
    0.0501904399307276f,   0.0503166241984321f,   0.0504427480685521f,
    0.0505688115378147f,   0.0506948146029596f,   0.0508207572607382f,
    0.0509466395079148f,   0.0510724613412656f,   0.0511982227575792f,
    0.0513239237536566f,   0.0514495643263109f,   0.0515751444723674f,
    0.0517006641886638f,   0.0518261234720504f,   0.0519515223193892f,
    0.0520768607275548f,   0.0522021386934342f,   0.0523273562139263f,
    0.0524525132859427f,   0.0525776099064070f,   0.0527026460722553f,
    0.0426238108578230f,   0.0427487261052961f,   0.0428735808890346f,
    0.0429983752060237f,   0.0431231090532607f,   0.0432477824277554f,
    0.0433723953265295f,   0.0434969477466176f,   0.0436214396850659f,
    0.0437458711389333f,   0.0438702421052907f,   0.0439945525812217f,
    0.0441188025638218f,   0.0442429920501990f,   0.0443671210374733f,
    0.0444911895227774f,   0.0446151975032558f,   0.0447391449760657f,
    0.0448630319383766f,   0.0449868583873697f,   0.0451106243202392f,
    0.0452343297341911f,   0.0453579746264439f,   0.0454815589942283f,
    0.0456050828347873f,   0.0457285461453763f,   0.0458519489232627f,
    0.0459752911657265f,   0.0460985728700598f,   0.0462217940335669f,
    0.0463449546535647f,   0.0464680547273820f,   0.0465910942523600f,
    0.0467140732258525f,   0.0468369916452251f,   0.0469598495078560f,
    0.0470826468111357f,   0.0472053835524666f,   0.0473280597292639f,
    0.0474506753389546f,   0.0475732303789784f,   0.0476957248467870f,
    0.0478181587398445f,   0.0479405320556273f,   0.0480628447916239f,
    0.0481850969453354f,   0.0483072885142748f,   0.0484294194959678f,
    0.0485514898879519f,   0.0486734996877775f,   0.0487954488930066f,
    0.0489173375012139f,   0.0490391655099864f,   0.0491609329169232f,
    0.0492826397196357f,   0.0494042859157477f,   0.0495258715028952f,
    0.0496473964787265f,   0.0497688608409022f,   0.0498902645870951f,
    0.0500116077149904f,   0.0501328902222853f,   0.0502541121066900f,
    0.0503752733659261f,   0.0504963739977278f,   0.0506174139998419f,
    0.0507383933700272f,   0.0508593121060547f,   0.0509801702057078f,
    0.0511009676667822f,   0.0512217044870858f,   0.0513423806644390f,
    0.0514629961966743f,   0.0515835510816363f,   0.0517040453171823f,
    0.0518244789011815f,   0.0519448518315156f,   0.0520651641060786f,
    0.0521854157227766f,   0.0523056066795282f,   0.0524257369742641f,
    0.0525458066049274f,   0.0526658155694733f,   0.0527857638658696f,
    0.0529056514920961f,   0.0530254784461450f,   0.0531452447260207f,
    0.0532649503297400f,   0.0533845952553320f,   0.0535041795008378f,
    0.0536237030643112f,   0.0537431659438180f,   0.0538625681374362f,
    0.0539819096432564f,   0.0541011904593813f,   0.0542204105839259f,
    0.0543395700150173f,   0.0544586687507953f,   0.0545777067894117f,
    0.0546966841290304f,   0.0548156007678281f,   0.0549344567039931f,
    0.0550532519357268f,   0.0551719864612422f,   0.0552906602787649f,
    0.0554092733865326f,   0.0555278257827956f,   0.0556463174658160f,
    0.0557647484338686f,   0.0558831186852405f,   0.0560014282182306f,
    0.0561196770311506f,   0.0562378651223242f,   0.0563559924900874f,
    0.0564740591327887f,   0.0565920650487887f,   0.0567100102364602f,
    0.0568278946941885f,   0.0569457184203710f,   0.0570634814134173f,
    0.0571811836717497f,   0.0572988251938025f,   0.0574164059780221f,
    0.0575339260228674f,   0.0576513853268097f,   0.0577687838883325f,
    0.0578861217059312f,   0.0580033987781141f,   0.0581206151034013f,
    0.0582377706803254f,   0.0583548655074314f,   0.0584718995832763f,
    0.0585888729064294f,   0.0587057854754726f,   0.0588226372889998f,
    0.0589394283456172f,   0.0590561586439433f,   0.0591728281826091f,
    0.0592894369602576f,   0.0594059849755440f,   0.0595224722271363f,
    0.0596388987137142f,   0.0597552644339699f,   0.0598715693866082f,
    0.0599878135703456f,   0.0601039969839111f,   0.0602201196260463f,
    0.0603361814955049f,   0.0604521825910525f,   0.0605681229114674f,
    0.0606840024555402f,   0.0607998212220735f,   0.0609155792098825f,
    0.0610312764177944f,   0.0611469128446488f,   0.0612624884892977f,
    0.0613780033506051f,   0.0614934574274476f,   0.0616088507187138f,
    0.0617241832233048f,   0.0618394549401339f,   0.0619546658681265f,
    0.0620698160062207f,   0.0621849053533666f,   0.0622999339085263f,
    0.0624149016706749f,   0.0625298086387993f,   0.0626446548118985f,
    0.0627594401889842f,   0.0628741647690804f,   0.0629888285512230f,
    0.0631034315344604f,   0.0632179737178534f,   0.0633324551004748f,
    0.0634468756814098f,   0.0635612354597560f,   0.0636755344346234f,
    0.0637897726051337f,   0.0639039499704215f,   0.0640180665296333f,
    0.0641321222819281f,   0.0642461172264772f,   0.0643600513624639f,
    0.0644739246890841f,   0.0645877372055457f,   0.0647014889110692f,
    0.0648151798048871f,   0.0649288098862443f,   0.0650423791543980f,
    0.0651558876086177f,   0.0652693352481850f,   0.0653827220723941f,
    0.0654960480805510f,   0.0656093132719747f,   0.0657225176459958f,
    0.0658356612019574f,   0.0659487439392150f,   0.0660617658571363f,
    0.0661747269551014f,   0.0662876272325024f,   0.0664004666887441f,
    0.0665132453232429f,   0.0666259631354284f,   0.0667386201247416f,
    0.0668512162906364f,   0.0669637516325786f,   0.0670762261500467f,
    0.0671886398425310f,   0.0673009927095344f,   0.0674132847505718f,
    0.0675255159651708f,   0.0676376863528710f,   0.0677497959132243f,
    0.0678618446457948f,   0.0679738325501591f,   0.0680857596259061f,
    0.0681976258726366f,   0.0683094312899641f,   0.0684211758775141f,
    0.0685328596349247f,   0.0686444825618459f,   0.0687560446579401f,
    0.0688675459228822f,   0.0689789863563593f,   0.0690903659580705f,
    0.0692016847277276f,   0.0693129426650541f,   0.0694241397697866f,
    0.0695352760416732f,   0.0594425405578624f,   0.0595535551633519f,
    0.0596645089353147f,   0.0597754018735483f,   0.0598862339778625f,
    0.0599970052480793f,   0.0601077156840334f,   0.0602183652855713f,
    0.0603289540525519f,   0.0604394819848466f,   0.0605499490823388f,
    0.0606603553449245f,   0.0607707007725117f,   0.0608809853650206f,
    0.0609912091223840f,   0.0611013720445469f,   0.0612114741314665f,
    0.0613215153831122f,   0.0614314957994659f,   0.0615414153805216f,
    0.0616512741262855f,   0.0617610720367764f,   0.0618708091120252f,
    0.0619804853520752f,   0.0620901007569816f,   0.0621996553268123f,
    0.0623091490616473f,   0.0624185819615789f,   0.0625279540267118f,
    0.0626372652571628f,   0.0627465156530610f,   0.0628557052145480f,
    0.0629648339417774f,   0.0630739018349152f,   0.0631829088941397f,
    0.0632918551196415f,   0.0634007405116234f,   0.0635095650703006f,
    0.0636183287959005f,   0.0637270316886627f,   0.0638356737488393f,
    0.0639442549766944f,   0.0640527753725047f,   0.0641612349365589f,
    0.0642696336691582f,   0.0643779715706160f,   0.0644862486412577f,
    0.0645944648814216f,   0.0647026202914576f,   0.0648107148717285f,
    0.0649187486226089f,   0.0650267215444860f,   0.0651346336377590f,
    0.0652424849028396f,   0.0653502753401518f,   0.0654580049501316f,
    0.0655656737332277f,   0.0656732816899009f,   0.0657808288206239f,
    0.0658883151258823f,   0.0659957406061736f,   0.0661031052620077f,
    0.0662104090939068f,   0.0663176521024054f,   0.0664248342880501f,
    0.0665319556513999f,   0.0666390161930263f,   0.0667460159135127f,
    0.0668529548134549f,   0.0669598328934612f,   0.0670666501541520f,
    0.0671734065961599f,   0.0672801022201299f,   0.0673867370267193f,
    0.0674933110165977f,   0.0675998241904468f,   0.0677062765489609f,
    0.0678126680928461f,   0.0679189988228213f,   0.0680252687396174f,
    0.0681314778439776f,   0.0682376261366575f,   0.0683437136184247f,
    0.0684497402900595f,   0.0685557061523543f,   0.0686616112061134f,
    0.0687674554521540f,   0.0688732388913053f,   0.0689789615244086f,
    0.0690846233523179f,   0.0691902243758991f,   0.0692957645960306f,
    0.0694012440136030f,   0.0695066626295191f,   0.0696120204446942f,
    0.0697173174600557f,   0.0698225536765433f,   0.0699277290951092f,
    0.0700328437167174f,   0.0701378975423447f,   0.0702428905729799f,
    0.0703478228096242f,   0.0704526942532909f,   0.0705575049050059f,
    0.0706622547658070f,   0.0707669438367446f,   0.0708715721188812f,
    0.0607723286906791f,   0.0608768353984505f,   0.0609812813206822f,
    0.0610856664584861f,   0.0611899908129857f,   0.0612942543853178f,
    0.0613984571766304f,   0.0615025991880848f,   0.0616066804208537f,
    0.0617107008761226f,   0.0618146605550893f,   0.0619185594589635f,
    0.0620223975889676f,   0.0621261749463358f,   0.0622298915323152f,
    0.0623335473481647f,   0.0624371423951557f,   0.0625406766745718f,
    0.0626441501877088f,   0.0627475629358750f,   0.0628509149203909f,
    0.0629542061425891f,   0.0630574366038147f,   0.0631606063054251f,
    0.0632637152487898f,   0.0633667634352907f,   0.0634697508663218f,
    0.0635726775432898f,   0.0636755434676132f,   0.0637783486407231f,
    0.0638810930640628f,   0.0639837767390877f,   0.0640863996672659f,
    0.0641889618500774f,   0.0642914632890144f,   0.0643939039855820f,
    0.0644962839412969f,   0.0645986031576883f,   0.0647008616362979f,
    0.0648030593786793f,   0.0649051963863989f,   0.0650072726610348f,
    0.0651092882041778f,   0.0652112430174308f,   0.0653131371024089f,
    0.0654149704607399f,   0.0655167430940633f,   0.0656184550040314f,
    0.0657201061923084f,   0.0658216966605708f,   0.0659232264105077f,
    0.0660246954438204f,   0.0661261037622221f,   0.0662274513674386f,
    0.0663287382612081f,   0.0664299644452808f,   0.0665311299214192f,
    0.0666322346913985f,   0.0667332787570056f,   0.0668342621200399f,
    0.0669351847823132f,   0.0670360467456497f,   0.0671368480118855f,
    0.0672375885828691f,   0.0673382684604614f,   0.0674388876465356f,
    0.0675394461429772f,   0.0676399439516836f,   0.0677403810745650f,
    0.0678407575135437f,   0.0679410732705540f,   0.0680413283475430f,
    0.0681415227464696f,   0.0682416564693053f,   0.0683417295180337f,
    0.0582379309720382f,   0.0583378826785522f,   0.0584377737169835f,
    0.0585376040893650f,   0.0586373737977419f,   0.0587370828441713f,
    0.0588367312307229f,   0.0589363189594787f,   0.0590358460325329f,
    0.0591353124519920f,   0.0592347182199747f,   0.0593340633386119f,
    0.0594333478100472f,   0.0595325716364361f,   0.0596317348199464f,
    0.0597308373627584f,   0.0598298792670645f,   0.0599288605350693f,
    0.0600277811689900f,   0.0601266411710558f,   0.0602254405435084f,
    0.0603241792886015f,   0.0604228574086014f,   0.0605214749057863f,
    0.0606200317824470f,   0.0607185280408865f,   0.0608169636834202f,
    0.0609153387123754f,   0.0610136531300921f,   0.0611119069389223f,
    0.0612101001412306f,   0.0613082327393934f,   0.0614063047357998f,
    0.0615043161328510f,   0.0616022669329606f,   0.0617001571385542f,
    0.0617979867520701f,   0.0618957557759585f,   0.0619934642126822f,
    0.0620911120647159f,   0.0621886993345471f,   0.0622862260246751f,
    0.0623836921376117f,   0.0624810976758809f,   0.0625784426420190f,
    0.0626757270385749f,   0.0627729508681091f,   0.0628701141331951f,
    0.0629672168364182f,   0.0630642589803762f,   0.0631612405676790f,
    0.0632581616009490f,   0.0633550220828208f,   0.0634518220159413f,
    0.0635485614029694f,   0.0636452402465769f,   0.0637418585494472f,
    0.0638384163142765f,   0.0639349135437730f,   0.0640313502406572f,
    0.0641277264076620f,   0.0642240420475324f,   0.0643202971630261f,
    0.0644164917569124f,   0.0645126258319735f,   0.0544048884683912f,
    0.0545009015141968f,   0.0545968540495967f,   0.0546927460774221f,
    0.0547885776005161f,   0.0548843486217346f,   0.0549800591439455f,
    0.0550757091700289f,   0.0551712987028775f,   0.0552668277453958f,
    0.0553622963005011f,   0.0554577043711226f,   0.0555530519602019f,
    0.0556483390706931f,   0.0557435657055621f,   0.0558387318677877f,
    0.0559338375603603f,   0.0560288827862831f,   0.0561238675485714f,
    0.0562187918502528f,   0.0563136556943673f,   0.0564084590839668f,
    0.0565032020221159f,   0.0565978845118913f,   0.0566925065563819f,
    0.0567870681586891f,   0.0568815693219265f,   0.0569760100492198f,
    0.0570703903437071f,   0.0571647102085391f,   0.0572589696468781f,
    0.0573531686618994f,   0.0574473072567899f,   0.0575413854347496f,
    0.0576354031989899f,   0.0577293605527351f,   0.0578232574992215f,
    0.0579170940416977f,   0.0580108701834249f,   0.0581045859276761f,
    0.0581982412777368f,   0.0582918362369048f,   0.0583853708084903f,
    0.0584788449958155f,   0.0585722588022151f,   0.0586656122310360f,
    0.0587589052856374f,   0.0588521379693908f,   0.0589453102856798f,
    0.0590384222379006f,   0.0591314738294614f,   0.0592244650637830f,
    0.0593173959442981f,   0.0594102664744519f,   0.0595030766577018f,
    0.0595958264975176f,   0.0596885159973813f,   0.0597811451607873f,
    0.0598737139912419f,   0.0599662224922641f,   0.0498548597447726f,
    0.0499472475975357f,   0.0500395751314966f,   0.0501318423502235f,
    0.0502240492572963f,   0.0503161958563079f,   0.0504082821508629f,
    0.0505003081445783f,   0.0505922738410838f,   0.0506841792440209f,
    0.0507760243570434f,   0.0508678091838178f,   0.0509595337280224f,
    0.0510511979933482f,   0.0511428019834981f,   0.0512343457021875f,
    0.0513258291531440f,   0.0514172523401077f,   0.0515086152668306f,
    0.0515999179370773f,   0.0516911603546245f,   0.0517823425232611f,
    0.0518734644467888f,   0.0519645261290209f,   0.0520555275737834f,
    0.0521464687849144f,   0.0522373497662645f,   0.0523281705216963f,
    0.0524189310550848f,   0.0525096313703174f,   0.0526002714712936f,
    0.0526908513619254f,   0.0527813710461367f,   0.0528718305278641f,
    0.0529622298110562f,   0.0530525688996742f,   0.0531428477976911f,
    0.0532330665090925f,   0.0533232250378763f,   0.0534133233880527f,
    0.0535033615636440f,   0.0535933395686848f,   0.0536832574072221f,
    0.0537731150833152f,   0.0538629126010356f,   0.0539526499644671f,
    0.0540423271777057f,   0.0541319442448598f,   0.0542215011700501f,
    0.0543109979574096f,   0.0544004346110833f,   0.0544898111352288f,
    0.0545791275340160f,   0.0546683838116266f,   0.0547575799722552f,
    0.0548467160201085f,   0.0549357919594052f,   0.0448209968717641f,
    0.0449099526066536f,   0.0449988482457168f,   0.0450876837932221f,
    0.0451764592534494f,   0.0452651746306915f,   0.0453538299292532f,
    0.0454424251534517f,   0.0455309603076164f,   0.0456194353960890f,
    0.0457078504232236f,   0.0457962053933864f,   0.0458845003109558f,
    0.0459727351803229f,   0.0460609100058907f,   0.0461490247920745f,
    0.0462370795433022f,   0.0463250742640136f,   0.0464130089586610f,
    0.0465008836317089f,   0.0465886982876342f,   0.0466764529309258f,
    0.0467641475660853f,   0.0468517821976262f,   0.0469393568300745f,
    0.0470268714679684f,   0.0471143261158584f,   0.0472017207783073f,
    0.0472890554598900f,   0.0473763301651941f,   0.0474635448988190f,
    0.0475506996653768f,   0.0476377944694915f,   0.0477248293157998f,
    0.0478118042089502f,   0.0478987191536039f,   0.0479855741544340f,
    0.0480723692161264f,   0.0481591043433788f,   0.0482457795409013f,
    0.0483323948134164f,   0.0484189501656588f,   0.0485054456023757f,
    0.0485918811283262f,   0.0486782567482818f,   0.0487645724670264f,
    0.0488508282893562f,   0.0489370242200796f,   0.0490231602640172f,
    0.0491092364260021f,   0.0491952527108794f,   0.0492812091235068f,
    0.0493671056687539f,   0.0494529423515031f,   0.0495387191766484f,
    0.0496244361490969f,   0.0395062823511546f,   0.0395918796329782f,
    0.0396774170768982f,   0.0397628946878707f,   0.0398483124708638f,
    0.0399336704308575f,   0.0400189685728449f,   0.0401042069018306f,
    0.0401893854228319f,   0.0402745041408782f,   0.0403595630610115f,
    0.0404445621882854f,   0.0405295015277667f,   0.0406143810845336f,
    0.0406992008636773f,   0.0407839608703008f,   0.0408686611095195f,
    0.0409533015864613f,   0.0410378823062660f,   0.0411224032740861f,
    0.0412068644950861f,   0.0412912659744427f,   0.0413756077173453f,
    0.0414598897289950f,   0.0415441120146058f,   0.0416282745794035f,
    0.0417123774286264f,   0.0417964205675252f,   0.0418804040013625f,
    0.0419643277354134f,   0.0420481917749655f,   0.0421319961253183f,
    0.0422157407917838f,   0.0422994257796863f,   0.0423830510943621f,
    0.0424666167411601f,   0.0425501227254416f,   0.0426335690525796f,
    0.0427169557279601f,   0.0428002827569807f,   0.0428835501450517f,
    0.0429667578975957f,   0.0430499060200474f,   0.0431329945178538f,
    0.0432160233964742f,   0.0432989926613803f,   0.0433819023180559f,
    0.0434647523719972f,   0.0435475428287129f,   0.0436302736937234f,
    0.0437129449725618f,   0.0437955566707733f,   0.0438781087939159f,
    0.0439606013475590f,   0.0440430343372850f,   0.0441254077686882f,
    0.0442077216473753f,   0.0340861650563528f,   0.0341683598464772f,
    0.0342504951007791f,   0.0343325708249148f,   0.0344145870245521f,
    0.0344965437053716f,   0.0345784408730658f,   0.0346602785333397f,
    0.0347420566919107f,   0.0348237753545080f,   0.0349054345268737f,
    0.0349870342147618f,   0.0350685744239386f,   0.0351500551601828f,
    0.0352314764292854f,   0.0353128382370495f,   0.0353941405892905f,
    0.0354753834918364f,   0.0355565669505271f,   0.0356376909712150f,
    0.0357187555597648f,   0.0357997607220532f,   0.0358807064639694f,
    0.0359615927914151f,   0.0360424197103039f,   0.0361231872265616f,
    0.0362038953461269f,   0.0362845440749502f,   0.0363651334189943f,
    0.0364456633842345f,   0.0365261339766582f,   0.0366065452022649f,
    0.0366868970670670f,   0.0367671895770884f,   0.0368474227383658f,
    0.0369275965569481f,   0.0370077110388962f,   0.0370877661902839f,
    0.0371677620171965f,   0.0372476985257322f,   0.0373275757220012f,
    0.0374073936121259f,   0.0374871522022413f,   0.0375668514984943f,
    0.0376464915070444f,   0.0377260722340632f,   0.0378055936857347f,
    0.0378850558682551f,   0.0379644587878328f,   0.0380438024506888f,
    0.0381230868630559f,   0.0382023120311796f,   0.0382814779613175f,
    0.0383605846597395f,   0.0384396321327278f,   0.0385186203865768f,
    0.0385975494275934f,   0.0386764192620965f,   0.0387552298964174f,
    0.0388339813368998f,   0.0389126735898995f,   0.0287874957391723f,
    0.0288660696363234f,   0.0289445843651332f,   0.0290230399320067f,
    0.0291014363433610f,   0.0291797736056260f,   0.0292580517252432f,
    0.0293362707086670f,   0.0294144305623635f,   0.0294925312928117f,
    0.0295705729065025f,   0.0296485554099390f,   0.0297264788096369f,
    0.0298043431121241f,   0.0298821483239404f,   0.0299598944516384f,
    0.0300375815017827f,   0.0301152094809503f,   0.0301927783957304f,
    0.0302702882527243f,   0.0303477390585462f,   0.0304251308198219f,
    0.0305024635431897f,   0.0305797372353004f,   0.0306569519028170f,
    0.0307341075524143f,   0.0308112041907801f,   0.0308882418246141f,
    0.0309652204606283f,   0.0310421401055471f,   0.0311190007661070f,
    0.0311958024490569f,   0.0312725451611581f,   0.0313492289091839f,
    0.0314258536999200f,   0.0315024195401646f,   0.0315789264367278f,
    0.0316553743964323f,   0.0317317634261129f,   0.0318080935326168f,
    0.0318843647228033f,   0.0319605770035441f,   0.0320367303817234f,
    0.0321128248642373f,   0.0321888604579942f,   0.0322648371699152f,
    0.0323407550069333f,   0.0324166139759939f,   0.0324924140840545f,
    0.0325681553380854f,   0.0326438377450685f,   0.0327194613119984f,
    0.0327950260458821f,   0.0328705319537385f,   0.0329459790425989f,
    0.0330213673195070f,   0.0330966967915189f,   0.0331719674657026f,
    0.0332471793491386f,   0.0333223324489197f,   0.0333974267721510f,
    0.0334724623259497f,   0.0335474391174456f,   0.0336223571537804f,
    0.0336972164421084f,   0.0337720169895960f,   0.0338467588034219f,
    0.0237176309681649f,   0.0237922553362531f,   0.0238668209922893f,
    0.0239413279435017f,   0.0240157761971304f,   0.0240901657604278f,
    0.0241644966406585f,   0.0242387688450997f,   0.0243129823810404f,
    0.0243871372557825f,   0.0244612334766396f,   0.0245352710509378f,
    0.0246092499860156f,   0.0246831702892237f,   0.0247570319679249f,
    0.0248308350294947f,   0.0249045794813202f,   0.0249782653308016f,
    0.0250518925853510f,   0.0251254612523924f,   0.0251989713393627f,
    0.0252724228537110f,   0.0253458158028981f,   0.0254191501943977f,
    0.0254924260356957f,   0.0255656433342901f,   0.0256388020976910f,
    0.0257119023334213f,   0.0257849440490158f,   0.0258579272520216f,
    0.0259308519499983f,   0.0260037181505175f,   0.0260765258611635f,
    0.0261492750895322f,   0.0262219658432325f,   0.0262945981298852f,
    0.0263671719571235f,   0.0264396873325928f,   0.0265121442639506f,
    0.0265845427588672f,   0.0266568828250247f,   0.0267291644701179f,
    0.0268013877018532f,   0.0268735525279502f,   0.0269456589561400f,
    0.0270177069941663f,   0.0270896966497854f,   0.0271616279307651f,
    0.0272335008448862f,   0.0273053153999414f,   0.0273770716037359f,
    0.0274487694640868f,   0.0275204089888242f,   0.0275919901857897f,
    0.0276635130628376f,   0.0277349776278345f,   0.0278063838886591f,
    0.0278777318532024f,   0.0279490215293680f,   0.0280202529250712f,
    0.0280914260482401f,   0.0281625409068147f,   0.0282335975087478f,
    0.0283045958620039f,   0.0283755359745601f,   0.0284464178544057f,
    0.0285172415095424f,   0.0285880069479839f,   0.0286587141777566f,
    0.0287293632068986f,   0.0287999540434609f,   0.0288704866955066f,
    0.0289409611711107f,   0.0290113774783608f,   0.0290817356253569f,
    0.0291520356202110f,   0.0292222774710476f,   0.0292924611860035f,
    0.0293625867732274f,   0.0294326542408808f,   0.0295026635971371f,
    0.0295726148501822f,   0.0194386970856016f,   0.0195085321568307f,
    0.0195783091494799f,   0.0196480280717839f,   0.0197176889319899f,
    0.0197872917383575f,   0.0198568364991585f,   0.0199263232226770f,
    0.0199957519172094f,   0.0200651225910641f,   0.0201344352525623f,
    0.0202036899100369f,   0.0202728865718336f,   0.0203420252463102f,
    0.0204111059418366f,   0.0204801286667952f,   0.0205490934295804f,
    0.0206180002385994f,   0.0206868491022712f,   0.0207556400290274f,
    0.0208243730273115f,   0.0208930481055796f,   0.0209616652723001f,
    0.0210302245359534f,   0.0210987259050326f,   0.0211671693880426f,
    0.0212355549935009f,   0.0213038827299372f,   0.0213721526058936f,
    0.0214403646299242f,   0.0215085188105956f,   0.0215766151564867f,
    0.0216446536761885f,   0.0217126343783045f,   0.0217805572714503f,
    0.0218484223642538f,   0.0219162296653552f,   0.0219839791834073f,
    0.0220516709270747f,   0.0221193049050345f,   0.0221868811259760f,
    0.0222543995986009f,   0.0223218603316232f,   0.0223892633337690f,
    0.0224566086137768f,   0.0225238961803974f,   0.0225911260423939f,
    0.0226582982085414f,   0.0227254126876278f,   0.0227924694884529f,
    0.0228594686198288f,   0.0229264100905801f,   0.0229932939095435f,
    0.0230601200855678f,   0.0231268886275146f,   0.0231935995442573f,
    0.0232602528446819f,   0.0233268485376864f,   0.0233933866321813f,
    0.0234598671370893f,   0.0235262900613454f,   0.0235926554138970f,
    0.0236589632037034f,   0.0237252134397367f,   0.0237914061309807f,
    0.0238575412864321f,   0.0239236189150995f,   0.0239896390260038f,
    0.0240556016281782f,   0.0241215067306683f,   0.0241873543425319f,
    0.0242531444728392f,   0.0243188771306724f,   0.0243845523251262f,
    0.0244501700653075f,   0.0245157303603356f,   0.0245812332193420f,
    0.0246466786514703f,   0.0247120666658767f,   0.0247773972717296f,
    0.0248426704782095f,   0.0249078862945094f,   0.0249730447298344f,
    0.0250381457934021f,   0.0251031894944420f,   0.0251681758421963f,
    0.0252331048459193f,   0.0252979765148776f,   0.0253627908583501f,
    0.0254275478856277f,   0.0254922476060143f,   0.0255568900288252f,
    0.0256214751633887f,   0.0256860030190448f,   0.0257504736051463f,
    0.0258148869310579f,   0.0258792430061567f,   0.0259435418398322f,
    0.0260077834414862f,   0.0260719678205325f,   0.0261360949863972f,
    0.0262001649485192f,   0.0262641777163491f,   0.0263281332993501f,
    0.0263920317069973f,   0.0264558729487787f,   0.0265196570341941f,
    0.0265833839727558f,   0.0266470537739881f,   0.0267106664474281f,
    0.0267742220026244f,   0.0268377204491388f,   0.0269011617965448f,
    0.0269645460544282f,   0.0270278732323873f,   0.0270911433400325f,
    0.0271543563869866f,   0.0272175123828847f,   0.0272806113373739f,
    0.0273436532601140f,   0.0274066381607769f,   0.0274695660490466f,
    0.0275324369346197f,   0.0275952508272048f,   0.0276580077365231f,
    0.0277207076723076f,   0.0277833506443041f,   0.0278459366622703f,
    0.0279084657359765f,   0.0279709378752049f,   0.0280333530897504f,
    0.0280957113894200f,   0.0281580127840327f,   0.0282202572834203f,
    0.0180786339748140f,   0.0181407647132949f,   0.0182028385861189f,
    0.0182648556031667f,   0.0183268157743312f,   0.0183887191095176f,
    0.0184505656186436f,   0.0185123553116388f,   0.0185740881984453f,
    0.0186357642890174f,   0.0186973835933219f,   0.0187589461213378f,
    0.0188204518830560f,   0.0188819008884801f,   0.0189432931476259f,
    0.0190046286705214f,   0.0190659074672070f,   0.0191271295477353f,
    0.0191882949221711f,   0.0192494036005917f,   0.0193104555930864f,
    0.0193714509097570f,   0.0194323895607176f,   0.0194932715560944f,
    0.0195540969060260f,   0.0196148656206632f,   0.0196755777101692f,
    0.0197362331847194f,   0.0197968320545017f,   0.0198573743297157f,
    0.0199178600205741f,   0.0199782891373010f,   0.0200386616901335f,
    0.0200989776893208f,   0.0201592371451241f,   0.0202194400678171f,
    0.0202795864676859f,   0.0203396763550284f,   0.0203997097401555f,
    0.0204596866333899f,   0.0205196070450666f,   0.0205794709855330f,
    0.0206392784651486f,   0.0206990294942856f,   0.0207587240833281f,
    0.0208183622426724f,   0.0208779439827276f,   0.0209374693139144f,
    0.0209969382466666f,   0.0210563507914293f,   0.0211157069586608f,
    0.0211750067588309f,   0.0212342502024225f,   0.0212934372999299f,
    0.0213525680618605f,   0.0214116424987335f,   0.0214706606210802f,
    0.0215296224394448f,   0.0215885279643833f,   0.0216473772064642f,
    0.0217061701762682f,   0.0217649068843883f,   0.0218235873414297f,
    0.0218822115580100f,   0.0219407795447591f,   0.0219992913123190f,
    0.0220577468713443f,   0.0221161462325014f,   0.0221744894064695f,
    0.0222327764039397f,   0.0222910072356156f,   0.0223491819122130f,
    0.0224073004444599f,   0.0224653628430968f,   0.0225233691188765f,
    0.0225813192825635f,   0.0226392133449354f,   0.0226970513167814f,
    0.0227548332089035f,   0.0228125590321158f,   0.0228702287972445f,
    0.0229278425151282f,   0.0229854001966177f,   0.0230429018525766f,
    0.0231003474938801f,   0.0231577371314159f,   0.0232150707760841f,
    0.0232723484387969f,   0.0233295701304792f,   0.0233867358620675f,
    0.0234438456445113f,   0.0235008994887718f,   0.0235578974058227f,
    0.0236148394066502f,   0.0236717255022525f,   0.0237285557036400f,
    0.0237853300218359f,   0.0238420484678749f,   0.0238987110528048f,
    0.0239553177876850f,   0.0240118686835876f,   0.0240683637515969f,
    0.0241248030028093f,   0.0241811864483337f,   0.0242375140992912f,
    0.0242937859668152f,   0.0243500020620512f,   0.0244061623961573f,
    0.0244622669803039f,   0.0245183158256730f,   0.0245743089434599f,
    0.0246302463448713f,   0.0246861280411268f,   0.0247419540434580f,
    0.0247977243631086f,   0.0248534390113350f,   0.0249090979994056f,
    0.0249647013386012f,   0.0250202490402147f,   0.0250757411155516f,
    0.0251311775759295f,   0.0251865584326782f,   0.0252418836971398f,
    0.0252971533806688f,   0.0253523674946320f,   0.0254075260504084f,
    0.0254626290593893f,   0.0255176765329782f,   0.0255726684825909f,
    0.0256276049196556f,   0.0256824858556128f,   0.0257373113019150f,
    0.0257920812700275f,   0.0258467957714273f,   0.0259014548176039f,
    0.0259560584200594f,   0.0260106065903076f,   0.0260650993398752f,
    0.0261195366803005f,   0.0261739186231348f,   0.0262282451799411f,
    0.0262825163622949f,   0.0263367321817843f,   0.0263908926500090f,
    0.0264449977785816f,   0.0264990475791265f,   0.0265530420632809f,
    0.0266069812426939f,   0.0266608651290270f,   0.0267146937339538f,
    0.0267684670691607f,   0.0268221851463456f,   0.0268758479772194f,
    0.0269294555735048f,   0.0269830079469371f,   0.0270365051092639f,
    0.0270899470722445f,   0.0271433338476514f,   0.0271966654472685f,
    0.0272499418828926f,   0.0273031631663327f,   0.0273563293094096f,
    0.0274094403239570f,   0.0274624962218204f,   0.0275154970148579f,
    0.0275684427149397f,   0.0276213333339486f,   0.0276741688837791f,
    0.0277269493763384f,   0.0277796748235459f,   0.0278323452373335f,
    0.0278849606296448f,   0.0279375210124363f,   0.0279900263976765f,
    0.0280424767973460f,   0.0280948722234382f,   0.0281472126879581f,
    0.0281994982029236f,   0.0282517287803646f,   0.0283039044323233f,
    0.0283560251708540f,   0.0284080910080237f,   0.0284601019559115f,
    0.0285120580266086f,   0.0285639592322186f,   0.0286158055848575f,
    0.0286675970966534f,   0.0287193337797467f,   0.0287710156462904f,
    0.0288226427084491f,   0.0288742149784006f,   0.0289257324683340f,
    0.0289771951904517f,   0.0290286031569674f,   0.0290799563801078f,
    0.0291312548721115f,   0.0291824986452294f,   0.0292336877117249f,
    0.0292848220838736f,   0.0293359017739633f,   0.0293869267942941f,
    0.0294378971571783f,   0.0294888128749408f,   0.0295396739599185f,
    0.0295904804244605f,   0.0296412322809285f,   0.0296919295416962f,
    0.0297425722191498f,   0.0297931603256875f,   0.0298436938737202f,
    0.0298941728756706f,   0.0299445973439740f,   0.0299949672910779f,
    0.0300452827294422f,   0.0300955436715389f,   0.0301457501298522f,
    0.0301959021168788f,   0.0302459996451278f,   0.0302960427271200f,
    0.0303460313753892f,   0.0303959656024811f,   0.0304458454209537f,
    0.0304956708433771f,   0.0305454418823342f,   0.0305951585504198f,
    0.0306448208602409f,   0.0306944288244172f,   0.0307439824555803f,
    0.0307934817663740f,   0.0308429267694549f,   0.0308923174774914f,
    0.0309416539031645f,   0.0309909360591671f,   0.0310401639582047f,
    0.0310893376129952f,   0.0311384570362682f,   0.0311875222407662f,
    0.0312365332392437f,   0.0312854900444675f,   0.0313343926692167f,
    0.0313832411262826f,   0.0314320354284691f,   0.0314807755885919f,
    0.0315294616194793f,   0.0315780935339718f,   0.0316266713449223f,
    0.0316751950651957f,   0.0317236647076694f,   0.0317720802852332f,
    0.0318204418107888f,   0.0318687492972504f,   0.0319170027575447f,
    0.0319652022046102f,   0.0320133476513981f,   0.0320614391108717f,
    0.0321094765960065f,   0.0321574601197905f,   0.0322053896952238f,
    0.0322532653353189f,   0.0323010870531006f,   0.0323488548616057f,
    0.0323965687738836f,   0.0324442288029959f,   0.0324918349620166f,
    0.0325393872640315f,   0.0325868857221393f,   0.0326343303494504f,
    0.0326817211590881f,   0.0327290581641876f,   0.0327763413778963f,
    0.0328235708133741f,   0.0328707464837932f,   0.0329178684023379f,
    0.0329649365822048f,   0.0330119510366030f,   0.0330589117787536f,
    0.0331058188218902f,   0.0331526721792586f,   0.0331994718641168f,
    0.0332462178897352f,   0.0332929102693964f,   0.0333395490163956f,
    0.0333861341440396f,   0.0334326656656481f,   0.0334791435945528f,
    0.0335255679440978f,   0.0335719387276394f,   0.0336182559585463f,
    0.0336645196501992f,   0.0337107298159914f,   0.0337568864693283f,
    0.0338029896236278f,   0.0338490392923198f,   0.0338950354888466f,
    0.0339409782266627f,   0.0339868675192350f,   0.0340327033800429f,
    0.0340784858225774f,   0.0341242148603425f,   0.0341698905068542f,
    0.0342155127756406f,   0.0342610816802423f,   0.0343065972342121f,
    0.0343520594511153f,   0.0343974683445292f,   0.0344428239280434f,
    0.0344881262152600f,   0.0345333752197932f,   0.0345785709552694f,
    0.0346237134353274f,   0.0346688026736185f,   0.0347138386838059f,
    0.0347588214795652f,   0.0348037510745845f,   0.0348486274825640f,
    0.0348934507172161f,   0.0349382207922655f,   0.0349829377214494f,
    0.0350276015185171f,   0.0350722121972303f,   0.0351167697713627f,
    0.0351612742547006f,   0.0352057256610425f,   0.0352501240041991f,
    0.0352944692979934f,   0.0353387615562609f,   0.0353830007928488f,
    0.0354271870216173f,   0.0354713202564384f,   0.0355154005111967f,
    0.0355594277997888f,   0.0356034021361236f,   0.0356473235341225f,
    0.0356911920077191f,   0.0357350075708592f,   0.0357787702375008f,
    0.0358224800216146f,   0.0256623260145703f,   0.0257059300755882f,
    0.0257494812960632f,   0.0257929796900145f,   0.0258364252714742f,
    0.0258798180544862f,   0.0259231580531069f,   0.0259664452814049f,
    0.0260096797534614f,   0.0260528614833693f,   0.0260959904852343f,
    0.0261390667731740f,   0.0261820903613187f,   0.0262250612638107f,
    0.0262679794948044f,   0.0263108450684670f,   0.0263536579989775f,
    0.0263964183005274f,   0.0264391259873206f,   0.0264817810735729f,
    0.0265243835735127f,   0.0265669335013807f,   0.0266094308714296f,
    0.0266518756979248f,   0.0266942679951436f,   0.0267366077773757f,
    0.0267788950589231f,   0.0268211298541002f,   0.0268633121772335f,
    0.0269054420426618f,   0.0269475194647364f,   0.0269895444578204f,
    0.0270315170362898f,   0.0270734372145324f,   0.0271153050069485f,
    0.0271571204279507f,   0.0271988834919638f,   0.0272405942134248f,
    0.0272822526067832f,   0.0273238586865006f,   0.0273654124670510f,
    0.0274069139629206f,   0.0274483631886078f,   0.0274897601586236f,
    0.0275311048874910f,   0.0275723973897453f,   0.0276136376799340f,
    0.0276548257726174f,   0.0276959616823674f,   0.0277370454237685f,
    0.0277780770114174f,   0.0278190564599235f,   0.0278599837839078f,
    0.0279008589980039f,   0.0279416821168578f,   0.0279824531551276f,
    0.0280231721274838f,   0.0280638390486091f,   0.0281044539331986f,
    0.0281450167959595f,   0.0281855276516114f,   0.0282259865148862f,
    0.0282663934005280f,   0.0283067483232932f,   0.0283470512979505f,
    0.0283873023392809f,   0.0284275014620776f,   0.0284676486811463f,
    0.0285077440113047f,   0.0285477874673831f,   0.0285877790642237f,
    0.0286277188166811f,   0.0286676067396224f,   0.0287074428479268f,
    0.0287472271564859f,   0.0287869596802033f,   0.0288266404339953f,
    0.0288662694327901f,   0.0289058466915284f,   0.0289453722251632f,
    0.0289848460486596f,   0.0290242681769951f,   0.0290636386251596f,
    0.0291029574081548f,   0.0291422245409955f,   0.0291814400387080f,
    0.0292206039163312f,   0.0292597161889165f,   0.0292987768715272f,
    0.0293377859792390f,   0.0293767435271399f,   0.0294156495303304f,
    0.0294545040039229f,   0.0294933069630424f,   0.0295320584228259f,
    0.0295707583984229f,   0.0296094069049953f,   0.0296480039577166f,
    0.0296865495717735f,   0.0297250437623645f,   0.0297634865447003f,
    0.0298018779340040f,   0.0298402179455111f,   0.0298785065944694f,
    0.0299167438961386f,   0.0299549298657911f,   0.0299930645187115f,
    0.0300311478701964f,   0.0300691799355550f,   0.0301071607301086f,
    0.0301450902691910f,   0.0301829685681479f,   0.0302207956423377f,
    0.0302585715071309f,   0.0302962961779103f,   0.0303339696700707f,
    0.0303715919990197f,   0.0304091631801768f,   0.0304466832289741f,
    0.0304841521608554f,   0.0305215699912776f,   0.0305589367357091f,
    0.0305962524096311f,   0.0306335170285369f,   0.0306707306079320f,
    0.0307078931633345f,   0.0307450047102743f,   0.0307820652642940f,
    0.0308190748409481f,   0.0308560334558039f,   0.0308929411244405f,
    0.0309297978624495f,   0.0309666036854346f,   0.0310033586090122f,
    0.0310400626488105f,   0.0310767158204703f,   0.0311133181396446f,
    0.0311498696219986f,   0.0311863702832097f,   0.0210190092163552f,
    0.0210554082823626f,   0.0210917565743333f,   0.0211280541079942f,
    0.0211643008990843f,   0.0212004969633546f,   0.0212366423165687f,
    0.0212727369745025f,   0.0213087809529439f,   0.0213447742676934f,
    0.0213807169345635f,   0.0214166089693792f,   0.0214524503879776f,
    0.0214882412062082f,   0.0215239814399328f,   0.0215596711050254f,
    0.0215953102173724f,   0.0216308987928723f,   0.0216664368474359f,
    0.0217019243969867f,   0.0217373614574597f,   0.0217727480448030f,
    0.0218080841749764f,   0.0218433698639522f,   0.0218786051277151f,
    0.0219137899822618f,   0.0219489244436015f,   0.0219840085277557f,
    0.0220190422507578f,   0.0220540256286542f,   0.0220889586775028f,
    0.0221238414133744f,   0.0221586738523517f,   0.0221934560105297f,
    0.0222281879040160f,   0.0222628695489301f,   0.0222975009614039f,
    0.0223320821575819f,   0.0223666131536204f,   0.0224010939656882f,
    0.0224355246099665f,   0.0224699051026486f,   0.0225042354599400f,
    0.0225385156980587f,   0.0225727458332349f,   0.0226069258817113f,
    0.0226410558597423f,   0.0226751357835951f,   0.0227091656695491f,
    0.0227431455338958f,   0.0227770753929393f,   0.0228109552629954f,
    0.0228447851603930f,   0.0228785651014724f,   0.0229122951025869f,
    0.0229459751801015f,   0.0229796053503942f,   0.0230131856298546f,
    0.0230467160348848f,   0.0230801965818993f,   0.0231136272873249f,
    0.0231470081676005f,   0.0231803392391772f,   0.0232136205185188f,
    0.0232468520221011f,   0.0232800337664121f,   0.0233131657679523f,
    0.0233462480432343f,   0.0233792806087829f,   0.0234122634811357f,
    0.0234451966768419f,   0.0234780802124634f,   0.0235109141045745f,
    0.0235436983697613f,   0.0235764330246225f,   0.0236091180857689f,
    0.0236417535698240f,   0.0236743394934231f,   0.0237068758732141f,
    0.0237393627258569f,   0.0237718000680239f,   0.0238041879163996f,
    0.0238365262876811f,   0.0238688151985775f,   0.0239010546658102f,
    0.0239332447061131f,   0.0239653853362320f,   0.0239974765729254f,
    0.0240295184329637f,   0.0240615109331300f,   0.0240934540902192f,
    0.0241253479210389f,   0.0241571924424087f,   0.0241889876711606f,
    0.0242207336241391f,   0.0242524303182004f,   0.0242840777702138f,
    0.0243156759970598f,   0.0243472250156324f,   0.0243787248428368f,
    0.0244101754955914f,   0.0244415769908261f,   0.0244729293454835f,
    0.0245042325765185f,   0.0245354867008983f,   0.0245666917356021f,
    0.0245978476976215f,   0.0246289546039604f,   0.0246600124716354f,
    0.0246910213176745f,   0.0247219811591188f,   0.0247528920130213f,
    0.0247837538964472f,   0.0248145668264743f,   0.0248453308201923f,
    0.0248760458947036f,   0.0249067120671227f,   0.0249373293545762f,
    0.0249678977742031f,   0.0249984173431548f,   0.0148250771559823f,
    0.0148554990750867f,   0.0148858721950435f,   0.0149161965330530f,
    0.0149464721063281f,   0.0149766989320937f,   0.0150068770275872f,
    0.0150370064100582f,   0.0150670870967684f,   0.0150971191049919f,
    0.0151271024520153f,   0.0151570371551371f,   0.0151869232316685f,
    0.0152167606989325f,   0.0152465495742647f,   0.0152762898750130f,
    0.0153059816185374f,   0.0153356248222104f,   0.0153652195034166f,
    0.0153947656795529f,   0.0154242633680285f,   0.0154537125862650f,
    0.0154831133516962f,   0.0155124656817680f,   0.0155417695939390f,
    0.0155710251056796f,   0.0156002322344730f,   0.0156293909978140f,
    0.0156585014132103f,   0.0156875634981817f,   0.0157165772702601f,
    0.0157455427469900f,   0.0157744599459278f,   0.0158033288846426f,
    0.0158321495807153f,   0.0158609220517395f,   0.0158896463153209f,
    0.0159183223890776f,   0.0159469502906399f,   0.0159755300376501f,
    0.0160040616477634f,   0.0160325451386467f,   0.0160609805279795f,
    0.0160893678334535f,   0.0161177070727727f,   0.0161459982636532f,
    0.0161742414238237f,   0.0162024365710251f,   0.0162305837230104f,
    0.0162586828975450f,   0.0162867341124064f,   0.0163147373853849f,
    0.0163426927342824f,   0.0163706001769136f,   0.0163984597311053f,
    0.0164262714146964f,   0.0164540352455384f,   0.0164817512414950f,
    0.0165094194204420f,   0.0165370398002675f,   0.0165646123988723f,
    0.0165921372341689f,   0.0166196143240825f,   0.0166470436865503f,
    0.0166744253395220f,   0.0167017593009595f,   0.0167290455888369f,
    0.0167562842211408f,   0.0167834752158696f,   0.0168106185910347f,
    0.0168377143646592f,   0.0168647625547787f,   0.0168917631794411f,
    0.0169187162567065f,   0.0169456218046473f,   0.0169724798413484f,
    0.0169992903849065f,   0.0170260534534310f,   0.0170527690650435f,
    0.0170794372378779f,   0.0171060579900800f,   0.0171326313398085f,
    0.0171591573052340f,   0.0171856359045393f,   0.0172120671559199f,
    0.0172384510775832f,   0.0172647876877488f,   0.0172910770046491f,
    0.0173173190465285f,   0.0173435138316433f,   0.0173696613782627f,
    0.0173957617046678f,   0.0174218148291521f,   0.0174478207700215f,
    0.0174737795455938f,   0.0174996911741995f,   0.0175255556741812f,
    0.0175513730638939f,   0.0175771433617046f,   0.0176028665859928f,
    0.0176285427551502f,   0.0176541718875811f,   0.0176797540017015f,
    0.0177052891159402f,   0.0177307772487379f,   0.0177562184185478f,
    0.0177816126438354f,   0.0178069599430784f,   0.0178322603347666f,
    0.0178575138374025f,   0.0178827204695006f,   0.0179078802495878f,
    0.0179329931962031f,   0.0179580593278980f,   0.0179830786632361f,
    0.0180080512207934f,   0.0180329770191581f,   0.0180578560769309f,
    0.0180826884127244f,   0.0181074740451639f,   0.0181322129928866f,
    0.00795309435192959f,  0.00797773998617998f,  0.00800233899169950f,
    0.00802689138717461f,  0.00805139719130404f,  0.00807585642279884f,
    0.00810026910038247f,  0.00812463524279045f,  0.00814895486877063f,
    0.00817322799708334f,  0.00819745464650101f,  0.00822163483580835f,
    0.00824576858380241f,  0.00826985590929247f,  0.00829389683110024f,
    0.00831789136805938f,  0.00834183953901635f,  0.00836574136282939f,
    0.00838959685836918f,  0.00841340604451885f,  0.00843716894017363f,
    0.00846088556424110f,  0.00848455593564104f,  0.00850818007330578f,
    0.00853175799617945f,  0.00855528972321903f,  0.00857877527339326f,
    0.00860221466568337f,  0.00862560791908315f,  0.00864895505259825f,
    0.00867225608524677f,  0.00869551103605912f,  0.00871871992407797f,
    0.00874188276835819f,  0.00876499958796712f,  0.00878807040198421f,
    0.00881109522950124f,  0.00883407408962228f,  0.00885700700146366f,
    0.00887989398415420f,  0.00890273505683453f,  0.00892553023865803f,
    0.00894827954879010f,  0.00897098300640842f,  0.00899364063070327f,
    0.00901625244087678f,  0.00903881845614354f,  0.00906133869573070f,
    0.00908381317887699f,  0.00910624192483434f,  0.00912862495286607f,
    0.00915096228224840f,  0.00917325393226956f,  0.00919549992223012f,
    0.00921770027144295f,  0.00923985499923324f,  0.00926196412493843f,
    0.00928402766790812f,  0.00930604564750417f,  0.00932801808310113f,
    0.00934994499408540f,  0.00937182639985590f,  0.00939366231982353f,
    0.00941545277341183f,  0.00943719778005636f,  0.00945889735920535f,
    0.00948055153031865f,  0.00950216031286910f,  0.00952372372634122f,
    0.00954524179023242f,  0.00956671452405170f,  0.00958814194732097f,
    0.00960952407957394f,  0.00963086094035709f,  0.00965215254922858f,
    0.00967339892575941f,  0.00969460008953246f,  0.00971575606014319f,
    0.00973686685719929f,  0.00975793250032039f,  0.00977895300913892f,
    0.00979992840329916f,  0.00982085870245789f,  0.00984174392628431f,
    0.00986258409445931f,  0.00988337922667690f,  0.00990412934264276f,
    0.00992483446207504f,  0.00994549460470412f,  0.00996610979027271f,
    0.00998668003853587f,  0.0100072053692609f,   0.0100276858022273f,
    0.0100481213572269f,   0.0100685120540638f,   0.0100888579125544f,
    0.0101091589525275f,   0.0101294151938238f,   0.0101496266562968f,
    0.0101697933598117f,   0.0101899153242468f,   0.0102099925694918f,
    0.0102300251154490f,   0.0102500129820334f,   0.0102699561891717f,
    0.0102898547568033f,   0.0103097087048794f,   0.0103295180533639f,
    0.0103492828222330f,   0.0103690030314749f,   0.0103886787010901f,
    0.0104083098510918f,   0.0104278965015051f,   0.0104474386723672f,
    0.0104669363837280f,   0.0104863896556498f,   0.0105057985082065f,
    0.0105251629614849f,   0.0105444830355838f,   0.0105637587506144f,
    0.0105829901267000f,   0.0106021771839766f,   0.0106213199425919f,
    0.0106404184227065f,   0.0106594726444926f,   0.0106784826281353f,
    0.0106974483938314f,   0.0107163699617908f,   0.0107352473522349f,
    0.0107540805853976f,   0.0107728696815252f,   0.0107916146608764f,
    0.0108103155437217f,   0.0108289723503445f,   0.0108475851010400f,
    0.0108661538161160f,   0.0108846785158922f,   0.0109031592207011f,
    0.0109215959508869f,   0.0109399887268067f,   0.0109583375688294f,
    0.0109766424973363f,   0.0109949035327211f,   0.0110131206953896f,
    0.0110312940057603f,   0.0110494234842632f,   0.000863698228728893f,
    0.000881740104837381f, 0.000899738210443285f, 0.000917692566026501f,
    0.000935603192078749f, 0.000953470109104099f, 0.000971293337619111f,
    0.000989072898152504f, 0.00100680881124518f,  0.00102450109745059f,
    0.00104214977733416f,  0.00105975487147383f,  0.00107731640045972f,
    0.00109483438489416f,  0.00111230884539187f,  0.00112973980257991f,
    0.00114712727709757f,  0.00116447128959615f,  0.00118177186073964f,
    0.00119902901120417f,  0.00121624276167817f,  0.00123341313286207f,
    0.00125054014546896f,  0.00126762382022416f,  0.00128466417786494f,
    0.00130166123914147f,  0.00131861502481540f,  0.00133552555566130f,
    0.00135239285246572f,  0.00136921693602776f,  0.00138599782715843f,
    0.00140273554668116f,  0.00141943011543194f,  0.00143608155425867f,
    0.00145268988402170f,  0.00146925512559357f,  0.00148577729985921f,
    0.00150225642771590f,  0.00151869253007300f,  0.00153508562785212f,
    0.00155143574198749f,  0.00156774289342521f,  0.00158400710312404f,
    0.00160022839205470f,  0.00161640678120040f,  0.00163254229155643f,
    0.00164863494413064f,  0.00166468475994314f,  0.00168069176002578f,
    0.00169665596542351f,  0.00171257739719283f,  0.00172845607640326f,
    0.00174429202413584f,  0.00176008526148430f,  0.00177583580955454f,
    0.00179154368946505f,  0.00180720892234626f,  0.00182283152934076f,
    0.00183841153160380f,  0.00185394895030278f,  0.00186944380661724f,
    0.00188489612173903f,  0.00190030591687257f,  0.00191567321323430f,
    0.00193099803205288f,  0.00194628039456934f,  0.00196152032203714f,
    0.00197671783572181f,  0.00199187295690129f,  0.00200698570686583f,
    0.00202205610691772f,  0.00203708417837167f,  0.00205206994255494f,
    0.00206701342080665f,  0.00208191463447846f,  0.00209677360493415f,
    0.00211159035354996f,  0.00212636490171420f,  0.00214109727082776f,
    0.00215578748230350f,  0.00217043555756677f,  0.00218504151805510f,
    0.00219960538521813f,  0.00221412718051822f,  0.00222860692542973f,
    0.00224304464143935f,  0.00225744035004599f,  0.00227179407276090f,
    0.00228610583110747f,  0.00230037564662183f,  0.00231460354085172f,
    0.00232878953535784f,  0.00234293365171256f,  0.00235703591150094f,
    0.00237109633632010f,  0.00238511494777985f,  0.00239909176750147f,
    0.00241302681711936f,  0.00242692011827975f,  0.00244077169264129f,
    0.00245458156187497f,  0.00246834974766388f,  0.00248207627170341f,
    0.00249576115570141f,  0.00250940442137795f,  0.00252300609046513f,
    0.00253656618470771f,  0.00255008472586260f,  0.00256356173569897f,
    0.00257699723599797f,  0.00259039124855365f,  0.00260374379517186f,
    0.00261705489767092f,  0.00263032457788148f,  0.00264355285764614f,
    0.00265673975882014f,  0.00266988530327097f,  0.00268298951287835f,
    0.00269605240953399f,  0.00270907401514245f,  0.00272205435162004f,
    0.00273499344089564f,  0.00274789130491049f,  0.00276074796561768f,
    0.00277356344498309f,  0.00278633776498463f,  0.00279907094761245f,
    0.00281176301486905f,  0.00282441398876931f,  0.00283702389134027f,
    0.00284959274462127f,  0.00286212057066382f,  0.00287460739153189f,
    0.00288705322930180f,  0.00289945810606193f,  0.00291182204391308f,
    0.00292414506496813f,  0.00293642719135262f,  0.00294866844520389f,
    0.00296086884867222f,  0.00297302842391931f,  0.00298514719311982f,
    0.00299722517846057f,  0.00300926240214039f,  0.00302125888637081f,
    0.00303321465337506f,  0.00304512972538920f,  0.00305700412466148f,
    0.00306883787345200f,  0.00308063099403366f,  0.00309238350869140f,
    0.00310409543972254f,  0.00311576680943643f,  0.00312739764015507f,
    0.00313898795421264f,  0.00315053777395527f,  0.00316204712174176f,
    0.00317351601994309f,  0.00318494449094245f,  0.00319633255713528f,
    0.00320768024092954f,  0.00321898756474520f,  0.00323025455101456f,
    0.00324148122218240f,  0.00325266760070547f,  0.00326381370905315f,
    0.00327491956970674f,  0.00328598520516016f,  0.00329701063791929f,
    0.00330799589050260f,  0.00331894098544075f,  0.00332984594527644f,
    0.00334071079256490f,  0.00335153554987361f,  0.00336232023978234f,
    0.00337306488488309f,  0.00338376950778002f,  0.00339443413108997f,
    0.00340505877744157f,  0.00341564346947612f,  0.00342618822984689f,
    0.00343669308121961f,  0.00344715804627233f,  0.00345758314769543f,
    0.00346796840819122f,  0.00347831385047465f,  0.00348861949727269f,
    0.00349888537132503f,  0.00350911149538313f,  0.00351929789221103f,
    0.00352944458458487f,  0.00353955159529323f,  0.00354961894713696f,
    0.00355964666292907f,  0.00356963476549479f,  0.00357958327767202f,
    0.00358949222231061f,  0.00359936162227278f,  0.00360919150043287f,
    0.00361898187967777f,  0.00362873278290654f,  0.00363844423303059f,
    0.00364811625297337f,  0.00365774886567080f,  0.00366734209407113f,
    0.00367689596113485f,  0.00368641048983448f,  0.00369588570315532f,
    0.00370532162409457f,  0.00371471827566178f,  0.00372407568087890f,
    0.00373339386277999f,  0.00374267284441140f,  0.00375191264883207f,
    0.00376111329911286f,  0.00377027481833711f,  0.00377939722960025f,
    0.00378848055601022f,  0.00379752482068707f,  0.00380653004676323f,
    0.00381549625738342f,  0.00382442347570458f,  0.00383331172489607f,
    0.00384216102813920f,  0.00385097140862786f,  0.00385974288956820f,
    0.00386847549417849f,  0.00387716924568945f,  0.00388582416734407f,
    0.00389444028239744f,  0.00390301761411704f,  0.00391155618578290f,
    0.00392005602068687f,  0.00392851714213324f,  0.00393693957343871f};

// Curve which is scaled by |kCurveCorrectionMultipliers| and added to the curve
// generated by the |kLowReverberationCorrectionCurve| polynomial, for
// reverberation times which result in a feedback factor index less than
// |kCurveChangeoverIndex|.
static const float kLowCorrectionCurve[kCorrectionCurveLength] = {
    0.0355835132858260f,   0.0361471820669677f,   0.0367104001412021f,
    0.0372731677107544f,   0.0378354849778072f,   0.0383973521445001f,
    0.0389587694129315f,   0.0395197369851564f,   0.0400802550631881f,
    0.0406403238489973f,   0.0411999435445122f,   0.0417591143516188f,
    0.0423178364721608f,   0.0428761101079397f,   0.0434339354607135f,
    0.0439913127321995f,   0.0445482421240714f,   0.0451047238379613f,
    0.0456607580754581f,   0.0462163450381097f,   0.0467714849274197f,
    0.0473261779448504f,   0.0478804242918223f,   0.0484342241697123f,
    0.0489875777798562f,   0.0495404853235463f,   0.0500929470020328f,
    0.0506449630165243f,   0.0511965335681860f,   0.0517476588581411f,
    0.0522983390874708f,   0.0528485744572133f,   0.0533983651683647f,
    0.0539477114218787f,   0.0544966134186670f,   0.0550450713595981f,
    0.0555930854454990f,   0.0561406558771536f,   0.0566877828553037f,
    0.0572344665806487f,   0.0577807072538459f,   0.0583265050755101f,
    0.0588718602462131f,   0.0594167729664848f,   0.0599612434368130f,
    0.0605052718576424f,   0.0610488584293762f,   0.0615920033523746f,
    0.0621347068269552f,   0.0626769690533939f,   0.0632187902319240f,
    0.0637601705627360f,   0.0643011102459783f,   0.0648416094817569f,
    0.0653816684701354f,   0.0659212874111350f,   0.0664604665047346f,
    0.0669992059508706f,   0.0675375059494370f,   0.0680753667002852f,
    0.0686127884032250f,   0.0691497712580228f,   0.0696863154644029f,
    0.0702224212220476f,   0.0707580887305970f,   0.0712933181896472f,
    0.0718281097987538f,   0.0723624637574293f,   0.0728963802651431f,
    0.0734298595213236f,   0.0739629017253557f,   0.0744955070765820f,
    0.0750276757743035f,   0.0755594080177776f,   0.0760907040062205f,
    0.0766215639388050f,   0.0771519880146623f,   0.0776819764328806f,
    0.0782115293925059f,   0.0787406470925415f,   0.0792693297319489f,
    0.0797975775096472f,   0.0803253906245122f,   0.0808527692753781f,
    0.0813797136610364f,   0.0819062239802365f,   0.0824323004316849f,
    0.0829579432140462f,   0.0834831525259416f,   0.0840079285659512f,
    0.0845322715326120f,   0.0850561816244186f,   0.0855796590398233f,
    0.0861027039772357f,   0.0866253166350237f,   0.0871474972115118f,
    0.0876692459049832f,   0.0881905629136774f,   0.0887114484357927f,
    0.0892319026694841f,   0.0897519258128650f,   0.0902715180640052f,
    0.0907906796209331f,   0.0913094106816348f,   0.0918277114440527f,
    0.0923455821060883f,   0.0928630228655998f,   0.0933800339204031f,
    0.0938966154682718f,   0.0944127677069371f,   0.0949284908340879f,
    0.0954437850473701f,   0.0959586505443875f,   0.0964730875227018f,
    0.0969870961798317f,   0.0975006767132541f,   0.0980138293204033f,
    0.0985265541986706f,   0.0990388515454054f,   0.0995507215579148f,
    0.100062164433463f,    0.100573180369272f,    0.101083769562521f,
    0.101593932210348f,    0.102103668509848f,    0.102612978658072f,
    0.103121862852030f,    0.103630321288690f,    0.104138354164977f,
    0.104645961677773f,    0.105153144023918f,    0.105659901400211f,
    0.106166234003407f,    0.106672142030217f,    0.107177625677312f,
    0.107682685141321f,    0.108187320618829f,    0.108691532306379f,
    0.109195320400472f,    0.109698685097565f,    0.110201626594074f,
    0.110704145086373f,    0.111206240770793f,    0.111707913843621f,
    0.112209164501105f,    0.112709992939447f,    0.113210399354808f,
    0.113710383943307f,    0.114209946901020f,    0.114709088423982f,
    0.115207808708182f,    0.115706107949571f,    0.116203986344054f,
    0.116701444087495f,    0.117198481375716f,    0.117695098404497f,
    0.118191295369573f,    0.118687072466638f,    0.119182429891344f,
    0.119677367839301f,    0.120171886506076f,    0.120665986087191f,
    0.121159666778131f,    0.121652928774333f,    0.122145772271196f,
    0.122638197464072f,    0.123130204548276f,    0.123621793719075f,
    0.124112965171697f,    0.124603719101328f,    0.125094055703108f,
    0.125583975172139f,    0.126073477703476f,    0.126562563492136f,
    0.127051232733090f,    0.127539485621269f,    0.128027322351559f,
    0.128514743118806f,    0.129001748117814f,    0.129488337543340f,
    0.129974511590103f,    0.130460270452779f,    0.130945614326000f,
    0.131430543404356f,    0.131915057882395f,    0.132399157954622f,
    0.132882843815500f,    0.133366115659450f,    0.133848973680849f,
    0.134331418074033f,    0.134813449033295f,    0.130933338745307f,
    0.131414543419434f,    0.131895335242262f,    0.132375714407915f,
    0.132855681110473f,    0.133335235543976f,    0.133814377902417f,
    0.134293108379751f,    0.134771427169888f,    0.135249334466697f,
    0.135726830464003f,    0.136203915355590f,    0.136680589335197f,
    0.137156852596525f,    0.137632705333228f,    0.138108147738920f,
    0.138583180007172f,    0.139057802331512f,    0.139532014905426f,
    0.140005817922358f,    0.140479211575709f,    0.140952196058836f,
    0.141424771565058f,    0.141896938287645f,    0.142368696419831f,
    0.142840046154803f,    0.143310987685707f,    0.143781521205648f,
    0.144251646907686f,    0.144721364984839f,    0.145190675630085f,
    0.145659579036357f,    0.146128075396545f,    0.146596164903499f,
    0.147063847750025f,    0.147531124128886f,    0.147997994232804f,
    0.148464458254458f,    0.148930516386483f,    0.149396168821473f,
    0.149861415751981f,    0.150326257370514f,    0.150790693869538f,
    0.151254725441479f,    0.151718352278716f,    0.152181574573589f,
    0.152644392518394f,    0.153106806305386f,    0.153568816126774f,
    0.154030422174728f,    0.154491624641374f,    0.154952423718796f,
    0.155412819599036f,    0.155872812474093f,    0.156332402535922f,
    0.156791589976438f,    0.157250374987511f,    0.157708757760971f,
    0.158166738488604f,    0.158624317362153f,    0.159081494573321f,
    0.159538270313766f,    0.159994644775104f,    0.160450618148909f,
    0.160906190626713f,    0.161361362400004f,    0.161816133660228f,
    0.162270504598790f,    0.162724475407050f,    0.163178046276328f,
    0.163631217397900f,    0.164083988962999f,    0.164536361162817f,
    0.164988334188502f,    0.165439908231161f,    0.165891083481858f,
    0.166341860131613f,    0.166792238371406f,    0.167242218392172f,
    0.167691800384806f,    0.168140984540158f,    0.168589771049037f,
    0.169038160102209f,    0.169486151890398f,    0.165572018596708f,
    0.166019216426932f,    0.166466017564089f,    0.166912422198731f,
    0.167358430521371f,    0.167804042722478f,    0.168249258992475f,
    0.168694079521749f,    0.169138504500638f,    0.169582534119443f,
    0.170026168568418f,    0.170469408037777f,    0.170912252717692f,
    0.171354702798289f,    0.171796758469656f,    0.172238419921836f,
    0.172679687344829f,    0.173120560928594f,    0.173561040863046f,
    0.174001127338058f,    0.174440820543462f,    0.174880120669046f,
    0.175319027904554f,    0.175757542439691f,    0.176195664464117f,
    0.176633394167450f,    0.177070731739265f,    0.177507677369096f,
    0.177944231246432f,    0.178380393560723f,    0.178816164501372f,
    0.179251544257744f,    0.179686533019159f,    0.180121130974894f,
    0.180555338314184f,    0.180989155226223f,    0.181422581900160f,
    0.181855618525104f,    0.182288265290119f,    0.182720522384228f,
    0.183152389996411f,    0.183583868315606f,    0.184014957530707f,
    0.184445657830568f,    0.184875969403997f,    0.185305892439763f,
    0.185735427126589f,    0.186164573653160f,    0.186593332208113f,
    0.187021702980047f,    0.187449686157516f,    0.187877281929033f,
    0.188304490483066f,    0.188731312008043f,    0.189157746692349f,
    0.189583794724325f,    0.190009456292270f,    0.190434731584443f,
    0.190859620789056f,    0.191284124094282f,    0.191708241688250f,
    0.192131973759046f,    0.192555320494716f,    0.192978282083259f,
    0.193400858712636f,    0.193823050570763f,    0.194244857845513f,
    0.190304552717141f,    0.190725591388591f,    0.191146246040030f,
    0.191566516859164f,    0.191986404033653f,    0.192405907751115f,
    0.192825028199128f,    0.193243765565223f,    0.193662120036893f,
    0.194080091801586f,    0.194497681046707f,    0.194914887959619f,
    0.195331712727645f,    0.195748155538060f,    0.196164216578102f,
    0.196579896034964f,    0.196995194095795f,    0.197410110947704f,
    0.197824646777756f,    0.198238801772974f,    0.198652576120338f,
    0.199065970006786f,    0.199478983619212f,    0.199891617144471f,
    0.200303870769370f,    0.200715744680679f,    0.201127239065121f,
    0.201538354109379f,    0.201949090000093f,    0.202359446923861f,
    0.202769425067235f,    0.203179024616730f,    0.203588245758813f,
    0.203997088679912f,    0.204405553566412f,    0.204813640604653f,
    0.205221349980936f,    0.205628681881516f,    0.206035636492609f,
    0.206442214000385f,    0.206848414590972f,    0.207254238450459f,
    0.207659685764888f,    0.208064756720260f,    0.208469451502535f,
    0.208873770297627f,    0.209277713291411f,    0.209681280669718f,
    0.210084472618335f,    0.210487289323009f,    0.210889730969442f,
    0.211291797743296f,    0.211693489830188f,    0.212094807415693f,
    0.212495750685345f,    0.212896319824633f,    0.213296515019006f,
    0.213696336453868f,    0.214095784314582f,    0.214494858786468f,
    0.210531832047226f,    0.210930160297245f,    0.211328115714141f,
    0.211725698483062f,    0.212122908789116f,    0.212519746817367f,
    0.212916212752837f,    0.213312306780505f,    0.213708029085308f,
    0.214103379852140f,    0.214498359265853f,    0.214892967511255f,
    0.215287204773112f,    0.215681071236149f,    0.216074567085046f,
    0.216467692504443f,    0.216860447678934f,    0.217252832793074f,
    0.217644848031373f,    0.218036493578299f,    0.218427769618278f,
    0.218818676335693f,    0.219209213914884f,    0.219599382540149f,
    0.219989182395744f,    0.220378613665880f,    0.220767676534728f,
    0.221156371186415f,    0.221544697805026f,    0.221932656574604f,
    0.222320247679147f,    0.222707471302613f,    0.223094327628916f,
    0.223480816841929f,    0.223866939125480f,    0.224252694663356f,
    0.224638083639300f,    0.225023106237015f,    0.225407762640159f,
    0.225792053032349f,    0.226175977597157f,    0.226559536518116f,
    0.226942729978713f,    0.227325558162394f,    0.227708021252562f,
    0.228090119432578f,    0.228471852885760f,    0.228853221795383f,
    0.229234226344680f,    0.229614866716842f,    0.229995143095015f,
    0.230375055662304f,    0.230754604601773f,    0.231133790096440f,
    0.231512612329282f,    0.231891071483235f,    0.227907439733612f,
    0.228285173278418f,    0.228662544292882f,    0.229039552959768f,
    0.229416199461797f,    0.229792483981648f,    0.230168406701958f,
    0.230543967805320f,    0.230919167474285f,    0.231294005891360f,
    0.231668483239013f,    0.232042599699666f,    0.232416355455699f,
    0.232789750689451f,    0.233162785583216f,    0.233535460319247f,
    0.233907775079755f,    0.234279730046907f,    0.234651325402827f,
    0.235022561329597f,    0.235393438009258f,    0.235763955623805f,
    0.236134114355194f,    0.236503914385335f,    0.236873355896098f,
    0.237242439069310f,    0.237611164086754f,    0.237979531130171f,
    0.238347540381260f,    0.238715192021677f,    0.239082486233034f,
    0.239449423196903f,    0.239816003094811f,    0.240182226108245f,
    0.240548092418646f,    0.240913602207415f,    0.241278755655909f,
    0.241643552945443f,    0.242007994257290f,    0.242372079772679f,
    0.242735809672797f,    0.243099184138788f,    0.243462203351754f,
    0.243824867492754f,    0.244187176742804f,    0.244549131282879f,
    0.244910731293909f,    0.245271976956783f,    0.245632868452347f,
    0.245993405961404f,    0.246353589664714f,    0.246713419742997f,
    0.247072896376926f,    0.247432019747135f,    0.243429062026635f,
    0.243787479411131f,    0.244145544073548f,    0.244503256194350f,
    0.244860615953955f,    0.245217623532740f,    0.245574279111040f,
    0.245930582869146f,    0.246286534987307f,    0.246642135645729f,
    0.246997385024576f,    0.247352283303969f,    0.247706830663986f,
    0.248061027284663f,    0.248414873345993f,    0.248768369027926f,
    0.249121514510370f,    0.249474309973191f,    0.249826755596210f,
    0.250178851559207f,    0.250530598041920f,    0.250881995224043f,
    0.251233043285228f,    0.251583742405084f,    0.251934092763177f,
    0.252284094539032f,    0.252633747912130f,    0.252983053061909f,
    0.253332010167765f,    0.253680619409052f,    0.254028880965080f,
    0.254376795015117f,    0.254724361738388f,    0.255071581314077f,
    0.255418453921322f,    0.255764979739221f,    0.256111158946830f,
    0.256456991723160f,    0.256802478247180f,    0.257147618697817f,
    0.257492413253955f,    0.257836862094436f,    0.258180965398058f,
    0.258524723343577f,    0.258868136109707f,    0.259211203875118f,
    0.259553926818439f,    0.259896305118254f,    0.260238338953107f,
    0.260580028501497f,    0.260921373941883f,    0.261262375452678f,
    0.261603033212256f,    0.261943347398944f,    0.262283318191031f,
    0.258261217759182f,    0.258600502296753f,    0.258939443974328f,
    0.259278042970020f,    0.259616299461904f,    0.259954213628010f,
    0.260291785646327f,    0.260629015694800f,    0.260965903951331f,
    0.261302450593780f,    0.261638655799966f,    0.261974519747662f,
    0.262310042614600f,    0.262645224578471f,    0.262980065816920f,
    0.263314566507552f,    0.263648726827928f,    0.263982546955566f,
    0.264316027067943f,    0.264649167342491f,    0.264981967956602f,
    0.265314429087624f,    0.265646550912860f,    0.265978333609575f,
    0.266309777354988f,    0.266640882326276f,    0.266971648700574f,
    0.267302076654974f,    0.267632166366524f,    0.267961918012231f,
    0.268291331769060f,    0.268620407813930f,    0.268949146323722f,
    0.269277547475269f,    0.269605611445366f,    0.269933338410763f,
    0.270260728548167f,    0.270587782034243f,    0.270914499045615f,
    0.271240879758860f,    0.271566924350517f,    0.271892632997080f,
    0.272218005874999f,    0.272543043160685f,    0.272867745030503f,
    0.273192111660776f,    0.273516143227786f,    0.273839839907771f,
    0.274163201876925f,    0.274486229311402f,    0.274808922387312f,
    0.275131281280722f,    0.275453306167657f,    0.275774997224098f,
    0.276096354625986f,    0.276417378549215f,    0.272376341162064f,
    0.272696698655497f,    0.273016723197707f,    0.273336414964418f,
    0.273655774131314f,    0.273974800874035f,    0.274293495368180f,
    0.274611857789301f,    0.274929888312913f,    0.275247587114485f,
    0.275564954369443f,    0.275881990253173f,    0.276198694941014f,
    0.276515068608266f,    0.276831111430185f,    0.277146823581984f,
    0.277462205238834f,    0.277777256575864f,    0.278091977768158f,
    0.278406368990758f,    0.278720430418665f,    0.279034162226836f,
    0.279347564590185f,    0.279660637683583f,    0.279973381681861f,
    0.280285796759803f,    0.280597883092155f,    0.280909640853616f,
    0.281221070218844f,    0.281532171362457f,    0.281842944459025f,
    0.282153389683078f,    0.282463507209106f,    0.282773297211551f,
    0.283082759864817f,    0.283391895343261f,    0.283700703821200f,
    0.284009185472909f,    0.284317340472619f,    0.284625168994516f,
    0.284932671212748f,    0.285239847301418f,    0.285546697434584f,
    0.285853221786265f,    0.286159420530436f,    0.286465293841028f,
    0.286770841891931f,    0.287076064856991f,    0.287380962910013f,
    0.287685536224756f,    0.287989784974941f,    0.288293709334242f,
    0.288597309476293f,    0.288900585574683f,    0.289203537802961f,
    0.289506166334631f,    0.289808471343155f,    0.290110453001953f,
    0.290412111484400f,    0.290713446963832f,    0.286652731605961f,
    0.286953421599191f,    0.287253789109152f,    0.287553834309004f,
    0.287853557371869f,    0.288152958470824f,    0.288452037778905f,
    0.288750795469103f,    0.289049231714367f,    0.289347346687604f,
    0.289645140561678f,    0.289942613509411f,    0.290239765703580f,
    0.290536597316922f,    0.290833108522128f,    0.291129299491851f,
    0.291425170398696f,    0.291720721415230f,    0.292015952713973f,
    0.292310864467405f,    0.292605456847963f,    0.292899730028041f,
    0.293193684179989f,    0.293487319476116f,    0.293780636088688f,
    0.294073634189927f,    0.294366313952014f,    0.294658675547085f,
    0.294950719147237f,    0.295242444924520f,    0.295533853050944f,
    0.295824943698475f,    0.296115717039037f,    0.296406173244510f,
    0.296696312486734f,    0.296986134937503f,    0.297275640768571f,
    0.297564830151647f,    0.297853703258398f,    0.298142260260449f,
    0.298430501329382f,    0.298718426636735f,    0.299006036354006f,
    0.299293330652647f,    0.299580309704069f,    0.299866973679640f,
    0.300153322750686f,    0.300439357088489f,    0.300725076864288f,
    0.301010482249282f,    0.301295573414623f,    0.301580350531424f,
    0.301864813770753f,    0.302148963303636f,    0.302432799301057f,
    0.302716321933956f,    0.302999531373231f,    0.303282427789736f,
    0.303565011354285f,    0.303847282237646f,    0.304129240610547f,
    0.304410886643670f,    0.304692220507658f,    0.304973242373109f,
    0.305253952410579f,    0.305534350790580f,    0.305814437683583f,
    0.301732485252438f,    0.302011949682684f,    0.302291103137086f,
    0.302569945785942f,    0.302848477799510f,    0.303126699348003f,
    0.303404610601592f,    0.303682211730404f,    0.303959502904526f,
    0.304236484294000f,    0.304513156068825f,    0.304789518398959f,
    0.305065571454316f,    0.305341315404768f,    0.305616750420142f,
    0.305891876670227f,    0.306166694324763f,    0.306441203553453f,
    0.306715404525953f,    0.306989297411878f,    0.307262882380802f,
    0.307536159602252f,    0.307809129245716f,    0.308081791480638f,
    0.308354146476418f,    0.308626194402416f,    0.308897935427946f,
    0.309169369722281f,    0.309440497454651f,    0.309711318794243f,
    0.309981833910203f,    0.310252042971631f,    0.310521946147587f,
    0.310791543607086f,    0.311060835519102f,    0.311329822052565f,
    0.311598503376364f,    0.311866879659343f,    0.312134951070305f,
    0.312402717778008f,    0.312670179951171f,    0.312937337758465f,
    0.313204191368524f,    0.313470740949935f,    0.313736986671243f,
    0.314002928700952f,    0.314268567207521f,    0.314533902359368f,
    0.314798934324867f,    0.315063663272350f,    0.315328089370105f,
    0.315592212786379f,    0.315856033689375f,    0.316119552247253f,
    0.316382768628132f,    0.316645683000087f,    0.316908295531148f,
    0.317170606389307f,    0.317432615742509f,    0.317694323758658f,
    0.317955730605615f,    0.318216836451199f,    0.318477641463185f,
    0.318738145809305f,    0.318998349657250f,    0.319258253174666f,
    0.319517856529158f,    0.319777159888288f,    0.320036163419573f,
    0.320294867290491f,    0.320553271668473f,    0.320811376720911f,
    0.321069182615152f,    0.321326689518500f,    0.321583897598218f,
    0.321840807021525f,    0.322097417955597f,    0.322353730567567f,
    0.322609745024527f,    0.322865461493525f,    0.323120880141565f,
    0.323376001135609f,    0.323630824642579f,    0.319523622821772f,
    0.319777851855178f,    0.320031783902010f,    0.320285419129017f,
    0.320538757702904f,    0.320791799790335f,    0.321044545557929f,
    0.321296995172263f,    0.321549148799872f,    0.321801006607247f,
    0.322052568760837f,    0.322303835427048f,    0.322554806772243f,
    0.322805482962742f,    0.323055864164824f,    0.323305950544722f,
    0.323555742268629f,    0.323805239502694f,    0.324054442413022f,
    0.324303351165679f,    0.324551965926684f,    0.324800286862015f,
    0.325048314137607f,    0.325296047919353f,    0.325543488373102f,
    0.325790635664660f,    0.326037489959792f,    0.326284051424219f,
    0.326530320223619f,    0.326776296523626f,    0.327021980489835f,
    0.327267372287794f,    0.327512472083010f,    0.327757280040949f,
    0.328001796327030f,    0.328246021106633f,    0.328489954545092f,
    0.328733596807703f,    0.328976948059713f,    0.329220008466331f,
    0.329462778192721f,    0.329705257404004f,    0.329947446265259f,
    0.330189344941523f,    0.330430953597789f,    0.330672272399006f,
    0.330913301510083f,    0.331154041095883f,    0.331394491321230f,
    0.331634652350901f,    0.331874524349634f,    0.332114107482121f,
    0.332353401913014f,    0.332592407806920f,    0.332831125328403f,
    0.333069554641987f,    0.333307695912150f,    0.333545549303328f,
    0.333783114979916f,    0.334020393106265f,    0.334257383846681f,
    0.334494087365431f,    0.334730503826737f,    0.334966633394778f,
    0.335202476233691f,    0.335438032507570f,    0.335673302380465f,
    0.335908286016386f,    0.336142983579296f,    0.336377395233120f,
    0.336611521141736f,    0.336845361468981f,    0.337078916378650f,
    0.337312186034494f,    0.337545170600220f,    0.337777870239495f,
    0.338010285115941f,    0.338242415393138f,    0.338474261234623f,
    0.338705822803890f,    0.338937100264391f,    0.339168093779534f,
    0.339398803512685f,    0.339629229627166f,    0.339859372286258f,
    0.340089231653198f,    0.340318807891179f,    0.340548101163354f,
    0.340777111632832f,    0.341005839462677f,    0.341234284815913f,
    0.341462447855520f,    0.341690328744436f,    0.341917927645554f,
    0.342145244721727f,    0.342372280135763f,    0.342599034050427f,
    0.342825506628444f,    0.343051698032493f,    0.343277608425212f,
    0.343503237969194f,    0.343728586826993f,    0.343953655161116f,
    0.344178443134029f,    0.344402950908157f,    0.344627178645878f,
    0.344851126509531f,    0.345074794661410f,    0.345298183263766f,
    0.345521292478809f,    0.345744122468705f,    0.345966673395577f,
    0.346188945421505f,    0.346410938708527f,    0.346632653418637f,
    0.346854089713787f,    0.347075247755886f,    0.347296127706800f,
    0.347516729728352f,    0.347737053982323f,    0.347957100630451f,
    0.348176869834429f,    0.348396361755910f,    0.348615576556502f,
    0.348834514397771f,    0.349053175441242f,    0.349271559848394f,
    0.349489667780664f,    0.349707499399448f,    0.349925054866097f,
    0.350142334341920f,    0.350359337988183f,    0.350576065966110f,
    0.350792518436880f,    0.351008695561632f,    0.351224597501459f,
    0.351440224417413f,    0.351655576470505f,    0.351870653821698f,
    0.352085456631918f,    0.352299985062043f,    0.352514239272912f,
    0.352728219425319f,    0.352941925680015f,    0.353155358197710f,
    0.353368517139069f,    0.353581402664716f,    0.353794014935231f,
    0.354006354111151f,    0.354218420352971f,    0.354430213821142f,
    0.354641734676074f,    0.354852983078131f,    0.355063959187638f,
    0.355274663164873f,    0.355485095170075f,    0.355695255363438f,
    0.355905143905114f,    0.356114760955211f,    0.356324106673794f,
    0.356533181220887f,    0.356741984756471f,    0.356950517440481f,
    0.357158779432813f,    0.357366770893317f,    0.357574491981803f,
    0.357781942858036f,    0.357989123681738f,    0.358196034612590f,
    0.358402675810229f,    0.358609047434249f,    0.358815149644201f,
    0.359020982599593f,    0.359226546459892f,    0.359431841384519f,
    0.359636867532855f,    0.359841625064236f,    0.360046114137957f,
    0.360250334913268f,    0.360454287549378f,    0.360657972205452f,
    0.360861389040612f,    0.356702810206361f,    0.356905691876891f,
    0.357108306203617f,    0.357310653345491f,    0.357512733461420f,
    0.357714546710270f,    0.357916093250863f,    0.358117373241979f,
    0.358318386842353f,    0.358519134210680f,    0.358719615505611f,
    0.358919830885752f,    0.359119780509670f,    0.359319464535885f,
    0.359518883122878f,    0.359718036429085f,    0.359916924612898f,
    0.360115547832669f,    0.360313906246705f,    0.360512000013271f,
    0.360709829290588f,    0.360907394236835f,    0.361104695010150f,
    0.361301731768624f,    0.361498504670307f,    0.361695013873208f,
    0.361891259535291f,    0.362087241814476f,    0.362282960868644f,
    0.362478416855630f,    0.362673609933225f,    0.362868540259182f,
    0.363063207991205f,    0.363257613286961f,    0.363451756304069f,
    0.363645637200109f,    0.363839256132616f,    0.364032613259082f,
    0.364225708736957f,    0.364418542723648f,    0.364611115376518f,
    0.364803426852889f,    0.364995477310039f,    0.365187266905203f,
    0.365378795795572f,    0.365570064138297f,    0.365761072090484f,
    0.365951819809197f,    0.366142307451455f,    0.366332535174237f,
    0.366522503134478f,    0.366712211489069f,    0.366901660394860f,
    0.367090850008657f,    0.367279780487222f,    0.367468451987276f,
    0.367656864665497f,    0.367845018678518f,    0.368032914182932f,
    0.368220551335288f,    0.368407930292089f,    0.368595051209801f,
    0.368781914244842f,    0.368968519553590f,    0.369154867292378f,
    0.369340957617498f,    0.369526790685198f,    0.369712366651684f,
    0.369897685673117f,    0.370082747905618f,    0.370267553505262f,
    0.370452102628084f,    0.370636395430075f,    0.370820432067181f,
    0.371004212695309f,    0.371187737470320f,    0.371371006548033f,
    0.371554020084224f,    0.371736778234628f,    0.371919281154933f,
    0.372101529000787f,    0.372283521927796f,    0.372465260091519f,
    0.372646743647477f,    0.372827972751145f,    0.373008947557956f,
    0.373189668223299f,    0.373370134902522f,    0.373550347750928f,
    0.373730306923778f,    0.373910012576292f,    0.374089464863644f,
    0.374268663940966f,    0.374447609963348f,    0.374626303085837f,
    0.374804743463435f,    0.374982931251104f,    0.375160866603761f,
    0.375338549676281f,    0.375515980623497f,    0.375693159600195f,
    0.375870086761124f,    0.376046762260986f,    0.376223186254440f,
    0.376399358896105f,    0.376575280340554f,    0.376750950742319f,
    0.376926370255889f,    0.377101539035708f,    0.377276457236179f,
    0.377451125011662f,    0.377625542516473f,    0.377799709904887f,
    0.377973627331134f,    0.378147294949401f,    0.378320712913835f,
    0.378493881378536f,    0.378666800497564f,    0.378839470424936f,
    0.379011891314623f,    0.379184063320557f,    0.379355986596625f,
    0.379527661296671f,    0.379699087574496f,    0.379870265583860f,
    0.380041195478477f,    0.380211877412020f,    0.380382311538119f,
    0.380552498010361f,    0.380722436982289f,    0.380892128607405f,
    0.381061573039165f,    0.381230770430986f,    0.381399720936239f,
    0.381568424708253f,    0.381736881900314f,    0.381905092665666f,
    0.382073057157508f,    0.382240775528999f,    0.382408247933252f,
    0.382575474523339f,    0.382742455452288f,    0.382909190873084f,
    0.383075680938671f,    0.383241925801948f,    0.383407925615771f,
    0.383573680532955f,    0.383739190706270f,    0.383904456288443f,
    0.384069477432160f,    0.384234254290062f,    0.384398787014749f,
    0.384563075758777f,    0.384727120674658f,    0.384890921914863f,
    0.385054479631818f,    0.385217793977909f,    0.385380865105476f,
    0.385543693166817f,    0.385706278314188f,    0.385868620699801f,
    0.386030720475826f,    0.386192577794389f,    0.386354192807573f,
    0.386515565667420f,    0.386676696525926f,    0.386837585535046f,
    0.386998232846693f,    0.387158638612734f,    0.387318802984995f,
    0.387478726115260f,    0.387638408155268f,    0.383436121249137f,
    0.383595321563679f,    0.383754281242925f,    0.383913000438444f,
    0.384071479301760f,    0.384229717984357f,    0.384387716637672f,
    0.384545475413102f,    0.384702994462001f,    0.384860273935677f,
    0.385017313985400f,    0.385174114762392f,    0.385330676417835f,
    0.385486999102868f,    0.385643082968586f,    0.385798928166041f,
    0.385954534846243f,    0.386109903160157f,    0.386265033258709f,
    0.386419925292778f,    0.386574579413201f,    0.386728995770773f,
    0.386883174516247f,    0.387037115800330f,    0.387190819773689f,
    0.387344286586945f,    0.387497516390679f,    0.387650509335427f,
    0.387803265571684f,    0.387955785249900f,    0.388108068520482f,
    0.388260115533796f,    0.388411926440164f,    0.388563501389864f,
    0.388714840533132f,    0.388865944020162f,    0.389016812001104f,
    0.389167444626063f,    0.389317842045105f,    0.389468004408251f,
    0.389617931865478f,    0.389767624566722f,    0.389917082661874f,
    0.390066306300784f,    0.390215295633258f,    0.390364050809059f,
    0.390512571977908f,    0.390660859289481f,    0.390808912893412f,
    0.390956732939294f,    0.391104319576674f,    0.391251672955058f,
    0.391398793223907f,    0.391545680532642f,    0.391692335030638f,
    0.391838756867229f,    0.391984946191705f,    0.392130903153313f,
    0.392276627901259f,    0.392422120584702f,    0.392567381352763f,
    0.392712410354515f,    0.392857207738992f,    0.393001773655183f,
    0.393146108252035f,    0.393290211678450f,    0.393434084083290f,
    0.393577725615372f,    0.393721136423470f,    0.393864316656316f,
    0.394007266462598f,    0.389788257983385f,    0.389930747382434f,
    0.390073006800726f,    0.390215036386779f,    0.390356836289067f,
    0.390498406656018f,    0.390639747636022f,    0.390780859377423f,
    0.390921742028522f,    0.391062395737578f,    0.391202820652806f,
    0.391343016922379f,    0.391482984694427f,    0.391622724117036f,
    0.391762235338250f,    0.391901518506070f,    0.392040573768452f,
    0.392179401273312f,    0.392318001168520f,    0.392456373601907f,
    0.392594518721257f,    0.392732436674313f,    0.392870127608774f,
    0.393007591672297f,    0.393144829012496f,    0.393281839776942f,
    0.393418624113161f,    0.393555182168638f,    0.393691514090816f,
    0.393827620027092f,    0.393963500124822f,    0.394099154531318f,
    0.394234583393851f,    0.394369786859647f,    0.394504765075889f,
    0.394639518189717f,    0.394774046348230f,    0.394908349698482f,
    0.395042428387484f,    0.395176282562205f,    0.395309912369570f,
    0.395443317956462f,    0.395576499469721f,    0.395709457056142f,
    0.395842190862479f,    0.395974701035442f,    0.396106987721700f,
    0.396239051067876f,    0.396370891220552f,    0.396502508326267f,
    0.396633902531515f,    0.396765073982748f,    0.396896022826377f,
    0.397026749208768f,    0.397157253276243f,    0.392925807167507f,
    0.393055867043950f,    0.393185705044190f,    0.393315321314378f,
    0.393444716000622f,    0.393573889248988f,    0.393702841205497f,
    0.393831572016130f,    0.393960081826821f,    0.394088370783464f,
    0.394216439031910f,    0.394344286717965f,    0.394471913987393f,
    0.394599320985916f,    0.394726507859211f,    0.394853474752914f,
    0.394980221812616f,    0.395106749183867f,    0.395233057012172f,
    0.395359145442995f,    0.395485014621754f,    0.395610664693828f,
    0.395736095804550f,    0.395861308099210f,    0.395986301723057f,
    0.396111076821296f,    0.396235633539087f,    0.396359972021550f,
    0.396484092413761f,    0.396607994860752f,    0.396731679507512f,
    0.396855146498989f,    0.396978395980086f,    0.397101428095663f,
    0.397224242990538f,    0.397346840809486f,    0.397469221697237f,
    0.397591385798481f,    0.397713333257862f,    0.397835064219984f,
    0.397956578829405f,    0.398077877230641f,    0.398198959568167f,
    0.398319825986412f,    0.398440476629763f,    0.398560911642564f,
    0.398681131169117f,    0.398801135353680f,    0.398920924340467f,
    0.394678770266074f,    0.394798129289784f,    0.394917273548105f,
    0.395036203185081f,    0.395154918344712f,    0.395273419170954f,
    0.395391705807720f,    0.395509778398882f,    0.395627637088268f,
    0.395745282019662f,    0.395862713336806f,    0.395979931183398f,
    0.396096935703093f,    0.396213727039505f,    0.396330305336203f,
    0.396446670736713f,    0.396562823384518f,    0.396678763423060f,
    0.396794490995734f,    0.396910006245896f,    0.397025309316857f,
    0.397140400351884f,    0.397255279494203f,    0.397369946886996f,
    0.397484402673401f,    0.397598646996516f,    0.397712679999392f,
    0.397826501825040f,    0.397940112616427f,    0.398053512516475f,
    0.398166701668066f,    0.398279680214038f,    0.398392448297186f,
    0.398505006060259f,    0.398617353645969f,    0.398729491196979f,
    0.398841418855912f,    0.398953136765347f,    0.399064645067821f,
    0.399175943905828f,    0.399287033421816f,    0.399397913758195f,
    0.399508585057326f,    0.399619047461532f,    0.399729301113091f,
    0.395477618146659f,    0.395587454719585f,    0.395697082966438f,
    0.395806503029326f,    0.395915715050310f,    0.396024719171410f,
    0.396133515534604f,    0.396242104281823f,    0.396350485554960f,
    0.396458659495860f,    0.396566626246330f,    0.396674385948129f,
    0.396781938742976f,    0.396889284772547f,    0.396996424178473f,
    0.397103357102343f,    0.397210083685704f,    0.397316604070058f,
    0.397422918396866f,    0.397529026807543f,    0.397634929443464f,
    0.397740626445960f,    0.397846117956317f,    0.397951404115781f,
    0.398056485065553f,    0.398161360946792f,    0.398266031900612f,
    0.398370498068087f,    0.398474759590244f,    0.398578816608071f,
    0.398682669262511f,    0.398786317694462f,    0.398889762044784f,
    0.398993002454288f,    0.399096039063747f,    0.399198872013887f,
    0.399301501445394f,    0.399403927498908f,    0.399506150315030f,
    0.399608170034313f,    0.399709986797270f,    0.399811600744371f,
    0.395551284008465f,    0.395652492745088f,    0.395753499087005f,
    0.395854303174512f,    0.395954905147863f,    0.396055305147269f,
    0.396155503312898f,    0.396255499784873f,    0.396355294703278f,
    0.396454888208150f,    0.396554280439485f,    0.396653471537236f,
    0.396752461641311f,    0.396851250891577f,    0.396949839427857f,
    0.397048227389931f,    0.397146414917536f,    0.397244402150366f,
    0.397342189228072f,    0.397439776290262f,    0.397537163476500f,
    0.397634350926308f,    0.397731338779165f,    0.397828127174505f,
    0.397924716251721f,    0.398021106150163f,    0.398117297009136f,
    0.398213288967904f,    0.398309082165687f,    0.398404676741662f,
    0.398500072834962f,    0.398595270584678f,    0.398690270129858f,
    0.398785071609507f,    0.398879675162585f,    0.398974080928012f,
    0.399068289044663f,    0.399162299651370f,    0.399256112886922f,
    0.399349728890064f,    0.399443147799501f,    0.395174641746314f,
    0.395267666884276f,    0.395360495344382f,    0.395453127265163f,
    0.395545562785107f,    0.395637802042658f,    0.395729845176217f,
    0.395821692324142f,    0.395913343624749f,    0.396004799216310f,
    0.396096059237054f,    0.396187123825166f,    0.396277993118789f,
    0.396368667256023f,    0.396459146374925f,    0.396549430613508f,
    0.396639520109742f,    0.396729415001555f,    0.396819115426830f,
    0.396908621523410f,    0.396997933429091f,    0.397087051281630f,
    0.397175975218736f,    0.397264705378080f,    0.397353241897287f,
    0.397441584913940f,    0.397529734565577f,    0.397617690989695f,
    0.397705454323747f,    0.397793024705144f,    0.397880402271252f,
    0.397967587159395f,    0.398054579506854f,    0.398141379450867f,
    0.398227987128627f,    0.398314402677288f,    0.398400626233956f,
    0.398486657935698f,    0.398572497919536f,    0.398658146322447f,
    0.394381875273792f,    0.394467140925618f,    0.394552215407196f,
    0.394637098855334f,    0.394721791406795f,    0.394806293198300f,
    0.394890604366525f,    0.394974725048106f,    0.395058655379632f,
    0.395142395497653f,    0.395225945538672f,    0.395309305639152f,
    0.395392475935511f,    0.395475456564125f,    0.395558247661327f,
    0.395640849363404f,    0.395723261806605f,    0.395805485127131f,
    0.395887519461143f,    0.395969364944757f,    0.396051021714048f,
    0.396132489905045f,    0.396213769653737f,    0.396294861096067f,
    0.396375764367938f,    0.396456479605206f,    0.396537006943688f,
    0.396617346519155f,    0.396697498467336f,    0.396777462923917f,
    0.396857240024539f,    0.396936829904804f,    0.397016232700266f,
    0.397095448546439f,    0.397174477578793f,    0.397253319932756f,
    0.397331975743711f,    0.397410445146998f,    0.397488728277915f,
    0.397566825271718f,    0.393283008256039f,    0.393360733381202f,
    0.393438272774754f,    0.393515626571778f,    0.393592794907312f,
    0.393669777916352f,    0.393746575733851f,    0.393823188494718f,
    0.393899616333820f,    0.393975859385979f,    0.394051917785976f,
    0.394127791668548f,    0.394203481168389f,    0.394278986420149f,
    0.394354307558436f,    0.394429444717814f,    0.394504398032805f,
    0.394579167637888f,    0.394653753667496f,    0.394728156256022f,
    0.394802375537816f,    0.394876411647182f,    0.394950264718382f,
    0.395023934885638f,    0.395097422283124f,    0.395170727044974f,
    0.395243849305278f,    0.395316789198083f,    0.395389546857393f,
    0.395462122417168f,    0.395534516011326f,    0.395606727773741f,
    0.395678757838244f,    0.395750606338623f,    0.395822273408624f,
    0.395893759181948f,    0.395965063792254f,    0.396036187373157f,
    0.396107130058230f,    0.396177891981003f,    0.396248473274960f,
    0.391957146065968f,    0.392027366502581f,    0.392097406710579f,
    0.392167266823275f,    0.392236946973940f,    0.392306447295801f,
    0.392375767922042f,    0.392444908985804f,    0.392513870620185f,
    0.392582652958240f,    0.392651256132980f,    0.392719680277373f,
    0.392787925524346f,    0.392855992006779f,    0.392923879857513f,
    0.392991589209342f,    0.393059120195020f,    0.393126472947256f,
    0.393193647598717f,    0.393260644282026f,    0.393327463129763f,
    0.393394104274464f,    0.393460567848624f,    0.393526853984694f,
    0.393592962815081f,    0.393658894472149f,    0.393724649088219f,
    0.393790226795571f,    0.393855627726437f,    0.393920852013011f,
    0.393985899787441f,    0.394050771181833f,    0.394115466328248f,
    0.394179985358706f,    0.394244328405183f,    0.394308495599612f,
    0.394372487073883f,    0.394436302959841f,    0.394499943389291f,
    0.394563408493992f,    0.394626698405663f,    0.390328085248398f,
    0.390391025168985f,    0.390453790291433f,    0.390516380747287f,
    0.390578796668047f,    0.390641038185172f,    0.390703105430078f,
    0.390764998534136f,    0.390826717628674f,    0.390888262844978f,
    0.390949634314291f,    0.391010832167811f,    0.391071856536696f,
    0.391132707552056f,    0.391193385344964f,    0.391253890046444f,
    0.391314221787480f,    0.391374380699013f,    0.391434366911939f,
    0.391494180557114f,    0.391553821765346f,    0.391613290667404f,
    0.391672587394012f,    0.391731712075852f,    0.391790664843562f,
    0.391849445827736f,    0.391908055158926f,    0.391966492967641f,
    0.392024759384347f,    0.392082854539465f,    0.392140778563375f,
    0.392198531586412f,    0.392256113738870f,    0.392313525150997f,
    0.392370765953001f,    0.392427836275045f,    0.392484736247248f,
    0.392541465999688f,    0.392598025662398f,    0.392654415365369f,
    0.392710635238548f,    0.392766685411840f,    0.392822566015106f,
    0.392878277178163f,    0.388572091023208f,    0.388627463695129f,
    0.388682667316036f,    0.388737702015574f,    0.388792567923345f,
    0.388847265168909f,    0.388901793881780f,    0.388956154191431f,
    0.389010346227292f,    0.389064370118749f,    0.389118225995145f,
    0.389171913985779f,    0.389225434219908f,    0.389278786826745f,
    0.389331971935462f,    0.389384989675184f,    0.389437840174996f,
    0.389490523563938f,    0.389543039971008f,    0.389595389525160f,
    0.389647572355306f,    0.389699588590314f,    0.389751438359007f,
    0.389803121790169f,    0.389854639012537f,    0.389905990154806f,
    0.389957175345630f,    0.390008194713615f,    0.390059048387330f,
    0.390109736495295f,    0.390160259165990f,    0.390210616527851f,
    0.390260808709272f,    0.390310835838602f,    0.390360698044147f,
    0.390410395454172f,    0.390459928196896f,    0.390509296400497f,
    0.390558500193108f,    0.390607539702819f,    0.390656415057679f,
    0.390705126385692f,    0.390753673814819f,    0.390802057472977f,
    0.390850277488041f,    0.390898333987844f,    0.390946227100173f,
    0.390993956952773f,    0.386679795665769f,    0.386727199381975f,
    0.386774440221429f,    0.386821518311702f,    0.386868433780325f,
    0.386915186754783f,    0.386961777362520f,    0.387008205730934f,
    0.387054471987382f,    0.387100576259178f,    0.387146518673590f,
    0.387192299357847f,    0.387237918439132f,    0.387283376044585f,
    0.387328672301304f,    0.387373807336342f,    0.387418781276710f,
    0.387463594249377f,    0.387508246381265f,    0.387552737799258f,
    0.387597068630192f,    0.387641239000863f,    0.387685249038022f,
    0.387729098868378f,    0.387772788618595f,    0.387816318415297f,
    0.387859688385061f,    0.387902898654424f,    0.387945949349877f,
    0.387988840597871f,    0.388031572524811f,    0.388074145257060f,
    0.388116558920937f,    0.388158813642719f,    0.388200909548640f,
    0.388242846764889f,    0.388284625417613f,    0.388326245632916f,
    0.388367707536857f,    0.388409011255455f,    0.388450156914683f,
    0.388491144640472f,    0.388531974558710f,    0.388572646795240f,
    0.388613161475865f,    0.388653518726341f,    0.388693718672384f,
    0.388733761439666f,    0.388773647153814f,    0.388813375940413f,
    0.388852947925006f,    0.388892363233092f,    0.388931621990125f,
    0.384608996313940f,    0.384647942345062f,    0.384686732201239f,
    0.384725366007753f,    0.384763843889844f,    0.384802165972708f,
    0.384840332381498f,    0.384878343241323f,    0.384916198677251f,
    0.384953898814305f,    0.384991443777464f,    0.385028833691666f,
    0.385066068681804f,    0.385103148872729f,    0.385140074389248f,
    0.385176845356125f,    0.385213461898081f,    0.385249924139793f,
    0.385286232205896f,    0.385322386220981f,    0.385358386309596f,
    0.385394232596246f,    0.385429925205392f,    0.385465464261452f,
    0.385500849888801f,    0.385536082211772f,    0.385571161354653f,
    0.385606087441689f,    0.385640860597082f,    0.385675480944992f,
    0.385709948609534f,    0.385744263714780f,    0.385778426384760f,
    0.385812436743460f,    0.385846294914823f,    0.385880001022748f,
    0.385913555191092f,    0.385946957543667f,    0.385980208204245f,
    0.386013307296550f,    0.386046254944268f,    0.386079051271037f,
    0.386111696400456f,    0.386144190456078f,    0.386176533561413f,
    0.386208725839929f,    0.386240767415049f,    0.386272658410155f,
    0.386304398948585f,    0.386335989153633f,    0.386367429148549f,
    0.386398719056542f,    0.386429859000777f,    0.386460849104374f,
    0.386491689490413f,    0.386522380281929f,    0.386552921601912f,
    0.386583313573312f,    0.386613556319033f,    0.386643649961939f,
    0.386673594624847f,    0.386703390430534f,    0.386733037501732f,
    0.386762535961129f,    0.382430157923795f,    0.382459359527486f,
    0.382488412887186f,    0.382517318125409f,    0.382546075364630f,
    0.382574684727277f,    0.382603146335737f,    0.382631460312354f,
    0.382659626779427f,    0.382687645859213f,    0.382715517673926f,
    0.382743242345736f,    0.382770819996769f,    0.382798250749110f,
    0.382825534724800f,    0.382852672045835f,    0.382879662834169f,
    0.382906507211714f,    0.382933205300338f,    0.382959757221863f,
    0.382986163098072f,    0.383012423050702f,    0.383038537201448f,
    0.383064505671961f,    0.383090328583849f,    0.383116006058678f,
    0.383141538217968f,    0.383166925183199f,    0.383192167075804f,
    0.383217264017177f,    0.383242216128666f,    0.383267023531576f,
    0.383291686347169f,    0.383316204696664f,    0.383340578701237f,
    0.383364808482021f,    0.383388894160103f,    0.383412835856531f,
    0.383436633692307f,    0.383460287788389f,    0.383483798265695f,
    0.383507165245098f,    0.383530388847426f,    0.383553469193466f,
    0.383576406403962f,    0.383599200599613f,    0.383621851901075f,
    0.383644360428963f,    0.383666726303846f,    0.383688949646252f,
    0.383711030576663f,    0.383732969215520f,    0.383754765683220f,
    0.383776420100117f,    0.383797932586521f,    0.383819303262701f,
    0.383840532248879f,    0.383861619665237f,    0.383882565631912f,
    0.383903370268999f,    0.383924033696548f,    0.383944556034568f,
    0.383964937403022f,    0.383985177921832f,    0.384005277710877f,
    0.384025236889990f,    0.384045055578963f,    0.384064733897545f,
    0.384084271965440f,    0.384103669902310f,    0.384122927827773f,
    0.384142045861405f,    0.384161024122738f,    0.384179862731259f,
    0.384198561806415f,    0.384217121467608f,    0.384235541834196f,
    0.384253823025494f,    0.384271965160776f,    0.384289968359270f,
    0.384307832740163f,    0.384325558422595f,    0.384343145525668f,
    0.384360594168436f,    0.384377904469912f,    0.384395076549067f,
    0.384412110524825f,    0.384429006516070f,    0.384445764641642f,
    0.384462385020336f,    0.384478867770907f,    0.384495213012063f,
    0.384511420862471f,    0.384527491440755f,    0.384543424865495f,
    0.384559221255226f,    0.380213152720866f,    0.380228675396019f,
    0.380244061391516f,    0.380259310825718f,    0.380274423816947f,
    0.380289400483481f,    0.380304240943551f,    0.380318945315350f,
    0.380333513717025f,    0.380347946266679f,    0.380362243082373f,
    0.380376404282125f,    0.380390429983908f,    0.380404320305654f,
    0.380418075365251f,    0.380431695280542f,    0.380445180169328f,
    0.380458530149368f,    0.380471745338376f,    0.380484825854023f,
    0.380497771813937f,    0.380510583335702f,    0.380523260536860f,
    0.380535803534910f,    0.380548212447305f,    0.380560487391458f,
    0.380572628484737f,    0.380584635844466f,    0.380596509587927f,
    0.380608249832359f,    0.380619856694957f,    0.380631330292873f,
    0.380642670743215f,    0.380653878163048f,    0.380664952669394f,
    0.380675894379233f,    0.380686703409499f,    0.380697379877085f,
    0.380707923898839f,    0.380718335591567f,    0.380728615072032f,
    0.380738762456952f,    0.380748777863002f,    0.380758661406816f,
    0.380768413204982f,    0.380778033374047f,    0.380787522030512f,
    0.380796879290837f,    0.380806105271438f,    0.380815200088688f,
    0.380824163858915f,    0.380832996698407f,    0.380841698723405f,
    0.380850270050110f,    0.380858710794677f,    0.380867021073220f,
    0.380875201001808f,    0.380883250696466f,    0.380891170273180f,
    0.380898959847887f,    0.380906619536485f,    0.380914149454827f,
    0.380921549718722f,    0.380928820443937f,    0.380935961746195f,
    0.380942973741177f,    0.380949856544519f,    0.380956610271814f,
    0.380963235038613f,    0.380969730960421f,    0.380976098152703f,
    0.380982336730880f,    0.380988446810326f,    0.380994428506377f,
    0.381000281934323f,    0.381006007209410f,    0.381011604446843f,
    0.381017073761781f,    0.381022415269341f,    0.381027629084598f,
    0.381032715322582f,    0.381037674098280f,    0.381042505526636f,
    0.381047209722551f,    0.381051786800881f,    0.381056236876441f,
    0.381060560064002f,    0.381064756478290f,    0.381068826233990f,
    0.381072769445743f,    0.381076586228145f,    0.381080276695752f,
    0.381083840963073f,    0.381087279144577f,    0.381090591354688f,
    0.381093777707787f,    0.381096838318210f,    0.381099773300253f,
    0.381102582768166f,    0.381105266836158f,    0.381107825618392f,
    0.381110259228989f,    0.381112567782028f,    0.381114751391543f,
    0.381116810171524f,    0.381118744235920f,    0.381120553698635f,
    0.381122238673531f,    0.381123799274425f,    0.381125235615091f,
    0.381126547809262f,    0.381127735970624f,    0.381128800212824f,
    0.381129740649461f,    0.381130557394094f,    0.381131250560238f,
    0.381131820261363f,    0.381132266610899f,    0.381132589722230f,
    0.381132789708696f,    0.381132866683598f,    0.381132820760188f,
    0.381132652051679f,    0.381132360671238f,    0.381131946731992f,
    0.381131410347020f,    0.381130751629362f,    0.381129970692011f,
    0.381129067647921f,    0.381128042609999f,    0.381126895691109f,
    0.381125627004075f,    0.381124236661673f,    0.381122724776639f,
    0.381121091461664f,    0.381119336829397f,    0.381117460992443f,
    0.381115464063363f,    0.381113346154677f,    0.381111107378858f,
    0.381108747848338f,    0.381106267675507f,    0.381103666972709f,
    0.381100945852245f,    0.381098104426374f,    0.381095142807312f,
    0.381092061107230f,    0.381088859438256f,    0.381085537912476f,
    0.381082096641931f,    0.381078535738620f,    0.381074855314498f,
    0.381071055481477f,    0.381067136351425f,    0.381063098036167f,
    0.381058940647486f,    0.381054664297119f,    0.381050269096762f,
    0.381045755158066f,    0.381041122592641f,    0.381036371512051f,
    0.381031502027818f,    0.381026514251421f,    0.381021408294295f,
    0.381016184267832f,    0.381010842283379f,    0.381005382452243f,
    0.380999804885685f,    0.380994109694924f,    0.380988296991135f,
    0.380982366885450f,    0.380976319488957f,    0.380970154912701f,
    0.380963873267684f,    0.380957474664866f,    0.380950959215160f,
    0.380944327029439f,    0.380937578218531f,    0.380930712893222f,
    0.380923731164253f,    0.380916633142322f,    0.380909418938086f,
    0.380902088662155f,    0.380894642425098f,    0.380887080337441f,
    0.380879402509665f,    0.380871609052208f,    0.380863700075466f,
    0.380855675689790f,    0.380847536005489f,    0.380839281132828f,
    0.380830911182028f,    0.380822426263269f,    0.380813826486684f,
    0.380805111962367f,    0.380796282800365f,    0.380787339110683f,
    0.380778281003283f,    0.380769108588083f,    0.380759821974958f,
    0.380750421273741f,    0.380740906594218f,    0.380731278046136f,
    0.380721535739195f,    0.380711679783055f,    0.380701710287330f,
    0.380691627361591f,    0.380681431115367f,    0.380671121658143f,
    0.380660699099360f,    0.380650163548417f,    0.376277787107090f,
    0.376267025899846f,    0.376256152028377f,    0.376245165601906f,
    0.376234066729615f,    0.376222855520642f,    0.376211532084083f,
    0.376200096528988f,    0.376188548964365f,    0.376176889499179f,
    0.376165118242352f,    0.376153235302761f,    0.376141240789241f,
    0.376129134810584f,    0.376116917475536f,    0.376104588892804f,
    0.376092149171047f,    0.376079598418884f,    0.376066936744890f,
    0.376054164257595f,    0.376041281065488f,    0.376028287277012f,
    0.376015183000569f,    0.376001968344516f,    0.375988643417169f,
    0.375975208326797f,    0.375961663181629f,    0.375948008089849f,
    0.375934243159598f,    0.375920368498973f,    0.375906384216028f,
    0.375892290418775f,    0.375878087215181f,    0.375863774713170f,
    0.375849353020623f,    0.375834822245377f,    0.375820182495227f,
    0.375805433877922f,    0.375790576501171f,    0.375775610472637f,
    0.375760535899941f,    0.375745352890661f,    0.375730061552329f,
    0.375714661992437f,    0.375699154318431f,    0.375683538637716f,
    0.375667815057652f,    0.375651983685556f,    0.375636044628702f,
    0.375619997994319f,    0.375603843889596f,    0.375587582421675f,
    0.375571213697657f,    0.375554737824598f,    0.375538154909513f,
    0.375521465059370f,    0.375504668381098f,    0.375487764981579f,
    0.375470754967654f,    0.375453638446118f,    0.375436415523726f,
    0.375419086307187f,    0.375401650903168f,    0.375384109418291f,
    0.375366461959138f,    0.375348708632243f,    0.375330849544101f,
    0.375312884801161f,    0.375294814509829f,    0.370914910768890f,
    0.370896629699820f,    0.370878243401316f,    0.370859751979612f,
    0.370841155540898f,    0.370822454191318f,    0.370803648036976f,
    0.370784737183932f,    0.370765721738200f,    0.370746601805755f,
    0.370727377492524f,    0.370708048904394f,    0.370688616147207f,
    0.370669079326762f,    0.370649438548815f,    0.370629693919078f,
    0.370609845543221f,    0.370589893526868f,    0.370569837975601f,
    0.370549678994961f,    0.370529416690441f,    0.370509051167494f,
    0.370488582531529f,    0.370468010887911f,    0.370447336341962f,
    0.370426558998959f,    0.370405678964140f,    0.370384696342694f,
    0.370363611239771f,    0.370342423760476f,    0.370321134009869f,
    0.370299742092970f,    0.370278248114753f,    0.370256652180149f,
    0.370234954394047f,    0.370213154861292f,    0.370191253686683f,
    0.370169250974980f,    0.370147146830898f,    0.370124941359106f,
    0.370102634664233f,    0.370080226850863f,    0.370057718023537f,
    0.370035108286753f,    0.370012397744965f,    0.369989586502583f,
    0.369966674663976f,    0.369943662333467f,    0.369920549615336f,
    0.369897336613821f,    0.369874023433116f,    0.369850610177372f,
    0.365465368943117f,    0.365441755849571f,    0.365418042993176f,
    0.365394230477909f,    0.365370318407704f,    0.365346306886452f,
    0.365322196017997f,    0.365297985906144f,    0.365273676654654f,
    0.365249268367241f,    0.365224761147580f,    0.365200155099300f,
    0.365175450325988f,    0.365150646931187f,    0.365125745018395f,
    0.365100744691070f,    0.365075646052624f,    0.365050449206426f,
    0.365025154255803f,    0.364999761304037f,    0.364974270454366f,
    0.364948681809988f,    0.364922995474054f,    0.364897211549674f,
    0.364871330139912f,    0.364845351347791f,    0.364819275276291f,
    0.364793102028345f,    0.364766831706847f,    0.364740464414645f,
    0.364714000254544f,    0.364687439329305f,    0.364660781741648f,
    0.364634027594247f,    0.364607176989734f,    0.364580230030696f,
    0.364553186819680f,    0.364526047459185f,    0.364498812051671f,
    0.364471480699551f,    0.364444053505197f,    0.364416530570936f,
    0.364388911999053f,    0.364361197891790f,    0.359971660343764f,
    0.359943755472287f,    0.359915755371891f,    0.359887660144644f,
    0.359859469892569f,    0.359831184717647f,    0.359802804721815f,
    0.359774330006967f,    0.359745760674952f,    0.359717096827579f,
    0.359688338566610f,    0.359659485993765f,    0.359630539210721f,
    0.359601498319112f,    0.359572363420526f,    0.359543134616511f,
    0.359513812008569f,    0.359484395698161f,    0.359454885786701f,
    0.359425282375564f,    0.359395585566078f,    0.359365795459529f,
    0.359335912157159f,    0.359305935760169f,    0.359275866369713f,
    0.359245704086904f,    0.359215449012811f,    0.359185101248459f,
    0.359154660894830f,    0.359124128052863f,    0.359093502823453f,
    0.359062785307452f,    0.359031975605668f,    0.359001073818866f,
    0.358970080047767f,    0.358938994393051f,    0.358907816955351f,
    0.358876547835258f,    0.358845187133321f,    0.354452006942467f,
    0.354420463378311f,    0.354388828533694f,    0.354357102508990f,
    0.354325285404530f,    0.354293377320600f,    0.354261378357445f,
    0.354229288615266f,    0.354197108194219f,    0.354164837194419f,
    0.354132475715934f,    0.354100023858793f,    0.354067481722978f,
    0.354034849408430f,    0.354002127015045f,    0.353969314642676f,
    0.353936412391133f,    0.353903420360182f,    0.353870338649546f,
    0.353837167358905f,    0.353803906587893f,    0.353770556436105f,
    0.353737117003089f,    0.353703588388350f,    0.353669970691352f,
    0.353636264011513f,    0.353602468448208f,    0.353568584100769f,
    0.353534611068485f,    0.353500549450601f,    0.353466399346319f,
    0.353432160854798f,    0.353397834075151f,    0.353363419106450f,
    0.353328916047724f,    0.353294324997957f,    0.353259646056089f,
    0.348863151313442f,    0.348828296884025f,    0.348793354859070f,
    0.348758325337345f,    0.348723208417575f,    0.348688004198440f,
    0.348652712778576f,    0.348617334256578f,    0.348581868730996f,
    0.348546316300336f,    0.348510677063062f,    0.348474951117594f,
    0.348439138562308f,    0.348403239495538f,    0.348367254015573f,
    0.348331182220659f,    0.348295024208999f,    0.348258780078753f,
    0.348222449928035f,    0.348186033854919f,    0.348149531957434f,
    0.348112944333565f,    0.348076271081254f,    0.348039512298401f,
    0.348002668082859f,    0.347965738532442f,    0.347928723744916f,
    0.347891623818009f,    0.347854438849400f,    0.347817168936728f,
    0.347779814177587f,    0.347742374669529f,    0.347704850510061f,
    0.347667241796648f,    0.343267820619132f,    0.343230043090047f,
    0.343192181299149f,    0.343154235343728f,    0.343116205321032f,
    0.343078091328263f,    0.343039893462582f,    0.343001611821106f,
    0.342963246500909f,    0.342924797599019f,    0.342886265212423f,
    0.342847649438065f,    0.342808950372843f,    0.342770168113615f,
    0.342731302757191f,    0.342692354400342f,    0.342653323139794f,
    0.342614209072227f,    0.342575012294282f,    0.342535732902554f,
    0.342496370993594f,    0.342456926663912f,    0.342417400009971f,
    0.342377791128193f,    0.342338100114958f,    0.342298327066598f,
    0.342258472079406f,    0.342218535249630f,    0.342178516673473f,
    0.342138416447096f,    0.342098234666617f,    0.342057971428110f,
    0.342017626827605f,    0.337615472953512f,    0.337574965916929f,
    0.337534377806179f,    0.337493708717118f,    0.337452958745561f,
    0.337412127987276f,    0.337371216537989f,    0.337330224493385f,
    0.337289151949102f,    0.337247999000736f,    0.337206765743839f,
    0.337165452273921f,    0.337124058686446f,    0.337082585076838f,
    0.337041031540475f,    0.336999398172692f,    0.336957685068781f,
    0.336915892323990f,    0.336874020033523f,    0.336832068292543f,
    0.336790037196167f,    0.336747926839469f,    0.336705737317481f,
    0.336663468725190f,    0.336621121157539f,    0.336578694709430f,
    0.336536189475720f,    0.336493605551223f,    0.336450943030708f,
    0.336408202008902f,    0.336365382580488f,    0.336322484840108f,
    0.336279508882355f,    0.331874726794207f,    0.331831594685327f,
    0.331788384642604f,    0.331745096760461f,    0.331701731133276f,
    0.331658287855384f,    0.331614767021079f,    0.331571168724609f,
    0.331527493060178f,    0.331483740121949f,    0.331439910004039f,
    0.331396002800524f,    0.331352018605435f,    0.331307957512759f,
    0.331263819616440f,    0.331219605010381f,    0.331175313788438f,
    0.331130946044424f,    0.331086501872111f,    0.331041981365225f,
    0.330997384617451f,    0.330952711722427f,    0.330907962773751f,
    0.330863137864976f,    0.330818237089610f,    0.330773260541122f,
    0.330728208312932f,    0.330683080498421f,    0.330637877190924f,
    0.330592598483733f,    0.330547244470097f,    0.330501815243221f,
    0.326094582888691f,    0.326049003514778f,    0.326003349206981f,
    0.325957620058331f,    0.325911816161815f,    0.325865937610379f,
    0.325819984496923f,    0.325773956914305f,    0.325727854955339f,
    0.325681678712794f,    0.325635428279400f,    0.325589103747838f,
    0.325542705210749f,    0.325496232760730f,    0.325449686490334f,
    0.325403066492070f,    0.325356372858405f,    0.325309605681762f,
    0.325262765054519f,    0.325215851069012f,    0.325168863817535f,
    0.325121803392334f,    0.325074669885617f,    0.325027463389544f,
    0.324980183996233f,    0.324932831797761f,    0.324885406886157f,
    0.324837909353411f,    0.324790339291466f,    0.324742696792224f,
    0.324694981947541f,    0.320285466841654f,    0.320237607581489f,
    0.320189676251195f,    0.320141672942456f,    0.320093597746911f,
    0.320045450756158f,    0.319997232061749f,    0.319948941755193f,
    0.319900579927957f,    0.319852146671464f,    0.319803642077092f,
    0.319755066236177f,    0.319706419240011f,    0.319657701179843f,
    0.319608912146878f,    0.319560052232277f,    0.319511121527159f,
    0.319462120122598f,    0.319413048109625f,    0.319363905579229f,
    0.319314692622353f,    0.319265409329898f,    0.319216055792721f,
    0.319166632101636f,    0.319117138347413f,    0.319067574620779f,
    0.319017941012417f,    0.318968237612967f,    0.318918464513024f,
    0.318868621803143f,    0.318818709573831f,    0.318768727915555f,
    0.314356948911160f,    0.314306828666179f,    0.314256639263369f,
    0.314206380793024f,    0.314156053345391f,    0.314105657010674f,
    0.314055191879036f,    0.314004658040593f,    0.313954055585421f,
    0.313903384603549f,    0.313852645184966f,    0.313801837419615f,
    0.313750961397396f,    0.313700017208166f,    0.313649004941739f,
    0.313597924687883f,    0.313546776536326f,    0.313495560576751f,
    0.313444276898796f,    0.313392925592057f,    0.313341506746087f,
    0.313290020450394f,    0.313238466794443f,    0.313186845867658f,
    0.313135157759415f,    0.313083402559049f,    0.313031580355852f,
    0.312979691239072f,    0.312927735297912f,    0.312875712621533f,
    0.312823623299054f,    0.312771467419547f,    0.308357517064465f,
    0.308305228337950f,    0.308252873321368f,    0.308200452103618f,
    0.308147964773557f,    0.308095411419997f,    0.308042792131707f,
    0.307990106997414f,    0.307937356105799f,    0.307884539545501f,
    0.307831657405116f,    0.307778709773194f,    0.307725696738244f,
    0.307672618388731f,    0.307619474813076f,    0.307566266099656f,
    0.307512992336805f,    0.307459653612815f,    0.307406250015931f,
    0.307352781634359f,    0.307299248556257f,    0.307245650869742f,
    0.307191988662888f,    0.307138262023724f,    0.307084471040236f,
    0.307030615800366f,    0.306976696392013f,    0.306922712903033f,
    0.306868665421238f,    0.306814554034397f,    0.306760378830233f,
    0.306706139896429f,    0.306651837320622f,    0.302235743182830f,
    0.302181313585758f,    0.302126820609336f,    0.302072264341027f,
    0.302017644868253f,    0.301962962278390f,    0.301908216658771f,
    0.301853408096686f,    0.301798536679381f,    0.301743602494059f,
    0.301688605627880f,    0.301633546167958f,    0.301578424201366f,
    0.301523239815132f,    0.301467993096242f,    0.301412684131638f,
    0.301357313008217f,    0.301301879812834f,    0.301246384632300f,
    0.301190827553382f,    0.301135208662805f,    0.301079528047249f,
    0.301023785793351f,    0.300967981987704f,    0.300912116716858f,
    0.300856190067319f,    0.300800202125552f,    0.300744152977974f,
    0.300688042710961f,    0.300631871410846f,    0.300575639163918f,
    0.300519346056422f,    0.300462992174559f,    0.300406577604487f,
    0.295988374424745f,    0.295931838736557f,    0.295875242618375f,
    0.295818586156181f,    0.295761869435917f,    0.295705092543480f,
    0.295648255564723f,    0.295591358585456f,    0.295534401691445f,
    0.295477384968414f,    0.295420308502041f,    0.295363172377963f,
    0.295305976681771f,    0.295248721499016f,    0.295191406915201f,
    0.295134033015788f,    0.295076599886197f,    0.295019107611801f,
    0.294961556277931f,    0.294903945969875f,    0.294846276772877f,
    0.294788548772138f,    0.294730762052815f,    0.294672916700020f,
    0.294615012798824f,    0.294557050434254f,    0.294499029691292f,
    0.294440950654877f,    0.294382813409906f,    0.294324618041230f,
    0.294266364633658f,    0.294208053271955f,    0.294149684040843f,
    0.294091257025001f,    0.294032772309062f,    0.293974229977617f,
    0.289553902107637f,    0.289495244798780f,    0.289436530127930f,
    0.289377758179504f,    0.289318929037873f,    0.289260042787369f,
    0.289201099512278f,    0.289142099296842f,    0.289083042225260f,
    0.289023928381689f,    0.288964757850239f,    0.288905530714981f,
    0.288846247059938f,    0.288786906969092f,    0.288727510526381f,
    0.288668057815699f,    0.288608548920898f,    0.288548983925785f,
    0.288489362914122f,    0.288429685969631f,    0.288369953175988f,
    0.288310164616827f,    0.288250320375736f,    0.288190420536262f,
    0.288130465181907f,    0.288070454396130f,    0.288010388262347f,
    0.287950266863930f,    0.287890090284206f,    0.287829858606460f,
    0.287769571913934f,    0.287709230289826f,    0.287648833817288f,
    0.287588382579433f,    0.287527876659327f,    0.287467316139992f,
    0.287406701104411f,    0.287346031635518f,    0.287285307816206f,
    0.282862801721747f,    0.282801969450102f,    0.282741083076456f,
    0.282680142683527f,    0.282619148353990f,    0.282558100170477f,
    0.282496998215576f,    0.282435842571830f,    0.282374633321742f,
    0.282313370547767f,    0.282252054332321f,    0.282190684757772f,
    0.282129261906449f,    0.282067785860633f,    0.282006256702564f,
    0.281944674514439f,    0.281883039378409f,    0.281821351376584f,
    0.281759610591029f,    0.281697817103765f,    0.281635970996772f,
    0.281574072351982f,    0.281512121251289f,    0.281450117776538f,
    0.281388062009534f,    0.281325954032037f,    0.281263793925765f,
    0.281201581772391f,    0.281139317653543f,    0.281077001650809f,
    0.281014633845731f,    0.280952214319808f,    0.280889743154496f,
    0.280827220431206f,    0.280764646231307f,    0.280702020636124f,
    0.280639343726938f,    0.280576615584986f,    0.280513836291463f,
    0.280451005927520f,    0.280388124574262f,    0.280325192312755f,
    0.275900481216440f,    0.275837447381448f,    0.275774362881135f,
    0.275711227796389f,    0.275648042208058f,    0.275584806196942f,
    0.275521519843800f,    0.275458183229346f,    0.275394796434254f,
    0.275331359539149f,    0.275267872624617f,    0.275204335771198f,
    0.275140749059389f,    0.275077112569644f,    0.275013426382372f,
    0.274949690577941f,    0.274885905236672f,    0.274822070438846f,
    0.274758186264697f,    0.274694252794419f,    0.274630270108159f,
    0.274566238286022f,    0.274502157408071f,    0.274438027554322f,
    0.274373848804750f,    0.274309621239286f,    0.274245344937817f,
    0.274181019980186f,    0.274116646446194f,    0.274052224415596f,
    0.273987753968107f,    0.273923235183394f,    0.273858668141084f,
    0.273794052920759f,    0.273729389601958f,    0.273664678264174f,
    0.273599918986861f,    0.273535111849425f,    0.273470256931231f,
    0.273405354311600f,    0.273340404069808f,    0.273275406285090f,
    0.273210361036635f,    0.273145268403589f,    0.273080128465056f,
    0.273014941300095f,    0.272949706987721f,    0.272884425606906f,
    0.272819097236579f,    0.268391993948047f,    0.268326571835307f,
    0.268261102969579f,    0.268195587429617f,    0.268130025294132f,
    0.268064416641790f,    0.267998761551216f,    0.267933060100989f,
    0.267867312369645f,    0.267801518435677f,    0.267735678377535f,
    0.267669792273623f,    0.267603860202304f,    0.267537882241896f,
    0.267471858470674f,    0.267405788966870f,    0.267339673808669f,
    0.267273513074218f,    0.267207306841617f,    0.267141055188921f,
    0.267074758194145f,    0.267008415935258f,    0.266942028490186f,
    0.266875595936813f,    0.266809118352976f,    0.266742595816472f,
    0.266676028405051f,    0.266609416196424f,    0.266542759268253f,
    0.266476057698160f,    0.266409311563722f,    0.266342520942473f,
    0.266275685911905f,    0.266208806549462f,    0.266141882932549f,
    0.266074915138525f,    0.266007903244705f,    0.265940847328363f,
    0.265873747466726f,    0.265806603736981f,    0.265739416216268f,
    0.265672184981686f,    0.265604910110289f,    0.265537591679087f,
    0.265470229765049f,    0.265402824445097f,    0.265335375796112f,
    0.265267883894930f,    0.265200348818344f,    0.265132770643102f,
    0.265065149445912f,    0.264997485303435f,    0.264929778292289f,
    0.264862028489049f,    0.264794235970247f,    0.264726400812370f,
    0.264658523091862f,    0.264590602885124f,    0.264522640268512f,
    0.264454635318341f,    0.264386588110879f,    0.264318498722353f,
    0.264250367228946f,    0.259820465699218f,    0.259752250224420f,
    0.259683992873027f,    0.259615693721046f,    0.259547352844442f,
    0.259478970319136f,    0.259410546221005f,    0.259342080625883f,
    0.259273573609560f,    0.259205025247783f,    0.259136435616255f,
    0.259067804790635f,    0.258999132846538f,    0.258930419859537f,
    0.258861665905160f,    0.258792871058893f,    0.258724035396177f,
    0.258655158992408f,    0.258586241922943f,    0.258517284263090f,
    0.258448286088117f,    0.258379247473248f,    0.258310168493662f,
    0.258241049224495f,    0.258171889740839f,    0.258102690117744f,
    0.258033450430215f,    0.257964170753213f,    0.257894851161658f,
    0.257825491730422f,    0.257756092534337f,    0.257686653648191f,
    0.257617175146727f,    0.257547657104645f,    0.257478099596601f,
    0.257408502697209f,    0.257338866481037f,    0.257269191022612f,
    0.257199476396415f,    0.257129722676885f,    0.257059929938416f,
    0.256990098255361f,    0.256920227702025f,    0.256850318352674f,
    0.256780370281527f,    0.256710383562763f,    0.256640358270512f,
    0.256570294478866f,    0.256500192261870f,    0.256430051693527f,
    0.256359872847795f,    0.256289655798589f,    0.256219400619781f,
    0.256149107385198f,    0.256078776168626f,    0.256008407043804f,
    0.255938000084429f,    0.255867555364156f,    0.255797072956594f,
    0.255726552935309f,    0.255655995373823f,    0.255585400345616f,
    0.255514767924123f,    0.255444098182735f,    0.255373391194802f,
    0.255302647033627f,    0.255231865772471f,    0.255161047484552f,
    0.255090192243042f,    0.255019300121073f,    0.254948371191731f,
    0.254877405528058f,    0.254806403203054f,    0.254735364289674f,
    0.254664288860830f,    0.254593176989391f,    0.254522028748181f,
    0.254450844209981f,    0.254379623447529f,    0.254308366533519f,
    0.254237073540600f,    0.254165744541380f,    0.254094379608422f,
    0.254022978814245f,    0.253951542231324f,    0.253880069932092f,
    0.253808561988937f,    0.253737018474204f,    0.253665439460195f,
    0.253593825019167f,    0.253522175223334f,    0.253450490144866f,
    0.253378769855891f,    0.253307014428491f,    0.253235223934706f,
    0.253163398446532f,    0.253091538035921f,    0.253019642774782f,
    0.252947712734979f,    0.252875747988335f,    0.252803748606627f,
    0.252731714661588f,    0.252659646224911f,    0.252587543368241f,
    0.252515406163181f,    0.252443234681292f,    0.252371028994090f,
    0.252298789173046f,    0.252226515289590f,    0.252154207415106f,
    0.252081865620937f,    0.252009489978380f,    0.251937080558689f,
    0.251864637433075f,    0.251792160672704f,    0.251719650348701f,
    0.251647106532145f,    0.251574529294072f,    0.251501918705475f,
    0.251429274837301f,    0.251356597760457f,    0.251283887545805f,
    0.251211144264161f,    0.251138367986300f,    0.251065558782954f,
    0.250992716724808f,    0.250919841882507f,    0.250846934326649f,
    0.250773994127792f,    0.250701021356448f,    0.250628016083085f,
    0.250554978378129f,    0.250481908311962f,    0.250408805954920f,
    0.250335671377300f,    0.250262504649351f,    0.250189305841280f,
    0.250116075023251f,    0.250042812265383f,    0.249969517637753f,
    0.249896191210394f,    0.249822833053293f,    0.249749443236397f,
    0.249676021829606f,    0.249602568902779f,    0.249529084525730f,
    0.249455568768230f,    0.249382021700006f,    0.249308443390741f,
    0.249234833910075f,    0.249161193327604f,    0.249087521712880f,
    0.249013819135412f,    0.248940085664666f,    0.248866321370063f,
    0.248792526320980f,    0.248718700586753f,    0.248644844236672f,
    0.248570957339983f,    0.248497039965891f,    0.248423092183554f,
    0.248349114062088f,    0.248275105670567f,    0.248201067078019f,
    0.248126998353429f,    0.248052899565738f,    0.247978770783845f,
    0.247904612076603f,    0.247830423512822f,    0.247756205161271f,
    0.247681957090672f,    0.247607679369705f,    0.247533372067005f,
    0.247459035251165f,    0.247384668990733f,    0.247310273354215f,
    0.247235848410072f,    0.247161394226722f,    0.247086910872538f,
    0.247012398415851f,    0.246937856924948f,    0.242501558460495f,
    0.242426959105845f,    0.242352330921577f,    0.242277673975804f,
    0.242202988336594f,    0.242128274071971f,    0.242053531249917f,
    0.241978759938370f,    0.241903960205222f,    0.241829132118326f,
    0.241754275745486f,    0.241679391154467f,    0.241604478412987f,
    0.241529537588722f,    0.241454568749305f,    0.241379571962322f,
    0.241304547295320f,    0.241229494815798f,    0.241154414591216f,
    0.241079306688985f,    0.241004171176476f,    0.240929008121016f,
    0.240853817589887f,    0.240778599650329f,    0.240703354369537f,
    0.240628081814662f,    0.240552782052813f,    0.240477455151054f,
    0.240402101176407f,    0.240326720195847f,    0.240251312276309f,
    0.240175877484683f,    0.240100415887814f,    0.240024927552506f,
    0.239949412545517f,    0.239873870933562f,    0.239798302783312f,
    0.239722708161397f,    0.239647087134399f,    0.239571439768860f,
    0.239495766131276f,    0.239420066288100f,    0.239344340305743f,
    0.239268588250570f,    0.239192810188903f,    0.239117006187022f,
    0.239041176311160f,    0.238965320627509f,    0.238889439202216f,
    0.238813532101387f,    0.238737599391080f,    0.238661641137313f,
    0.238585657406058f,    0.238509648263246f,    0.238433613774760f,
    0.238357554006444f,    0.238281469024095f,    0.238205358893469f,
    0.238129223680276f,    0.238053063450183f,    0.237976878268814f,
    0.237900668201750f,    0.237824433314526f,    0.237748173672635f,
    0.237671889341525f,    0.237595580386603f,    0.237519246873230f,
    0.237442888866723f,    0.237366506432358f,    0.237290099635364f,
    0.237213668540929f,    0.237137213214196f,    0.237060733720264f,
    0.236984230124190f,    0.236907702490985f,    0.236831150885619f,
    0.236754575373016f,    0.236677976018057f,    0.236601352885581f,
    0.236524706040380f,    0.236448035547206f,    0.236371341470764f,
    0.236294623875718f,    0.236217882826687f,    0.236141118388246f,
    0.236064330624927f,    0.235987519601219f,    0.235910685381566f,
    0.235833828030369f,    0.235756947611985f,    0.235680044190727f,
    0.235603117830866f,    0.235526168596627f,    0.235449196552193f,
    0.235372201761703f,    0.235295184289252f,    0.235218144198891f,
    0.235141081554629f,    0.235063996420428f,    0.234986888860210f,
    0.234909758937852f,    0.234832606717186f,    0.234755432262002f,
    0.234678235636045f,    0.234601016903018f,    0.234523776126579f,
    0.234446513370342f,    0.234369228697878f,    0.234291922172716f,
    0.234214593858338f,    0.234137243818184f,    0.234059872115652f,
    0.233982478814092f,    0.233905063976814f,    0.233827627667084f,
    0.233750169948123f,    0.233672690883109f,    0.233595190535175f,
    0.233517668967413f,    0.233440126242869f,    0.233362562424547f,
    0.233284977575406f,    0.233207371758361f,    0.233129745036284f,
    0.233052097472005f,    0.232974429128308f,    0.232896740067933f,
    0.232819030353578f,    0.232741300047898f,    0.232663549213500f,
    0.232585777912953f,    0.232507986208779f,    0.232430174163456f,
    0.232352341839420f,    0.232274489299062f,    0.232196616604730f,
    0.227756995811151f,    0.227679082995740f,    0.227601150213137f,
    0.227523197525514f,    0.227445224995001f,    0.227367232683684f,
    0.227289220653605f,    0.227211188966762f,    0.227133137685110f,
    0.227055066870559f,    0.226976976584978f,    0.226898866890189f,
    0.226820737847972f,    0.226742589520064f,    0.226664421968157f,
    0.226586235253900f,    0.226508029438898f,    0.226429804584713f,
    0.226351560752862f,    0.226273298004819f,    0.226195016402015f,
    0.226116716005836f,    0.226038396877625f,    0.225960059078682f,
    0.225881702670262f,    0.225803327713576f,    0.225724934269794f,
    0.225646522400040f,    0.225568092165393f,    0.225489643626893f,
    0.225411176845531f,    0.225332691882258f,    0.225254188797980f,
    0.225175667653558f,    0.225097128509812f,    0.225018571427517f,
    0.224939996467403f,    0.224861403690158f,    0.224782793156426f,
    0.224704164926808f,    0.224625519061859f,    0.224546855622093f,
    0.224468174667977f,    0.224389476259939f,    0.224310760458359f,
    0.224232027323575f,    0.224153276915881f,    0.224074509295528f,
    0.223995724522724f,    0.223916922657630f,    0.223838103760367f,
    0.223759267891009f,    0.223680415109590f,    0.223601545476097f,
    0.223522659050475f,    0.223443755892626f,    0.223364836062405f,
    0.223285899619627f,    0.223206946624062f,    0.223127977135435f,
    0.223048991213429f,    0.222969988917683f,    0.222890970307791f,
    0.222811935443306f,    0.222732884383734f,    0.222653817188539f,
    0.222574733917142f,    0.222495634628919f,    0.222416519383203f,
    0.222337388239283f,    0.222258241256404f,    0.222179078493767f,
    0.217738172002954f,    0.217658977858233f,    0.217579768111098f,
    0.217500542820575f,    0.217421302045647f,    0.217342045845254f,
    0.217262774278292f,    0.217183487403611f,    0.217104185280022f,
    0.217024867966288f,    0.216945535521129f,    0.216866188003224f,
    0.216786825471206f,    0.216707447983663f,    0.216628055599143f,
    0.216548648376148f,    0.216469226373135f,    0.216389789648521f,
    0.216310338260676f,    0.216230872267927f,    0.216151391728558f,
    0.216071896700810f,    0.215992387242879f,    0.215912863412916f,
    0.215833325269032f,    0.215753772869290f,    0.215674206271714f,
    0.215594625534279f,    0.215515030714921f,    0.215435421871530f,
    0.215355799061952f,    0.215276162343990f,    0.215196511775403f,
    0.215116847413907f,    0.215037169317173f,    0.214957477542830f,
    0.214877772148461f,    0.214798053191607f,    0.214718320729766f,
    0.214638574820389f,    0.214558815520887f,    0.214479042888626f,
    0.214399256980927f,    0.214319457855068f,    0.214239645568284f,
    0.214159820177766f,    0.214079981740661f,    0.214000130314072f,
    0.213920265955059f,    0.213840388720638f,    0.213760498667781f,
    0.213680595853417f,    0.213600680334430f,    0.213520752167662f,
    0.213440811409909f,    0.213360858117925f,    0.213280892348421f,
    0.213200914158062f,    0.213120923603470f,    0.208679192733648f,
    0.208599177620285f,    0.208519150312294f,    0.208439110866124f,
    0.208359059338178f,    0.208278995784815f,    0.208198920262354f,
    0.208118832827066f,    0.208038733535179f,    0.207958622442880f,
    0.207878499606310f,    0.207798365081566f,    0.207718218924703f,
    0.207638061191730f,    0.207557891938615f,    0.207477711221280f,
    0.207397519095604f,    0.207317315617423f,    0.207237100842528f,
    0.207156874826667f,    0.207076637625544f,    0.206996389294820f,
    0.206916129890112f,    0.206835859466992f,    0.206755578080990f,
    0.206675285787592f,    0.206594982642238f,    0.206514668700328f,
    0.206434344017215f,    0.206354008648211f,    0.206273662648582f,
    0.206193306073550f,    0.206112938978297f,    0.206032561417956f,
    0.205952173447621f,    0.205871775122340f,    0.205791366497116f,
    0.205710947626911f,    0.205630518566642f,    0.205550079371182f,
    0.205469630095361f,    0.205389170793964f,    0.205308701521733f,
    0.205228222333368f,    0.205147733283521f,    0.205067234426806f,
    0.204986725817787f,    0.204906207510990f,    0.204825679560893f,
    0.204745142021933f,    0.204664594948501f,    0.200222310387369f,
    0.200141744407996f,    0.200061169057067f,    0.199980584388797f,
    0.199899990457361f,    0.199819387316889f,    0.199738775021466f,
    0.199658153625134f,    0.199577523181893f,    0.199496883745697f,
    0.199416235370458f,    0.199335578110042f,    0.199254912018273f,
    0.199174237148931f,    0.199093553555752f,    0.199012861292428f,
    0.198932160412609f,    0.198851450969898f,    0.198770733017858f,
    0.198690006610006f,    0.198609271799814f,    0.198528528640714f,
    0.198447777186091f,    0.198367017489288f,    0.198286249603603f,
    0.198205473582292f,    0.198124689478565f,    0.198043897345590f,
    0.197963097236492f,    0.197882289204349f,    0.197801473302198f,
    0.197720649583031f,    0.197639818099798f,    0.197558978905402f,
    0.197478132052707f,    0.197397277594528f,    0.197316415583640f,
    0.197235546072773f,    0.197154669114612f,    0.197073784761801f,
    0.196992893066939f,    0.196911994082579f,    0.196831087861235f,
    0.196750174455372f,    0.196669253917415f,    0.196588326299744f,
    0.196507391654695f,    0.196426450034560f,    0.191983773484012f,
    0.191902818070409f,    0.191821855838336f,    0.191740886839911f,
    0.191659911127206f,    0.191578928752253f,    0.191497939767036f,
    0.191416944223500f,    0.191335942173542f,    0.191254933669018f,
    0.191173918761738f,    0.191092897503471f,    0.191011869945940f,
    0.190930836140826f,    0.190849796139763f,    0.190768749994346f,
    0.190687697756122f,    0.190606639476596f,    0.190525575207231f,
    0.190444504999443f,    0.190363428904605f,    0.190282346974049f,
    0.190201259259060f,    0.190120165810881f,    0.190039066680710f,
    0.189957961919703f,    0.189876851578970f,    0.189795735709579f,
    0.189714614362553f,    0.189633487588873f,    0.189552355439475f,
    0.189471217965251f,    0.189390075217050f,    0.189308927245676f,
    0.189227774101891f,    0.189146615836412f,    0.189065452499913f,
    0.188984284143024f,    0.188903110816330f,    0.188821932570374f,
    0.188740749455655f,    0.188659561522627f,    0.188578368821702f,
    0.188497171403246f,    0.188415969317584f,    0.188334762614995f,
    0.183891823338137f,    0.183810607552358f,    0.183729387300228f,
    0.183648162631853f,    0.183566933597294f,    0.183485700246567f,
    0.183404462629646f,    0.183323220796461f,    0.183241974796898f,
    0.183160724680798f,    0.183079470497960f,    0.182998212298140f,
    0.182916950131046f,    0.182835684046347f,    0.182754414093667f,
    0.182673140322583f,    0.182591862782633f,    0.182510581523309f,
    0.182429296594058f,    0.182348008044284f,    0.182266715923350f,
    0.182185420280571f,    0.182104121165221f,    0.182022818626530f,
    0.181941512713682f,    0.181860203475820f,    0.181778890962042f,
    0.181697575221403f,    0.181616256302912f,    0.181534934255536f,
    0.181453609128199f,    0.181372280969781f,    0.181290949829115f,
    0.181209615754995f,    0.181128278796168f,    0.181046939001338f,
    0.180965596419165f,    0.180884251098267f,    0.180802903087216f,
    0.180721552434541f,    0.180640199188728f,    0.180558843398218f,
    0.180477485111409f,    0.180396124376655f,    0.175953033234688f,
    0.175871667748931f,    0.175790299960027f,    0.175708929916157f,
    0.175627557665455f,    0.175546183256013f,    0.175464806735878f,
    0.175383428153054f,    0.175302047555502f,    0.175220664991136f,
    0.175139280507831f,    0.175057894153414f,    0.174976505975670f,
    0.174895116022341f,    0.174813724341124f,    0.174732330979673f,
    0.174650935985597f,    0.174569539406463f,    0.174488141289793f,
    0.174406741683064f,    0.174325340633713f,    0.174243938189129f,
    0.174162534396661f,    0.174081129303611f,    0.173999722957239f,
    0.173918315404761f,    0.173836906693349f,    0.173755496870131f,
    0.173674085982191f,    0.173592674076571f,    0.173511261200267f,
    0.173429847400233f,    0.173348432723377f,    0.173267017216566f,
    0.173185600926620f,    0.173104183900319f,    0.173022766184396f,
    0.172941347825542f,    0.172859928870403f,    0.172778509365582f,
    0.172697089357639f,    0.172615668893089f,    0.172534248018403f,
    0.172452826780009f,    0.168009677216713f,    0.167928255390010f,
    0.167846833338621f,    0.167765411108796f,    0.167683988746745f,
    0.167602566298634f,    0.167521143810582f,    0.167439721328668f,
    0.167358298898926f,    0.167276876567344f,    0.167195454379870f,
    0.167114032382406f,    0.167032610620809f,    0.166951189140896f,
    0.166869767988436f,    0.166788347209158f,    0.166706926848743f,
    0.166625506952833f,    0.166544087567022f,    0.166462668736864f,
    0.166381250507865f,    0.166299832925490f,    0.166218416035160f,
    0.166136999882253f,    0.166055584512100f,    0.165974169969991f,
    0.165892756301173f,    0.165811343550845f,    0.165729931764167f,
    0.165648520986252f,    0.165567111262170f,    0.165485702636949f,
    0.165404295155570f,    0.165322888862974f,    0.165241483804054f,
    0.165160080023663f,    0.165078677566607f,    0.164997276477651f,
    0.164915876801515f,    0.164834478582874f,    0.164753081866361f,
    0.164671686696565f,    0.164590293118030f,    0.164508901175258f,
    0.164427510912706f,    0.159984394367208f,    0.159903007598291f,
    0.159821622642703f,    0.159740239544724f,    0.159658858348595f,
    0.159577479098508f,    0.159496101838615f,    0.159414726613023f,
    0.159333353465794f,    0.159251982440948f,    0.159170613582459f,
    0.159089246934261f,    0.159007882540240f,    0.158926520444241f,
    0.158845160690063f,    0.158763803321464f,    0.158682448382156f,
    0.158601095915807f,    0.158519745966043f,    0.158438398576445f,
    0.158357053790551f,    0.158275711651853f,    0.158194372203803f,
    0.158113035489805f,    0.158031701553222f,    0.157950370437373f,
    0.157869042185531f,    0.157787716840929f,    0.157706394446753f,
    0.157625075046145f,    0.157543758682207f,    0.157462445397992f,
    0.157381135236513f,    0.157299828240738f,    0.157218524453592f,
    0.157137223917953f,    0.157055926676660f,    0.156974632772505f,
    0.156893342248236f,    0.156812055146560f,    0.156730771510136f,
    0.156649491381584f,    0.156568214803476f,    0.156486941818342f,
    0.156405672468670f,    0.156324406796900f,    0.151881416837854f,
    0.151800158649042f,    0.151718904265197f,    0.151637653728586f,
    0.151556407081432f,    0.151475164365916f,    0.151393925624172f,
    0.151312690898293f,    0.151231460230326f,    0.151150233662276f,
    0.151069011236103f,    0.150987792993723f,    0.150906578977011f,
    0.150825369227794f,    0.150744163787857f,    0.150662962698943f,
    0.150581766002748f,    0.150500573740927f,    0.150419385955089f,
    0.150338202686801f,    0.150257023977585f,    0.150175849868919f,
    0.150094680402237f,    0.150013515618932f,    0.149932355560350f,
    0.149851200267794f,    0.149770049782523f,    0.149688904145754f,
    0.149607763398657f,    0.149526627582362f,    0.149445496737952f,
    0.149364370906467f,    0.149283250128905f,    0.149202134446217f,
    0.149121023899313f,    0.149039918529058f,    0.148958818376273f,
    0.148877723481736f,    0.148796633886180f,    0.148715549630295f,
    0.148634470754727f,    0.148553397300078f,    0.148472329306907f,
    0.148391266815728f,    0.148310209867012f,    0.148229158501186f,
    0.148148112758633f,    0.148067072679692f,    0.143624310297081f,
    0.143543281666207f,    0.143462258819700f,    0.143381241797725f,
    0.143300230640400f,    0.143219225387804f,    0.143138226079968f,
    0.143057232756881f,    0.142976245458488f,    0.142895264224689f,
    0.142814289095343f,    0.142733320110263f,    0.142652357309218f,
    0.142571400731934f,    0.142490450418093f,    0.142409506407333f,
    0.142328568739249f,    0.142247637453390f,    0.142166712589265f,
    0.142085794186334f,    0.142004882284019f,    0.141923976921693f,
    0.141843078138688f,    0.141762185974292f,    0.141681300467748f,
    0.141600421658257f,    0.141519549584974f,    0.141438684287012f,
    0.141357825803438f,    0.141276974173279f,    0.141196129435513f,
    0.141115291629079f,    0.141034460792870f,    0.140953636965734f,
    0.140872820186478f,    0.140792010493862f,    0.140711207926605f,
    0.140630412523381f,    0.140549624322819f,    0.140468843363506f,
    0.140388069683985f,    0.140307303322754f,    0.140226544318269f,
    0.140145792708939f,    0.140065048533132f,    0.139984311829173f,
    0.139903582635339f,    0.139822860989867f,    0.139742146930948f,
    0.139661440496731f,    0.139580741725320f,    0.139500050654775f,
    0.139419367323112f,    0.134976963760728f,    0.134896296020705f,
    0.134815636133351f,    0.134734984136508f,    0.134654340067973f,
    0.134573703965498f,    0.134493075866795f,    0.134412455809528f,
    0.134331843831321f,    0.134251239969750f,    0.134170644262351f,
    0.134090056746613f,    0.134009477459984f,    0.133928906439867f,
    0.133848343723620f,    0.133767789348558f,    0.133687243351954f,
    0.133606705771033f,    0.133526176642981f,    0.133445656004937f,
    0.133365143893997f,    0.133284640347212f,    0.133204145401592f,
    0.133123659094101f,    0.133043181461659f,    0.132962712541143f,
    0.132882252369387f,    0.132801800983178f,    0.132721358419264f,
    0.132640924714344f,    0.132560499905078f,    0.132480084028077f,
    0.132399677119913f,    0.132319279217111f,    0.132238890356154f,
    0.132158510573480f,    0.132078139905483f,    0.131997778388515f,
    0.131917426058882f,    0.131837082952847f,    0.131756749106629f,
    0.131676424556405f,    0.131596109338304f,    0.131515803488415f,
    0.131435507042783f,    0.131355220037405f,    0.131274942508240f,
    0.131194674491199f,    0.131114416022150f,    0.131034167136919f,
    0.130953927871285f,    0.130873698260986f,    0.130793478341716f,
    0.130713268149122f,    0.130633067718811f,    0.130552877086344f,
    0.130472696287238f,    0.130392525356968f,    0.130312364330964f,
    0.130232213244611f,    0.130152072133252f,    0.130071941032186f,
    0.125630091969089f,    0.125549980994328f,    0.125469880135491f,
    0.125389789427702f,    0.125309708906040f,    0.125229638605540f,
    0.125149578561194f,    0.125069528807950f,    0.124989489380710f,
    0.124909460314337f,    0.124829441643644f,    0.124749433403405f,
    0.124669435628348f,    0.124589448353158f,    0.124509471612476f,
    0.124429505440897f,    0.124349549872976f,    0.124269604943221f,
    0.124189670686097f,    0.124109747136027f,    0.124029834327388f,
    0.123949932294513f,    0.123870041071692f,    0.123790160693172f,
    0.123710291193154f,    0.123630432605798f,    0.123550584965217f,
    0.123470748305482f,    0.123390922660619f,    0.123311108064613f,
    0.123231304551401f,    0.123151512154880f,    0.123071730908899f,
    0.122991960847268f,    0.122912202003750f,    0.122832454412063f,
    0.122752718105885f,    0.122672993118847f,    0.122593279484538f,
    0.122513577236502f,    0.122433886408239f,    0.122354207033207f,
    0.122274539144817f,    0.122194882776439f,    0.122115237961398f,
    0.122035604732975f,    0.121955983124407f,    0.121876373168889f,
    0.121796774899569f,    0.121717188349554f,    0.121637613551904f,
    0.121558050539640f,    0.121478499345734f,    0.121398960003116f,
    0.121319432544675f,    0.121239917003251f,    0.121160413411644f,
    0.121080921802609f,    0.121001442208857f,    0.120921974663054f,
    0.120842519197825f,    0.120763075845748f,    0.120683644639360f,
    0.120604225611152f,    0.120524818793572f,    0.120445424219023f,
    0.120366041919867f,    0.120286671928419f,    0.120207314276951f,
    0.120127968997693f,    0.120048636122829f,    0.119969315684500f,
    0.119890007714802f,    0.119810712245789f,    0.119731429309471f,
    0.119652158937811f,    0.119572901162733f,    0.119493656016113f,
    0.119414423529785f,    0.119335203735540f,    0.114894268657545f,
    0.114815074342658f,    0.114735892814960f,    0.114656724106065f,
    0.114577568247544f,    0.114498425270923f,    0.114419295207686f,
    0.114340178089270f,    0.114261073947072f,    0.114181982812442f,
    0.114102904716688f,    0.114023839691073f,    0.113944787766818f,
    0.113865748975096f,    0.113786723347042f,    0.113707710913742f,
    0.113628711706240f,    0.113549725755538f,    0.113470753092591f,
    0.113391793748312f,    0.113312847753569f,    0.113233915139187f,
    0.113154995935948f,    0.113076090174588f,    0.112997197885800f,
    0.112918319100234f,    0.112839453848495f,    0.112760602161145f,
    0.112681764068701f,    0.112602939601637f,    0.112524128790383f,
    0.112445331665325f,    0.112366548256805f,    0.112287778595122f,
    0.112209022710530f,    0.112130280633239f,    0.112051552393417f,
    0.111972838021186f,    0.111894137546624f,    0.111815450999768f,
    0.111736778410607f,    0.111658119809091f,    0.111579475225121f,
    0.111500844688557f,    0.111422228229216f,    0.111343625876868f,
    0.111265037661242f,    0.111186463612022f,    0.111107903758848f,
    0.111029358131316f,    0.110950826758979f,    0.110872309671346f,
    0.110793806897879f,    0.110715318468002f,    0.110636844411090f,
    0.110558384756477f,    0.110479939533451f,    0.110401508771258f,
    0.110323092499099f,    0.110244690746132f,    0.110166303541471f,
    0.110087930914184f,    0.110009572893298f,    0.109931229507795f,
    0.109852900786614f,    0.109774586758647f,    0.109696287452746f,
    0.109618002897716f,    0.109539733122321f,    0.109461478155279f,
    0.109383238025265f,    0.109305012760910f,    0.109226802390801f,
    0.109148606943480f,    0.109070426447448f,    0.108992260931160f,
    0.108914110423026f,    0.108835974951415f,    0.108757854544651f,
    0.108679749231012f,    0.108601659038736f,    0.108523583996014f,
    0.108445524130995f,    0.108367479471782f,    0.108289450046436f,
    0.108211435882973f,    0.108133437009367f,    0.108055453453546f,
    0.107977485243395f,    0.107899532406754f,    0.107821594971421f,
    0.107743672965149f,    0.107665766415648f,    0.107587875350582f,
    0.107509999797574f,    0.107432139784200f,    0.107354295337995f,
    0.107276466486449f,    0.107198653257007f,    0.107120855677072f,
    0.107043073774001f,    0.106965307575109f,    0.106887557107667f,
    0.106809822398901f,    0.106732103475993f,    0.106654400366083f,
    0.106576713096265f,    0.106499041693590f,    0.106421386185065f,
    0.106343746597653f,    0.106266122958274f,    0.106188515293803f,
    0.106110923631072f,    0.106033347996867f,    0.105955788417934f,
    0.105878244920971f,    0.105800717532635f,    0.105723206279537f,
    0.105645711188245f,    0.105568232285285f,    0.105490769597136f,
    0.105413323150235f,    0.105335892970973f,    0.105258479085701f,
    0.105181081520723f,    0.105103700302298f,    0.105026335456646f,
    0.104948987009938f,    0.104871654988304f,    0.104794339417828f,
    0.104717040324553f,    0.104639757734476f,    0.104562491673551f,
    0.104485242167686f,    0.104408009242749f,    0.104330792924560f,
    0.104253593238899f,    0.104176410211498f,    0.104099243868048f,
    0.104022094234196f,    0.103944961335543f,    0.103867845197649f,
    0.103790745846028f,    0.103713663306151f,    0.103636597603444f,
    0.103559548763290f,    0.103482516811028f,    0.103405501771954f,
    0.103328503671319f,    0.103251522534330f,    0.103174558386149f,
    0.103097611251898f,    0.103020681156651f,    0.102943768125441f,
    0.102866872183254f,    0.102789993355035f,    0.102713131665683f,
    0.102636287140056f,    0.102559459802964f,    0.102482649679177f,
    0.102405856793418f,    0.102329081170369f,    0.102252322834665f,
    0.102175581810900f,    0.102098858123621f,    0.102022151797335f,
    0.101945462856502f,    0.101868791325538f,    0.101792137228817f,
    0.101715500590669f,    0.101638881435378f,    0.101562279787186f,
    0.101485695670291f,    0.101409129108846f,    0.101332580126960f,
    0.101256048748701f,    0.101179534998088f,    0.101103038899101f,
    0.101026560475674f,    0.100950099751696f,    0.100873656751014f,
    0.100797231497430f,    0.100720824014703f,    0.100644434326547f,
    0.100568062456633f,    0.100491708428587f,    0.100415372265992f,
    0.100339053992388f,    0.100262753631269f,    0.100186471206086f,
    0.100110206740246f,    0.100033960257113f,    0.0999577317800052f,
    0.0998815213321992f,   0.0998053289369260f,   0.0997291546173731f,
    0.0996529983966844f,   0.0995768602979594f,   0.0995007403442542f,
    0.0994246385585807f,   0.0993485549639068f,   0.0992724895831567f,
    0.0991964424392106f,   0.0991204135549048f,   0.0990444029530317f,
    0.0989684106563397f,   0.0988924366875334f,   0.0988164810692734f,
    0.0987405438241764f,   0.0986646249748153f,   0.0985887245437189f,
    0.0985128425533722f,   0.0984369790262163f,   0.0983611339846482f,
    0.0982853074510214f,   0.0982094994476449f,   0.0981337099967844f,
    0.0980579391206610f,   0.0979821868414527f,   0.0979064531812928f,
    0.0978307381622712f,   0.0977550418064336f,   0.0976793641357820f,
    0.0976037051722743f,   0.0975280649378246f,   0.0974524434543029f,
    0.0973768407435356f,   0.0973012568273050f,   0.0972256917273493f,
    0.0971501454653631f,   0.0970746180629969f,   0.0969991095418574f,
    0.0969236199235071f,   0.0968481492294650f,   0.0967726974812058f,
    0.0966972647001606f,   0.0966218509077163f,   0.0965464561252159f,
    0.0964710803739588f,   0.0963957236752001f,   0.0963203860501512f,
    0.0962450675199795f,   0.0961697681058085f,   0.0960944878287176f,
    0.0960192267097427f,   0.0959439847698754f,   0.0958687620300635f,
    0.0957935585112108f,   0.0957183742341774f,   0.0956432092197791f,
    0.0955680634887883f,   0.0954929370619329f,   0.0954178299598974f,
    0.0953427422033218f,   0.0952676738128029f,   0.0951926248088929f,
    0.0951175952121004f,   0.0950425850428900f,   0.0949675943216825f,
    0.0948926230688546f,   0.0948176713047392f,   0.0947427390496252f,
    0.0946678263237576f,   0.0945929331473374f,   0.0945180595405218f,
    0.0944432055234240f,   0.0943683711161133f,   0.0942935563386150f,
    0.0942187612109106f,   0.0941439857529376f,   0.0940692299845894f,
    0.0939944939257160f,   0.0939197775961227f,   0.0938450810155716f,
    0.0937704042037804f,   0.0936957471804231f,   0.0936211099651296f,
    0.0935464925774860f,   0.0934718950370345f,   0.0933973173632733f,
    0.0933227595756565f,   0.0932482216935947f,   0.0931737037364541f,
    0.0930992057235573f,   0.0930247276741828f,   0.0929502696075653f,
    0.0928758315428953f,   0.0928014134993197f,   0.0927270154959413f,
    0.0926526375518190f,   0.0925782796859676f,   0.0925039419173584f,
    0.0924296242649182f,   0.0923553267475303f,   0.0922810493840339f,
    0.0922067921932243f,   0.0921325551938528f,   0.0920583384046269f,
    0.0919841418442100f,   0.0919099655312217f,   0.0918358094842375f,
    0.0917616737217892f,   0.0916875582623645f,   0.0916134631244072f,
    0.0915393883263172f,   0.0914653338864503f,   0.0913912998231188f,
    0.0913172861545903f,   0.0912432928990893f,   0.0911693200747959f,
    0.0910953676998462f,   0.0910214357923326f,   0.0909475243703036f,
    0.0908736334517634f,   0.0907997630546726f,   0.0907259131969478f,
    0.0906520838964616f,   0.0905782751710426f,   0.0905044870384756f,
    0.0904307195165015f,   0.0903569726228168f,   0.0902832463750749f,
    0.0902095407908844f,   0.0901358558878106f,   0.0900621916833744f,
    0.0899885481950531f,   0.0899149254402798f,   0.0898413234364439f,
    0.0897677422008906f,   0.0896941817509215f,   0.0896206421037937f,
    0.0895471232767210f,   0.0894736252868730f,   0.0894001481513751f,
    0.0893266918873091f,   0.0892532565117127f,   0.0891798420415798f,
    0.0891064484938601f,   0.0890330758854596f,   0.0889597242332403f,
    0.0888863935540201f,   0.0888130838645731f,   0.0887397951816295f,
    0.0886665275218755f,   0.0885932809019533f,   0.0885200553384611f,
    0.0884468508479535f,   0.0883736674469406f,   0.0883005051518891f,
    0.0882273639792214f,   0.0881542439453161f,   0.0880811450665078f,
    0.0880080673590872f,   0.0879350108393010f,   0.0878619755233519f,
    0.0877889614273989f,   0.0877159685675568f,   0.0876429969598966f,
    0.0875700466204452f,   0.0874971175651857f,   0.0874242098100572f,
    0.0873513233709547f,   0.0872784582637296f,   0.0872056145041890f,
    0.0871327921080962f,   0.0870599910911707f,   0.0869872114690878f,
    0.0869144532574789f,   0.0868417164719315f,   0.0867690011279892f,
    0.0866963072411517f,   0.0866236348268744f,   0.0865509839005692f,
    0.0864783544776037f,   0.0864057465733018f,   0.0863331602029433f,
    0.0862605953817640f,   0.0861880521249560f,   0.0861155304476673f,
    0.0860430303650017f,   0.0859705518920195f,   0.0858980950437366f,
    0.0858256598351254f,   0.0857532462811140f,   0.0856808543965868f,
    0.0856084841963839f,   0.0855361356953018f,   0.0854638089080928f,
    0.0853915038494655f,   0.0853192205340843f,   0.0852469589765697f,
    0.0851747191914984f,   0.0851025011934029f,   0.0850303049967719f,
    0.0849581306160502f,   0.0848859780656385f,   0.0848138473598935f,
    0.0847417385131283f,   0.0846696515396116f,   0.0845975864535684f,
    0.0845255432691795f,   0.0844535220005823f,   0.0843815226618695f,
    0.0843095452670904f,   0.0842375898302500f,   0.0841656563653097f,
    0.0840937448861865f,   0.0840218554067539f,   0.0839499879408410f,
    0.0838781425022332f,   0.0838063191046720f,   0.0837345177618548f,
    0.0836627384874350f,   0.0835909812950222f,   0.0835192461981819f,
    0.0834475332104356f,   0.0833758423452611f,   0.0833041736160921f,
    0.0832325270363181f,   0.0831609026192851f,   0.0830893003782947f,
    0.0830177203266048f,   0.0829461624774291f,   0.0828746268439379f,
    0.0828031134392568f,   0.0827316222764678f,   0.0826601533686091f,
    0.0825887067286746f,   0.0825172823696144f,   0.0824458803043347f,
    0.0823745005456977f,   0.0823031431065214f,   0.0822318079995802f,
    0.0821604952376043f,   0.0820892048332801f,   0.0820179367992499f,
    0.0819466911481119f,   0.0818754678924208f,   0.0818042670446868f,
    0.0817330886173767f,   0.0816619326229126f,   0.0815907990736732f,
    0.0815196879819933f,   0.0814485993601633f,   0.0813775332204299f,
    0.0813064895749958f,   0.0812354684360197f,   0.0811644698156163f,
    0.0810934937258564f,   0.0810225401787669f,   0.0809516091863306f,
    0.0808807007604862f,   0.0808098149131290f,   0.0807389516561096f,
    0.0806681110012352f,   0.0805972929602686f,   0.0805264975449289f,
    0.0804557247668912f,   0.0803849746377866f,   0.0803142471692022f,
    0.0802435423726812f,   0.0801728602597226f,   0.0801022008417819f,
    0.0800315641302702f,   0.0799609501365547f,   0.0798903588719589f,
    0.0798197903477619f,   0.0797492445751992f,   0.0796787215654622f,
    0.0796082213296982f,   0.0795377438790107f,   0.0794672892244592f,
    0.0793968573770592f,   0.0793264483477822f,   0.0792560621475557f,
    0.0791856987872634f,   0.0791153582777449f,   0.0790450406297956f,
    0.0789747458541675f,   0.0789044739615680f,   0.0788342249626610f,
    0.0787639988680661f,   0.0786937956883591f,   0.0786236154340719f,
    0.0785534581156922f,   0.0784833237436637f,   0.0784132123283866f,
    0.0783431238802165f,   0.0782730584094654f,   0.0782030159264013f,
    0.0781329964412480f,   0.0780629999641855f,   0.0779930265053501f,
    0.0779230760748334f,   0.0778531486826837f,   0.0777832443389050f,
    0.0777133630534574f,   0.0776435048362570f,   0.0775736696971760f,
    0.0775038576460424f,   0.0774340686926407f,   0.0773643028467107f,
    0.0772945601179488f,   0.0772248405160073f,   0.0771551440504945f,
    0.0770854707309746f,   0.0770158205669677f,   0.0769461935679505f,
    0.0768765897433551f,   0.0768070091025700f,   0.0767374516549394f,
    0.0766679174097639f,   0.0765984063762999f,   0.0765289185637596f,
    0.0764594539813117f,   0.0763900126380807f,   0.0763205945431470f,
    0.0762511997055470f,   0.0761818281342734f,   0.0761124798382747f,
    0.0760431548264555f,   0.0759738531076762f,   0.0759045746907537f,
    0.0758353195844604f,   0.0757660877975250f,   0.0756968793386320f,
    0.0756276942164223f,   0.0755585324394925f,   0.0754893940163953f,
    0.0754202789556392f,   0.0753511872656892f,   0.0752821189549660f,
    0.0752130740318462f,   0.0751440525046627f,   0.0750750543817043f,
    0.0750060796712157f,   0.0749371283813978f,   0.0748682005204073f,
    0.0747992960963572f,   0.0747304151173163f,   0.0746615575913094f,
    0.0745927235263174f,   0.0745239129302773f,   0.0744551258110818f,
    0.0743863621765800f,   0.0699558940269990f,   0.0698871773852553f,
    0.0698184842514879f,   0.0697498146333701f,   0.0696811685385305f,
    0.0696125459745543f,   0.0695439469489825f,   0.0694753714693121f,
    0.0694068195429960f,   0.0693382911774433f,   0.0692697863800191f,
    0.0692013051580444f,   0.0691328475187962f,   0.0690644134695077f,
    0.0689960030173678f,   0.0689276161695217f,   0.0688592529330704f,
    0.0687909133150711f,   0.0687225973225368f,   0.0686543049624368f,
    0.0685860362416959f,   0.0685177911671956f,   0.0684495697457728f,
    0.0683813719842208f,   0.0683131978892886f,   0.0682450474676814f,
    0.0681769207260604f,   0.0681088176710428f,   0.0680407383092017f,
    0.0679726826470663f,   0.0679046506911219f,   0.0678366424478096f,
    0.0677686579235266f,   0.0677006971246261f,   0.0676327600574174f,
    0.0675648467281656f,   0.0674969571430920f,   0.0674290913083737f,
    0.0673612492301441f,   0.0672934309144923f,   0.0672256363674636f,
    0.0671578655950593f,   0.0670901186032365f,   0.0670223953979085f,
    0.0669546959849446f,   0.0668870203701700f,   0.0668193685593660f,
    0.0667517405582697f,   0.0666841363725746f,   0.0666165560079299f,
    0.0665489994699407f,   0.0664814667641684f,   0.0664139578961303f,
    0.0663464728712994f,   0.0662790116951054f,   0.0662115743729332f,
    0.0661441609101242f,   0.0660767713119756f,   0.0660094055837408f,
    0.0659420637306289f,   0.0658747457578054f,   0.0658074516703914f,
    0.0657401814734641f,   0.0656729351720569f,   0.0656057127711590f,
    0.0655385142757157f,   0.0654713396906282f,   0.0654041890207538f,
    0.0653370622709057f,   0.0652699594458532f,   0.0652028805503216f,
    0.0651358255889920f,   0.0650687945665018f,   0.0650017874874442f,
    0.0649348043563684f,   0.0648678451777795f,   0.0648009099561390f,
    0.0647339986958640f,   0.0646671114013277f,   0.0646002480768593f,
    0.0645334087267441f,   0.0644665933552233f,   0.0643998019664940f,
    0.0643330345647095f,   0.0642662911539789f,   0.0641995717383674f,
    0.0641328763218963f,   0.0640662049085427f,   0.0639995575022397f,
    0.0639329341068766f,   0.0638663347262984f,   0.0637997593643063f,
    0.0637332080246576f,   0.0636666807110652f,   0.0636001774271983f,
    0.0635336981766821f,   0.0634672429630977f,   0.0634008117899821f,
    0.0633344046608284f,   0.0632680215790859f,   0.0632016625481595f,
    0.0631353275714102f,   0.0630690166521551f,   0.0630027297936674f,
    0.0629364669991760f,   0.0628702282718661f,   0.0628040136148784f,
    0.0627378230313101f,   0.0626716565242143f,   0.0626055140965999f,
    0.0625393957514318f,   0.0624733014916310f,   0.0624072313200746f,
    0.0623411852395954f,   0.0622751632529823f,   0.0622091653629804f,
    0.0621431915722904f,   0.0620772418835694f,   0.0620113162994302f,
    0.0619454148224416f,   0.0618795374551285f,   0.0618136841999718f,
    0.0617478550594083f,   0.0616820500358309f,   0.0616162691315882f,
    0.0615505123489852f,   0.0614847796902825f,   0.0614190711576971f,
    0.0613533867534016f,   0.0612877264795248f,   0.0612220903381512f,
    0.0611564783313218f,   0.0610908904610332f,   0.0610253267292380f,
    0.0609597871378448f,   0.0608942716887185f,   0.0608287803836795f,
    0.0607633132245045f,   0.0606978702129261f,   0.0606324513506329f,
    0.0605670566392694f,   0.0605016860804361f,   0.0604363396756896f,
    0.0603710174265425f,   0.0603057193344631f,   0.0602404454008760f,
    0.0601751956271616f,   0.0601099700146564f,   0.0600447685646528f,
    0.0599795912783991f,   0.0599144381570998f,   0.0598493092019153f,
    0.0597842044139619f,   0.0597191237943117f,   0.0596540673439933f,
    0.0595890350639909f,   0.0595240269552448f,   0.0594590430186511f,
    0.0593940832550622f,   0.0593291476652861f,   0.0592642362500872f,
    0.0591993490101854f,   0.0591344859462571f,   0.0590696470589343f,
    0.0590048323488052f,   0.0589400418164136f,   0.0588752754622598f,
    0.0588105332867999f,   0.0587458152904457f,   0.0586811214735651f,
    0.0586164518364824f,   0.0585518063794773f,   0.0584871851027858f,
    0.0584225880065996f,   0.0583580150910669f,   0.0582934663562913f,
    0.0582289418023328f,   0.0581644414292068f,   0.0580999652368856f,
    0.0580355132252967f,   0.0579710853943238f,   0.0579066817438065f,
    0.0578423022735407f,   0.0577779469832779f,   0.0577136158727258f,
    0.0576493089415478f,   0.0575850261893637f,   0.0575207676157490f,
    0.0574565332202351f,   0.0573923230023095f,   0.0573281369614158f,
    0.0572639750969533f,   0.0571998374082775f,   0.0571357238946997f,
    0.0570716345554873f,   0.0570075693898635f,   0.0569435283970079f,
    0.0568795115760553f,   0.0568155189260974f,   0.0567515504461811f,
    0.0566876061353098f,   0.0566236859924424f,   0.0565597900164942f,
    0.0564959182063363f,   0.0564320705607957f,   0.0563682470786554f,
    0.0563044477586545f,   0.0562406725994879f,   0.0561769215998064f,
    0.0561131947582173f,   0.0560494920732831f,   0.0559858135435229f,
    0.0559221591674113f,   0.0558585289433793f,   0.0557949228698135f,
    0.0557313409450567f,   0.0556677831674074f,   0.0556042495351205f,
    0.0555407400464066f,   0.0554772546994322f,   0.0554137934923198f,
    0.0553503564231481f,   0.0552869434899515f,   0.0552235546907205f,
    0.0551601900234014f,   0.0550968494858967f,   0.0550335330760648f,
    0.0549702407917199f,   0.0549069726306323f,   0.0548437285905283f,
    0.0547805086690902f,   0.0547173128639560f,   0.0546541411727200f,
    0.0545909935929322f,   0.0545278701220988f,   0.0544647707576817f,
    0.0544016954970989f,   0.0543386443377246f,   0.0542756172768886f,
    0.0542126143118766f,   0.0541496354399307f,   0.0540866806582487f,
    0.0540237499639842f,   0.0539608433542472f,   0.0538979608261031f,
    0.0538351023765739f,   0.0537722680026371f,   0.0537094577012262f,
    0.0536466714692308f,   0.0535839093034965f,   0.0535211712008247f,
    0.0534584571579729f,   0.0533957671716543f,   0.0533331012385385f,
    0.0532704593552508f,   0.0532078415183724f,   0.0531452477244405f,
    0.0530826779699484f,   0.0530201322513452f,   0.0529576105650360f,
    0.0528951129073819f,   0.0528326392746999f,   0.0527701896632632f,
    0.0527077640693004f,   0.0526453624889966f,   0.0525829849184928f,
    0.0525206313538857f,   0.0524583017912279f,   0.0523959962265285f,
    0.0523337146557520f,   0.0522714570748190f,   0.0522092234796062f,
    0.0521470138659462f,   0.0520848282296275f,   0.0520226665663946f,
    0.0519605288719478f,   0.0518984151419437f,   0.0518363253719946f,
    0.0517742595576687f,   0.0517122176944903f,   0.0516501997779397f,
    0.0515882058034530f,   0.0515262357664224f,   0.0514642896621959f,
    0.0514023674860775f,   0.0513404692333274f,   0.0512785948991613f,
    0.0512167444787511f,   0.0511549179672250f,   0.0510931153596663f,
    0.0510313366511152f,   0.0509695818365671f,   0.0509078509109738f,
    0.0508461438692430f,   0.0507844607062381f,   0.0507228014167787f,
    0.0506611659956403f,   0.0505995544375543f,   0.0505379667372080f,
    0.0504764028892448f,   0.0504148628882640f,   0.0503533467288209f,
    0.0502918544054265f,   0.0502303859125480f,   0.0501689412446085f,
    0.0501075203959871f,   0.0500461233610187f,   0.0499847501339942f,
    0.0499234007091606f,   0.0498620750807206f,   0.0498007732428331f,
    0.0497394951896126f,   0.0496782409151302f,   0.0496170104134122f,
    0.0495558036784412f,   0.0494946207041559f,   0.0494334614844505f,
    0.0493723260131757f,   0.0493112142841377f,   0.0492501262910988f,
    0.0491890620277774f,   0.0491280214878476f,   0.0490670046649396f,
    0.0490060115526395f,   0.0489450421444893f,   0.0488840964339871f,
    0.0488231744145867f,   0.0487622760796981f,   0.0487014014226872f,
    0.0486405504368756f,   0.0485797231155411f,   0.0485189194519174f,
    0.0484581394391941f,   0.0483973830705168f,   0.0483366503389869f,
    0.0482759412376620f,   0.0482152557595553f,   0.0481545938976364f,
    0.0480939556448302f,   0.0480333409940183f,   0.0479727499380377f,
    0.0479121824696815f,   0.0478516385816988f,   0.0477911182667946f,
    0.0477306215176298f,   0.0476701483268213f,   0.0476096986869419f,
    0.0475492725905205f,   0.0474888700300417f,   0.0474284909979461f,
    0.0473681354866303f,   0.0473078034884470f,   0.0472474949957045f,
    0.0471872100006673f,   0.0471269484955557f,   0.0470667104725461f,
    0.0470064959237706f,   0.0469463048413175f,   0.0468861372172308f,
    0.0468259930435107f,   0.0467658723121130f,   0.0467057750149499f,
    0.0466457011438889f,   0.0465856506907542f,   0.0465256236473254f,
    0.0464656200053382f,   0.0464056397564841f,   0.0463456828924109f,
    0.0462857494047220f,   0.0462258392849768f,   0.0461659525246906f,
    0.0461060891153350f,   0.0460462490483370f,   0.0459864323150799f,
    0.0459266389069028f,   0.0458668688151008f,   0.0458071220309249f,
    0.0457473985455820f,   0.0456876983502350f,   0.0456280214360027f,
    0.0455683677939599f,   0.0455087374151372f,   0.0454491302905213f,
    0.0453895464110547f,   0.0453299857676360f,   0.0452704483511194f,
    0.0452109341523154f,   0.0451514431619903f,   0.0450919753708663f,
    0.0450325307696215f,   0.0449731093488901f,   0.0449137110992621f,
    0.0448543360112835f,   0.0447949840754560f,   0.0447356552822377f,
    0.0446763496220422f,   0.0446170670852392f,   0.0445578076621543f,
    0.0444985713430691f,   0.0444393581182212f,   0.0443801679778039f,
    0.0443210009119664f,   0.0442618569108143f,   0.0442027359644086f,
    0.0441436380627666f,   0.0440845631958612f,   0.0440255113536215f,
    0.0439664825259325f,   0.0439074767026350f,   0.0438484938735257f,
    0.0437895340283575f,   0.0437305971568391f,   0.0436716832486349f,
    0.0436127922933654f,   0.0435539242806073f,   0.0434950791998928f,
    0.0434362570407102f,   0.0433774577925038f,   0.0433186814446737f,
    0.0432599279865760f,   0.0432011974075228f,   0.0431424896967819f,
    0.0430838048435774f,   0.0430251428370888f,   0.0429665036664521f,
    0.0429078873207587f,   0.0428492937890565f,   0.0427907230603488f,
    0.0427321751235950f,   0.0426736499677106f,   0.0426151475815668f,
    0.0425566679539909f,   0.0424982110737659f,   0.0424397769296309f,
    0.0423813655102811f,   0.0423229768043671f,   0.0422646108004960f,
    0.0422062674872303f,   0.0421479468530891f,   0.0420896488865467f,
    0.0420313735760336f,   0.0419731209099364f,   0.0419148908765975f,
    0.0418566834643152f,   0.0417984986613437f,   0.0417403364558930f,
    0.0416821968361295f,   0.0416240797901751f,   0.0415659853061075f,
    0.0415079133719608f,   0.0414498639757248f,   0.0413918371053450f,
    0.0413338327487231f,   0.0412758508937167f,   0.0412178915281392f,
    0.0411599546397599f,   0.0411020402163043f,   0.0410441482454534f,
    0.0409862787148446f,   0.0409284316120707f,   0.0408706069246808f,
    0.0408128046401798f,   0.0407550247460286f,   0.0406972672296439f,
    0.0406395320783983f,   0.0405818192796204f,   0.0405241288205948f,
    0.0404664606885619f,   0.0404088148707178f,   0.0403511913542151f,
    0.0402935901261618f,   0.0402360111736221f,   0.0401784544836158f,
    0.0401209200431191f,   0.0400634078390637f,   0.0400059178583375f,
    0.0399484500877839f,   0.0398910045142029f,   0.0398335811243497f,
    0.0397761799049359f,   0.0397188008426288f,   0.0396614439240516f,
    0.0396041091357837f,   0.0395467964643600f,   0.0394895058962715f,
    0.0394322374179654f,   0.0393749910158443f,   0.0393177666762671f,
    0.0392605643855483f,   0.0392033841299587f,   0.0391462258957247f,
    0.0390890896690288f,   0.0390319754360092f,   0.0389748831827604f,
    0.0389178128953323f,   0.0388607645597312f,   0.0388037381619189f,
    0.0387467336878135f,   0.0386897511232887f,   0.0386327904541742f,
    0.0385758516662558f,   0.0385189347452749f,   0.0384620396769291f,
    0.0384051664468718f,   0.0383483150407121f,   0.0382914854440155f,
    0.0382346776423029f,   0.0381778916210515f,   0.0381211273656940f,
    0.0380643848616195f,   0.0380076640941726f,   0.0379509650486541f,
    0.0378942877103206f,   0.0378376320643845f,   0.0377809980960143f,
    0.0377243857903341f,   0.0376677951324245f,   0.0376112261073215f,
    0.0375546787000170f,   0.0374981528954590f,   0.0374416486785515f,
    0.0373851660341542f,   0.0373287049470828f,   0.0372722654021088f,
    0.0372158473839598f,   0.0371594508773193f,   0.0371030758668266f,
    0.0370467223370766f,   0.0369903902726209f,   0.0369340796579662f,
    0.0368777904775757f,   0.0368215227158680f,   0.0367652763572181f,
    0.0367090513859566f,   0.0366528477863700f,   0.0365966655427007f,
    0.0365405046391474f,   0.0364843650598642f,   0.0364282467889613f,
    0.0363721498105048f,   0.0363160741085168f,   0.0362600196669752f,
    0.0362039864698137f,   0.0361479745009222f,   0.0360919837441462f,
    0.0360360141832873f,   0.0359800658021030f,   0.0359241385843064f,
    0.0358682325135671f,   0.0358123475735100f,   0.0357564837477163f,
    0.0357006410197228f,   0.0356448193730226f,   0.0355890187910643f,
    0.0355332392572527f,   0.0354774807549481f,   0.0354217432674673f,
    0.0353660267780825f,   0.0353103312700220f,   0.0352546567264699f,
    0.0351990031305666f,   0.0351433704654078f,   0.0350877587140454f,
    0.0350321678594873f,   0.0349765978846971f,   0.0349210487725945f,
    0.0348655205060549f,   0.0348100130679097f,   0.0347545264409461f,
    0.0346990606079075f,   0.0346436155514929f,   0.0345881912543573f,
    0.0345327876991115f,   0.0344774048683225f,   0.0344220427445127f,
    0.0343667013101609f,   0.0343113805477016f,   0.0342560804395250f,
    0.0342008009679776f,   0.0341455421153614f,   0.0340903038639347f,
    0.0340350861959113f,   0.0339798890934610f,   0.0339247125387098f,
    0.0338695565137393f,   0.0338144210005871f,   0.0337593059812464f,
    0.0337042114376669f,   0.0336491373517538f,   0.0335940837053681f,
    0.0335390504803269f,   0.0334840376584033f,   0.0334290452213260f,
    0.0333740731507797f,   0.0333191214284051f,   0.0332641900357988f,
    0.0332092789545132f,   0.0331543881660566f,   0.0330995176518931f,
    0.0330446673934429f,   0.0329898373720821f,   0.0329350275691426f,
    0.0328802379659119f,   0.0328254685436341f,   0.0327707192835085f,
    0.0327159901666906f,   0.0326612811742918f,   0.0326065922873795f,
    0.0325519234869767f,   0.0324972747540625f,   0.0324426460695717f,
    0.0323880374143954f,   0.0323334487693801f,   0.0322788801153286f,
    0.0322243314329991f,   0.0321698027031064f,   0.0321152939063204f,
    0.0320608050232677f,   0.0320063360345299f,   0.0319518869206454f,
    0.0318974576621078f,   0.0318430482393670f,   0.0317886586328284f,
    0.0317342888228537f,   0.0316799387897604f,   0.0316256085138217f,
    0.0315712979752667f,   0.0315170071542806f,   0.0314627360310044f,
    0.0314084845855349f,   0.0313542527979248f,   0.0313000406481829f,
    0.0312458481162737f,   0.0311916751821174f,   0.0311375218255906f,
    0.0310833880265254f,   0.0310292737647098f,   0.0309751790198877f,
    0.0309211037717592f,   0.0308670479999799f,   0.0308130116841615f,
    0.0307589948038714f,   0.0307049973386331f,   0.0306510192679259f,
    0.0305970605711850f,   0.0305431212278014f,   0.0304892012171220f,
    0.0304353005184498f,   0.0303814191110434f,   0.0303275569741174f,
    0.0302737140868423f,   0.0302198904283447f,   0.0301660859777065f,
    0.0301123007139661f,   0.0300585346161174f,   0.0300047876631105f,
    0.0299510598338510f,   0.0298973511072006f,   0.0298436614619771f,
    0.0297899908769537f,   0.0297363393308599f,   0.0296827068023808f,
    0.0296290932701575f,   0.0295754987127871f,   0.0295219231088224f,
    0.0294683664367722f,   0.0294148286751011f,   0.0293613098022296f,
    0.0293078097965342f,   0.0292543286363469f,   0.0292008662999562f,
    0.0291474227656060f,   0.0290939980114962f,   0.0290405920157826f,
    0.0289872047565770f,   0.0289338362119469f,   0.0288804863599157f,
    0.0288271551784627f,   0.0287738426455233f,   0.0287205487389884f,
    0.0286672734367052f,   0.0286140167164761f,   0.0285607785560604f,
    0.0285075589331723f,   0.0284543578254825f,   0.0284011752106171f,
    0.0283480110661588f,   0.0282948653696453f,   0.0282417380985708f,
    0.0281886292303851f,   0.0281355387424940f,   0.0280824666122592f,
    0.0280294128169981f,   0.0279763773339841f,   0.0279233601404466f,
    0.0278703612135707f,   0.0278173805304972f,   0.0277644180683234f,
    0.0277114738041018f,   0.0276585477148412f,   0.0276056397775058f,
    0.0275527499690165f,   0.0274998782662493f,   0.0274470246460364f,
    0.0273941890851657f,   0.0273413715603814f,   0.0272885720483831f,
    0.0272357905258265f,   0.0271830269693230f,   0.0271302813554403f,
    0.0270775536607015f,   0.0270248438615858f,   0.0269721519345281f,
    0.0269194778559194f,   0.0268668216021066f,   0.0268141831493923f,
    0.0267615624740348f,   0.0267089595522488f,   0.0266563743602045f,
    0.0266038068740280f,   0.0265512570698012f,   0.0264987249235622f,
    0.0264462104113047f,   0.0263937135089783f,   0.0263412341924885f,
    0.0262887724376968f,   0.0262363282204203f,   0.0261839015164322f,
    0.0261314923014614f,   0.0260791005511929f,   0.0260267262412675f,
    0.0259743693472816f,   0.0259220298447877f,   0.0258697077092944f,
    0.0258174029162656f,   0.0257651154411217f,   0.0257128452592383f,
    0.0256605923459475f,   0.0256083566765370f,   0.0255561382262503f,
    0.0255039369702867f,   0.0254517528838017f,   0.0253995859419064f,
    0.0253474361196679f,   0.0252953033921090f,   0.0252431877342086f,
    0.0251910891209013f,   0.0251390075270776f,   0.0250869429275840f,
    0.0250348952972226f,   0.0249828646107516f,   0.0249308508428848f,
    0.0248788539682924f,   0.0248268739615999f,   0.0247749107973889f,
    0.0247229644501969f,   0.0246710348945172f,   0.0246191221047991f,
    0.0245672260554474f,   0.0245153467208232f,   0.0244634840752433f,
    0.0244116380929803f,   0.0243598087482627f,   0.0243079960152748f,
    0.0242561998681570f,   0.0242044202810054f,   0.0241526572278720f,
    0.0241009106827644f,   0.0240491806196466f,   0.0239974670124380f,
    0.0239457698350141f,   0.0238940890612061f,   0.0238424246648012f,
    0.0237907766195426f,   0.0237391448991289f,   0.0236875294772150f,
    0.0236359303274115f,   0.0235843474232849f,   0.0235327807383576f,
    0.0234812302461075f,   0.0234296959199691f,   0.0233781777333319f,
    0.0233266756595420f,   0.0232751896719008f,   0.0232237197436660f,
    0.0231722658480509f,   0.0231208279582248f,   0.0230694060473126f,
    0.0230180000883954f,   0.0229666100545100f,   0.0229152359186491f,
    0.0228638776537611f,   0.0228125352327506f,   0.0227612086284777f,
    0.0227098978137585f,   0.0226586027613650f,   0.0226073234440251f,
    0.0225560598344226f,   0.0225048119051967f,   0.0224535796289430f,
    0.0224023629782129f,   0.0223511619255133f,   0.0222999764433073f,
    0.0222488065040137f,   0.0221976520800073f,   0.0221465131436186f,
    0.0220953896671340f,   0.0220442816227956f,   0.0219931889828018f,
    0.0219421117193066f,   0.0218910498044196f,   0.0218400032102066f,
    0.0217889719086893f,   0.0217379558718450f,   0.0216869550716069f,
    0.0216359694798642f,   0.0215849990684619f,   0.0215340438092009f,
    0.0214831036738377f,   0.0214321786340850f,   0.0213812686616112f,
    0.0213303737280405f,   0.0212794938049529f,   0.0212286288638847f,
    0.0255395068839051f,   0.0254886718213066f,   0.0254378516550702f,
    0.0253870463565555f,   0.0253362558970777f,   0.0252854802479078f,
    0.0252347193802728f,   0.0251839732653554f,   0.0251332418742945f,
    0.0250825251781844f,   0.0250318231480754f,   0.0249811357549739f,
    0.0249304629698419f,   0.0248798047635972f,   0.0248291611071137f,
    0.0247785319712210f,   0.0247279173267045f,   0.0246773171443056f,
    0.0246267313947214f,   0.0245761600486050f,   0.0245256030765652f,
    0.0244750604491668f,   0.0244245321369303f,   0.0243740181103322f,
    0.0243235183398047f,   0.0242730327957360f,   0.0242225614484700f,
    0.0241721042683066f,   0.0241216612255015f,   0.0240712322902661f,
    0.0240208174327678f,   0.0239704166231300f,   0.0239200298314316f,
    0.0238696570277076f,   0.0238192981819486f,   0.0237689532641015f,
    0.0237186222440686f,   0.0236683050917082f,   0.0236180017768345f,
    0.0235677122692175f,   0.0235174365385832f,   0.0234671745546130f,
    0.0234169262869447f,   0.0233666917051716f,   0.0233164707788430f,
    0.0232662634774639f,   0.0232160697704954f,   0.0231658896273542f,
    0.0231157230174128f,   0.0230655699099999f,   0.0230154302743997f,
    0.0229653040798524f,   0.0229151912955540f,   0.0228650918906563f,
    0.0228150058342672f,   0.0227649330954500f,   0.0227148736432244f,
    0.0226648274465652f,   0.0226147944744040f,   0.0225647746956273f,
    0.0225147680790781f,   0.0224647745935549f,   0.0224147942078123f,
    0.0223648268905605f,   0.0223148726104657f,   0.0222649313361497f,
    0.0222150030361906f,   0.0221650876791220f,   0.0221151852334334f,
    0.0220652956675701f,   0.0220154189499333f,   0.0219655550488802f,
    0.0219157039327235f,   0.0218658655697320f,   0.0218160399281303f,
    0.0217662269760989f,   0.0217164266817738f,   0.0216666390132473f,
    0.0216168639385673f,   0.0215671014257376f,   0.0215173514427178f,
    0.0214676139574232f,   0.0214178889377253f,   0.0213681763514512f,
    0.0213184761663838f,   0.0212687883502620f,   0.0212191128707804f,
    0.0211694496955896f,   0.0211197987922958f,   0.0210701601284612f,
    0.0210205336716039f,   0.0209709193891977f,   0.0209213172486724f,
    0.0208717272174133f,   0.0208221492627619f,   0.0207725833520154f,
    0.0207230294524270f,   0.0206734875312053f,   0.0206239575555151f,
    0.0205744394924771f,   0.0205249333091677f,   0.0204754389726188f,
    0.0204259564498189f,   0.0203764857077117f,   0.0203270267131969f,
    0.0202775794331301f,   0.0202281438343228f,   0.0201787198835422f,
    0.0201293075475113f,   0.0200799067929092f,   0.0200305175863706f,
    0.0199811398944860f,   0.0199317736838018f,   0.0198824189208204f,
    0.0198330755720000f,   0.0197837436037543f,   0.0197344229824530f,
    0.0196851136744220f,   0.0196358156459426f,   0.0195865288632520f,
    0.0195372532925434f,   0.0194879888999657f,   0.0194387356516237f,
    0.0193894935135779f,   0.0193402624518448f,   0.0192910424323968f,
    0.0192418334211618f,   0.0191926353840239f,   0.0191434482868227f,
    0.0190942720953539f,   0.0190451067753690f,   0.0189959522925752f,
    0.0189468086126355f,   0.0188889190431616f,   0.0188228895446890f,
    0.0187568748920607f,   0.0186908750819390f,   0.0186248901109865f,
    0.0185589199758671f,   0.0184929646732450f,   0.0184270241997851f,
    0.0183610985521537f,   0.0182951877270173f,   0.0182292917210430f,
    0.0181634105308991f,   0.0180975441532544f,   0.0180316925847787f,
    0.0179658558221421f,   0.0179000338620157f,   0.0178342267010716f,
    0.0177684343359822f,   0.0177026567634210f,   0.0176368939800620f,
    0.0175711459825800f,   0.0175054127676507f,   0.0174396943319504f,
    0.0217357186797338f,   0.0216700297925235f,   0.0216043556745757f,
    0.0215386963225698f,   0.0214730517331858f,   0.0214074219031045f,
    0.0213418068290075f,   0.0212762065075772f,   0.0212106209354964f,
    0.0211450501094493f,   0.0210794940261201f,   0.0210139526821943f,
    0.0209484260743579f,   0.0208829141992977f,   0.0208174170537012f,
    0.0207519346342566f,   0.0206864669376530f,   0.0206210139605803f,
    0.0205555756997288f,   0.0204901521517898f,   0.0204247433134555f,
    0.0203593491814184f,   0.0202939697523721f,   0.0202286050230107f,
    0.0201632549900294f,   0.0200979196501238f,   0.0200325989999903f,
    0.0199672930363261f,   0.0199020017558292f,   0.0198367251551984f,
    0.0197714632311330f,   0.0197062159803331f,   0.0196409833994998f,
    0.0195757654853346f,   0.0195105622345400f,   0.0194453736438190f,
    0.0193801997098757f,   0.0193150404294146f,   0.0192498957991409f,
    0.0191847658157610f,   0.0191196504759815f,   0.0190545497765102f,
    0.0189894637140552f,   0.0189243922853257f,   0.0188593354870315f,
    0.0187942933158832f,   0.0187292657685919f,   0.0186642528418697f,
    0.0185992545324294f,   0.0185342708369846f,   0.0184693017522493f,
    0.0184043472749386f,   0.0183394074017683f,   0.0182744821294548f,
    0.0182095714547152f,   0.0181446753742675f,   0.0180797938848304f,
    0.0180149269831233f,   0.0179500746658664f,   0.0178852369297804f,
    0.0178204137715871f,   0.0177556051880087f,   0.0176908111757684f,
    0.0176260317315900f,   0.0175612668521982f,   0.0174965165343181f,
    0.0174317807746758f,   0.0173670595699981f,   0.0173023529170125f,
    0.0172376608124473f,   0.0171729832530314f,   0.0171083202354946f,
    0.0170436717565673f,   0.0169790378129807f,   0.0169144184014667f,
    0.0168498135187580f,   0.0167852231615880f,   0.0167206473266909f,
    0.0166560860108013f,   0.0165915392106550f,   0.0165270069229884f,
    0.0164624891445385f,   0.0163979858720428f,   0.0163334971022403f,
    0.0162690228318700f,   0.0162045630576720f,   0.0161401177763870f,
    0.0160756869847564f,   0.0160112706795224f,   0.0159468688574280f,
    0.0158824815152168f,   0.0158181086496333f,   0.0157537502574225f,
    0.0156894063353304f,   0.0156250768801034f,   0.0155607618884889f,
    0.0154964613572351f,   0.0154321752830905f,   0.0153679036628048f,
    0.0153036464931283f,   0.0152394037708118f,   0.0151751754926072f,
    0.0151109616552667f,   0.0150467622555436f,   0.0149825772901918f,
    0.0149184067559660f,   0.0148542506496213f,   0.0147901089679141f,
    0.0147259817076010f,   0.0146618688654396f,   0.0145977704381882f,
    0.0145336864226058f,   0.0144696168154521f,   0.0144055616134876f,
    0.0143415208134734f,   0.0142774944121715f,   0.0142134824063446f,
    0.0141494847927560f,   0.0140855015681696f,   0.0140215327293505f,
    0.0139575782730643f,   0.0182553662036547f,   0.0181914405027335f,
    0.0181275291746461f,   0.0180636322161610f,   0.0179997496240473f,
    0.0179358813950749f,   0.0178720275260146f,   0.0178081880136376f,
    0.0177443628547161f,   0.0176805520460228f,   0.0176167555843314f,
    0.0175529734664161f,   0.0174892056890518f,   0.0174254522490144f,
    0.0173617131430802f,   0.0172979883680264f,   0.0172342779206310f,
    0.0171705817976724f,   0.0171068999959302f,   0.0170432325121844f,
    0.0169795793432156f,   0.0169159404858055f,   0.0168523159367363f,
    0.0167887056927909f,   0.0167251097507530f,   0.0166615281074072f,
    0.0165979607595384f,   0.0165344077039326f,   0.0164708689373761f,
    0.0164073444566565f,   0.0163438342585618f,   0.0162803383398807f,
    0.0162168566974025f,   0.0161533893279177f,   0.0160899362282169f,
    0.0160264973950919f,   0.0159630728253350f,   0.0158996625157394f,
    0.0158362664630987f,   0.0157728846642075f,   0.0157095171158610f,
    0.0156461638148553f,   0.0155828247579869f,   0.0155194999420533f,
    0.0154561893638526f,   0.0153928930201836f,   0.0153296109078459f,
    0.0152663430236398f,   0.0152030893643661f,   0.0151398499268268f,
    0.0150766247078242f,   0.0150134137041614f,   0.0149502169126424f,
    0.0148870343300717f,   0.0148238659532546f,   0.0147607117789972f,
    0.0146975718041061f,   0.0146344460253890f,   0.0145713344396539f,
    0.0145082370437097f,   0.0144451538343661f,   0.0143820848084334f,
    0.0143190299627227f,   0.0142559892940457f,   0.0141929627992148f,
    0.0141299504750434f,   0.0140669523183453f,   0.0140039683259351f,
    0.0139409984946283f,   0.0138780428212408f,   0.0138151013025895f,
    0.0137521739354917f,   0.0136892607167660f,   0.0136263616432309f,
    0.0135634767117064f,   0.0135006059190125f,   0.0134377492619707f,
    0.0133749067374024f,   0.0133120783421303f,   0.0132492640729776f,
    0.0131864639267682f,   0.0131236779003268f,   0.0130609059904787f,
    0.0129981481940499f,   0.0129354045078674f,   0.0128726749287585f,
    0.0128099594535516f,   0.0127472580790754f,   0.0126845708021597f,
    0.0126218976196349f,   0.0125592385283320f,   0.0124965935250826f,
    0.0124339626067194f,   0.0123713457700756f,   0.0123087430119851f,
    0.0122461543292824f,   0.0121835797188031f,   0.0121210191773830f,
    0.0120584727018589f,   0.0119959402890683f,   0.0119334219358495f,
    0.0118709176390412f,   0.0118084273954832f,   0.0117459512020156f,
    0.0116834890554797f,   0.0116210409527169f,   0.0115586068905700f,
    0.0114961868658818f,   0.0114337808754965f,   0.0113713889162585f,
    0.0113090109850131f,   0.0112466470786063f,   0.0111842971938848f,
    0.0111219613276961f,   0.0110596394768881f,   0.0109973316383098f,
    0.0152967658163884f,   0.0152344859928189f,   0.0151722201720297f,
    0.0151099683508726f,   0.0150477305262000f,   0.0149855066948649f,
    0.0149232968537211f,   0.0148611009996230f,   0.0147989191294260f,
    0.0147367512399860f,   0.0146745973281595f,   0.0146124573908037f,
    0.0145503314247770f,   0.0144882194269378f,   0.0144261213941456f,
    0.0143640373232607f,   0.0143019672111439f,   0.0142399110546566f,
    0.0141778688506612f,   0.0141158405960207f,   0.0140538262875987f,
    0.0139918259222596f,   0.0139298394968685f,   0.0138678670082912f,
    0.0138059084533941f,   0.0137439638290446f,   0.0136820331321104f,
    0.0136201163594601f,   0.0135582135079633f,   0.0134963245744897f,
    0.0134344495559101f,   0.0133725884490961f,   0.0133107412509195f,
    0.0132489079582535f,   0.0131870885679713f,   0.0131252830769473f,
    0.0130634914820565f,   0.0130017137801745f,   0.0129399499681775f,
    0.0128782000429427f,   0.0128164640013478f,   0.0127547418402713f,
    0.0126930335565922f,   0.0126313391471905f,   0.0125696586089468f,
    0.0125079919387422f,   0.0124463391334587f,   0.0123847001899790f,
    0.0123230751051865f,   0.0122614638759652f,   0.0121998664991998f,
    0.0121382829717759f,   0.0120767132905796f,   0.0120151574524978f,
    0.0119536154544179f,   0.0118920872932283f,   0.0118305729658180f,
    0.0117690724690765f,   0.0117075857998943f,   0.0116461129551625f,
    0.0115846539317728f,   0.0115232087266176f,   0.0114617773365900f,
    0.0114003597585841f,   0.0113389559894942f,   0.0112775660262157f,
    0.0112161898656444f,   0.0111548275046771f,   0.0110934789402111f,
    0.0110321441691445f,   0.0109708231883759f,   0.0109095159948048f,
    0.0108482225853314f,   0.0107869429568564f,   0.0107256771062815f,
    0.0106644250305088f,   0.0106031867264412f,   0.0105419621909824f,
    0.0104807514210367f,   0.0104195544135091f,   0.0103583711653052f,
    0.0102972016733315f,   0.0102360459344952f,   0.0101749039457040f,
    0.0101137757038663f,   0.0100526612058913f,   0.00999156044868904f,
    0.00993047342916997f,  0.00986940014424537f,  0.00980834059082714f,
    0.00974729476582803f,  0.00968626266616141f,  0.00962524428874123f,
    0.00956423963048225f,  0.00950324868829994f,  0.00944227145911042f,
    0.00938130793983050f,  0.00932035812737764f,  0.00925942201867014f,
    0.00919849961062680f,  0.00913759090016730f,  0.00907669588421184f,
    0.00901581455968142f,  0.00895494692349780f,  0.00889409297258320f,
    0.00883325270386071f,  0.00877242611425416f,  0.00871161320068786f,
    0.00865081396008705f,  0.00859002838937745f,  0.00852925648548564f,
    0.00846849824533880f,  0.00840775366586477f,  0.00834702274399218f,
    0.00828630547665032f,  0.00822560186076915f,  0.00816491189327928f,
    0.00810423557111206f,  0.00804357289119956f,  0.00798292385047450f,
    0.00792228844587023f,  0.00786166667432087f,  0.00780105853276131f,
    0.00774046401812689f,  0.00767988312735390f,  0.00761931585737907f,
    0.00755876220514012f,  0.00749822216757512f,  0.0117994237492008f,
    0.0117389109318013f,   0.0116784117198946f,   0.0116179261104218f,
    0.0115574541003245f,   0.0114969956865451f,   0.0114365508660266f,
    0.0113761196357128f,   0.0113157019925481f,   0.0112552979334775f,
    0.0111949074554470f,   0.0111345305554030f,   0.0110741672302926f,
    0.0110138174770638f,   0.0109534812926650f,   0.0108931586740455f,
    0.0108328496181552f,   0.0107725541219447f,   0.0107122721823654f,
    0.0106520037963691f,   0.0105917489609085f,   0.0105315076729371f,
    0.0104712799294088f,   0.0104110657272783f,   0.0103508650635009f,
    0.0102906779350330f,   0.0102305043388311f,   0.0101703442718527f,
    0.0101101977310560f,   0.0100500647133998f,   0.00998994521584351f,
    0.00992983923534746f,  0.00986974676887237f,  0.00980966781338000f,
    0.00974960236583244f,  0.00968955042319258f,  0.00962951198242412f,
    0.00956948704049132f,  0.00950947559435911f,  0.00944947764099319f,
    0.00938949317735982f,  0.00932952220042610f,  0.00926956470715973f,
    0.00920962069452902f,  0.00914969015950301f,  0.00908977309905157f,
    0.00902986951014501f,  0.00896997938975452f,  0.00891010273485171f,
    0.00885023954240927f,  0.00879038980940028f,  0.00873055353279850f,
    0.00867073070957841f,  0.00861092133671532f,  0.00855112541118508f,
    0.00849134292996412f,  0.00843157389002974f,  0.00837181828835992f,
    0.00831207612193319f,  0.00825234738772879f,  0.00819263208272664f,
    0.00813293020390743f,  0.00807324174825247f,  0.00801356671274367f,
    0.00795390509436372f,  0.00789425689009604f,  0.00783462209692454f,
    0.00777500071183396f,  0.00771539273180966f,  0.00765579815383771f,
    0.00759621697490487f,  0.00753664919199842f,  0.00747709480210662f,
    0.00741755380221809f,  0.00735802618932235f,  0.00729851196040943f,
    0.00723901111247016f,  0.00717952364249613f,  0.00712004954747930f,
    0.00706058882441252f,  0.00700114147028941f,  0.00694170748210410f,
    0.00688228685685138f,  0.00682287959152672f,  0.00676348568312646f,
    0.00670410512864744f,  0.00664473792508719f,  0.00658538406944387f,
    0.00652604355871650f,  0.00646671638990454f,  0.00640740256000838f,
    0.00634810206602876f,  0.00628881490496747f,  0.00622954107382667f,
    0.00617028056960933f,  0.00611103338931901f,  0.00605179952996010f,
    0.00599257898853756f,  0.00593337176205699f,  0.00587417784752470f,
    0.00581499724194773f,  0.00575582994233370f,  0.00569667594569095f,
    0.00563753524902838f,  0.00557840784935593f,  0.00551929374368371f,
    0.00546019292902289f,  0.00540110540238498f,  0.00534203116078258f,
    0.00528297020122859f,  0.00522392252073672f,  0.00516488811632138f,
    0.00510586698499765f,  0.00504685912378122f,  0.00498786452968852f,
    0.00492888319973650f,  0.00486991513094304f,  0.00481096032032652f,
    0.00475201876490594f,  0.00469309046170108f,  0.00463417540773242f,
    0.00457527360002097f,  0.00451638503558854f,  0.00445750971145748f,
    0.00439864762465103f,  0.00433979877219282f,  0.00428096315110732f,
    0.00422214075841959f,  0.00416333159115556f,  0.00410453564634156f,
    0.00404575292100462f,  0.00398698341217263f,  0.00392822711687407f,
    0.00386948403213799f,  0.00381075415499416f,  0.00375203748247305f,
    0.00369333401160576f,  0.00363464373942418f,  0.00357596666296056f,
    0.00351730277924822f,  0.00345865208532087f,  0.00340001457821293f,
    0.00334139025495950f,  0.00328277911259650f,  0.00322418114816031f,
    0.00316559635868802f,  0.00310702474121738f,  0.00304846629278699f,
    0.00298992101043588f,  0.00293138889120381f,  0.00287286993213121f,
    0.00281436413025932f,  0.00275587148262979f,  0.00269739198628516f,
    0.00263892563826845f,  0.00258047243562354f,  0.00252203237539478f,
    0.00246360545462732f,  0.00240519167036685f,  0.00234679101965996f,
    0.00228840349955362f,  0.00223002910709563f,  0.00217166783933437f,
    0.00211331969331902f,  0.00205498466609932f,  0.00199666275472560f,
    0.00193835395624892f,  0.00188005826772120f,  0.00182177568619468f,
    0.00176350620872251f,  0.00170524983235837f,  0.00164700655415667f,
    0.00158877637117250f,  0.00153055928046153f,  0.00147235527908007f,
    0.00141416436408531f,  0.00135598653253494f,  0.00129782178148719f,
    0.00123967010800113f,  0.00118153150913658f,  0.00112340598195376f,
    0.00106529352351364f,  0.00100719413087796f,  0.000949107801109128f,
    0.000891034531269985f, 0.000832974318424273f, 0.000774927159636230f,
    0.000716893051970924f, 0.000658871992493926f, 0.000600863978271471f,
    0.000542869006370628f, 0.000484887073858964f, 0.000426918177804714f,
    0.00473069032285445f,  0.00467274749092250f,  0.00461481768665678f,
    0.00455690090712813f,  0.00449899714940810f,  0.00444110641056905f,
    0.00438322868768370f,  0.00432536397782568f,  0.00426751227806910f,
    0.00420967358548896f,  0.00415184789716061f,  0.00409403521016033f,
    0.00403623552156485f,  0.00397844882845178f,  0.00392067512789918f,
    0.00386291441698589f,  0.00380516669279124f,  0.00374743195239552f,
    0.00368971019287939f,  0.00363200141132430f,  0.00357430560481226f,
    0.00351662277042614f,  0.00345895290524928f,  0.00340129600636568f,
    0.00334365207086007f,  0.00328602109581783f,  0.00322840307832495f,
    0.00317079801546810f,  0.00311320590433462f,  0.00305562674201249f,
    0.00299806052559032f,  0.00294050725215744f,  0.00288296691880374f,
    0.00282543952261988f,  0.00276792506069706f,  0.00271042353012718f,
    0.00265293492800284f,  0.00259545925141724f,  0.00253799649746422f,
    0.00248054666323838f,  0.00242310974583471f,  0.00236568574234924f,
    0.00230827464987843f,  0.00225087646551925f,  0.00219349118636958f,
    0.00213611880952797f,  0.00207875933209334f,  0.00202141275116546f,
    0.00196407906384477f,  0.00190675826723236f,  0.00184945035842982f,
    0.00179215533453958f,  0.00173487319266458f,  0.00167760392990851f,
    0.00162034754337570f,  0.00156310403017101f,  0.00150587338740016f,
    0.00144865561216939f,  0.00139145070158553f,  0.00133425865275616f,
    0.00127707946278949f,  0.00121991312879444f,  0.00116275964788048f,
    0.00110561901715772f,  0.00104849123373701f,  0.000991376294729840f,
    0.000934274197248231f, 0.000877184938404996f, 0.000820108515313556f,
    0.000763044925087941f, 0.000705994164842849f, 0.000648956231693587f,
    0.000591931122756240f, 0.000534918835147447f, 0.000477919365984458f,
    0.000420932712385191f, 0.000363958871468284f, 0.000306997840353040f,
    0.000250049616159320f, 0.000193114196007482f, 0.000136191577018996f,
    7.92817563154968e-05f, 2.23847310195091e-05f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f,
    0.000000000000000000f, 0.000000000000000000f, 0.000000000000000000f};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_SPECTRAL_REVERB_CONSTANTS_AND_TABLES_H_
