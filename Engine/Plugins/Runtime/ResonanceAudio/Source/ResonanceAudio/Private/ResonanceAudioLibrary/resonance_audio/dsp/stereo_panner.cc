/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/stereo_panner.h"

#include <cmath>

#include "base/constants_and_types.h"

namespace vraudio {

const float kStereoLeftRadians = kRadiansFromDegrees * kStereoLeftDegrees;
const float kStereoRightRadians = kRadiansFromDegrees * kStereoRightDegrees;

void CalculateStereoPanGains(const SphericalAngle& source_direction,
                             std::vector<float>* stereo_gains) {
  // Note this applies the same panning law as was applied by the ambisonic
  // equivalent panner to ensure consistency.
  DCHECK(stereo_gains);
  stereo_gains->resize(kNumStereoChannels);

  const float cos_direction_elevation = std::cos(source_direction.elevation());

  (*stereo_gains)[0] =
      0.5f * (1.0f + std::cos(kStereoLeftRadians - source_direction.azimuth()) *
                         cos_direction_elevation);
  (*stereo_gains)[1] =
      0.5f *
      (1.0f + std::cos(kStereoRightRadians - source_direction.azimuth()) *
                  cos_direction_elevation);
}

}  // namespace vraudio
