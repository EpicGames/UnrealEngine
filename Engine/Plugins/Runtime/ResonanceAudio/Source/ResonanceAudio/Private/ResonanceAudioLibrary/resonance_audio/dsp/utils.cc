/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "dsp/utils.h"

#include <algorithm>
#include <cmath>
#include <limits>
#include <random>
#include <vector>

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"

#include "dsp/biquad_filter.h"
#include "dsp/filter_coefficient_generators.h"

namespace {

// The mean and standard deviation of the normal distribution for bandlimited
// Gaussian noise.
const float kMean = 0.0f;
const float kStandardDeviation = 1.0f;

// Maximum group delay in seconds for each filter. In order to avoid audible
// distortion, the maximum phase shift of a re-combined stereo sequence should
// not exceed 5ms at high frequencies. That is why, maximum phase shift of
// each filter is set to 1/2 of that value.
const float kMaxGroupDelaySeconds = 0.0025f;

// Phase modulation depth, chosen so that for a given max group delay filters
// provide the lowest cross-correlation coefficient.
const float kPhaseModulationDepth = 1.18f;

// Constants used in the generation of uniform random number distributions.
// https://en.wikipedia.org/wiki/Linear_congruential_generator
const uint64 kMultiplier = 1664525L;
const uint64 kIncrement = 1013904223L;
const float kInt32ToFloat =
    1.0f / static_cast<float>(std::numeric_limits<uint32>::max());

}  // namespace

namespace vraudio {

void GenerateGaussianNoise(float mean, float std_deviation, unsigned seed,
                           AudioBuffer::Channel* noise_channel) {
  DCHECK(noise_channel);
  // First generate uniform noise.
  GenerateUniformNoise(0.0f, 1.0f, seed, noise_channel);
  const size_t length = noise_channel->size();

  // Gaussian distribution with mean and standard deviation in pairs via the
  // box-muller transform
  // https://en.wikipedia.org/wiki/Box%E2%80%93Muller_transform.
  for (size_t i = 0; i < length - 1; i += 2) {
    const float part_one = std::sqrt(-2.0f * std::log((*noise_channel)[i]));
    const float part_two = kTwoPi * (*noise_channel)[i + 1];
    const float z0 = part_one * std::cos(part_two);
    const float z1 = part_one * std::sin(part_two);
    (*noise_channel)[i] = std_deviation * z0 + mean;
    (*noise_channel)[i + 1] = std_deviation * z1 + mean;
  }
  // Handle the odd buffer length case cheaply.
  if (length % 2 > 0) {
    (*noise_channel)[length - 1] = (*noise_channel)[0];
  }
}

void GenerateUniformNoise(float min, float max, unsigned seed,
                          AudioBuffer::Channel* noise_channel) {
  // Simple random generator to avoid the use of std::uniform_real_distribution
  // affected by https://gcc.gnu.org/bugzilla/show_bug.cgi?id=56202
  DCHECK(noise_channel);
  DCHECK_LT(min, max);
  const float scaled_conversion_factor = kInt32ToFloat * (max - min);
  uint32 state = static_cast<uint32>(seed);
  for (float& sample : *noise_channel) {
    state = static_cast<uint32>(state * kMultiplier + kIncrement);
    sample = min + static_cast<float>(state) * scaled_conversion_factor;
  }
}

void GenerateBandLimitedGaussianNoise(float center_frequency, int sampling_rate,
                                      unsigned seed,
                                      AudioBuffer* noise_buffer) {


  DCHECK(noise_buffer);
  DCHECK_GT(sampling_rate, 0);
  DCHECK_LT(center_frequency, static_cast<float>(sampling_rate) / 2.0f);
  const size_t num_frames = noise_buffer->num_frames();

  BiquadCoefficients bandpass_coefficients = ComputeBandPassBiquadCoefficients(
      sampling_rate, center_frequency, /*bandwidth=*/1);
  BiquadFilter bandpass_filter(bandpass_coefficients, num_frames);

  for (auto& channel : *noise_buffer) {
    GenerateGaussianNoise(kMean, kStandardDeviation, seed, &channel);
    bandpass_filter.Filter(channel, &channel);
    bandpass_filter.Clear();
  }
}

std::unique_ptr<AudioBuffer> GenerateDecorrelationFilters(int sampling_rate) {

  const int kMaxGroupDelaySamples = static_cast<int>(
      roundf(kMaxGroupDelaySeconds * static_cast<float>(sampling_rate)));

  // Filter coefficients according to:
  // [1]  F. Zotter, M. Frank, "Efficient Phantom Source Widening", Archives of
  //      Acoustics, Vol. 38, No. 1, pp. 27–37 (2013).
  const float g0 = 1.0f - 0.25f * IntegerPow(kPhaseModulationDepth, 2);
  const float g1 = 0.5f * kPhaseModulationDepth -
                   0.0625f * IntegerPow(kPhaseModulationDepth, 3);
  const float g2 = 0.1250f * IntegerPow(kPhaseModulationDepth, 2);
  std::vector<float> filter1_coefficients{g2, g1, g0, -g1, g2};
  std::vector<float> filter2_coefficients{g2, -g1, g0, g1, g2};

  const size_t filter_length =
      filter1_coefficients.size() * kMaxGroupDelaySamples;
  std::unique_ptr<AudioBuffer> decorrelation_filters(
      new AudioBuffer(kNumStereoChannels, filter_length));
  decorrelation_filters->Clear();

  for (size_t coefficient = 0; coefficient < filter1_coefficients.size();
       ++coefficient) {
    (*decorrelation_filters)[0][coefficient * kMaxGroupDelaySamples] =
        filter1_coefficients[coefficient];
    (*decorrelation_filters)[1][coefficient * kMaxGroupDelaySamples] =
        filter2_coefficients[coefficient];
  }

  return decorrelation_filters;
}

size_t GetNumReverbOctaveBands(int sampling_rate) {
  DCHECK_GT(sampling_rate, 0);

  const float max_band =
      log2f(0.5f * static_cast<float>(sampling_rate) / kLowestOctaveBandHz);
  return std::min(kNumReverbOctaveBands, static_cast<size_t>(roundf(max_band)));
}

size_t GetNumSamplesFromMilliseconds(float milliseconds, int sampling_rate) {
  DCHECK_GE(milliseconds, 0.0f);
  DCHECK_GT(sampling_rate, 0);
  return static_cast<size_t>(milliseconds * kSecondsFromMilliseconds *
                             static_cast<float>(sampling_rate));
}

size_t CeilToMultipleOfFramesPerBuffer(size_t size, size_t frames_per_buffer) {
  DCHECK_NE(frames_per_buffer, 0U);
  const size_t remainder = size % frames_per_buffer;
  return remainder == 0 ? std::max(size, frames_per_buffer)
                        : size + frames_per_buffer - remainder;
}

void GenerateHannWindow(bool full_window, size_t window_length,
                        AudioBuffer::Channel* buffer) {

  DCHECK(buffer);
  DCHECK_LE(window_length, buffer->size());
  const float full_window_scaling_factor =
      kTwoPi / (static_cast<float>(window_length) - 1.0f);
  const float half_window_scaling_factor =
      kTwoPi / (2.0f * static_cast<float>(window_length) - 1.0f);
  const float scaling_factor =
      (full_window) ? full_window_scaling_factor : half_window_scaling_factor;
  for (size_t i = 0; i < window_length; ++i) {
    (*buffer)[i] =
        0.5f * (1.0f - std::cos(scaling_factor * static_cast<float>(i)));
  }
}

}  // namespace vraudio
