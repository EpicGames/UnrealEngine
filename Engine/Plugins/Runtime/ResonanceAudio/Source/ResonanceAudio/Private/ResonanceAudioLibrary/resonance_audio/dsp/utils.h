/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_DSP_UTILS_H_
#define RESONANCE_AUDIO_DSP_UTILS_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Generates a gaussian white noise signal.
//
// @param mean The mean distribution parameter.
// @param std_deviation The standard deviation distribution parameter.
// @param seed A seed for the random generator.
// @param noise_channel Buffer channel in which to store the noise.
void GenerateGaussianNoise(float mean, float std_deviation, unsigned seed,
                           AudioBuffer::Channel* noise_channel);

// Generates  a gaussian white noise signal.
//
// @param min The lowest value in the distribution.
// @param max The highest value in te distribution, must be > min.
// @param seed A seed for the random generator.
// @param noise_channel Buffer channel in which to store the noise.
void GenerateUniformNoise(float min, float max, unsigned seed,
                          AudioBuffer::Channel* noise_channel);

// Generates a band limited gaussian white noise signal, one octave band wide.
//
// @param center_frequency Center frequency of the given octave band in Hz.
// @param sampling_rate System sampling rate in Hz.
// @param seed A seed for the random generator.
// @param noise_buffer Buffer in which to store the band limited noise.
void GenerateBandLimitedGaussianNoise(float center_frequency, int sampling_rate,
                                      unsigned seed, AudioBuffer* noise_buffer);

// Genarates a pair of decorrelation filters (for use in low quality/high
// effiency mode reverb).
//
// @param sampling_rate System sampling rate in Hz.
// @return Buffer containing the stereo filters.
std::unique_ptr<AudioBuffer> GenerateDecorrelationFilters(int sampling_rate);

// Returns the number of octave bands necessary for the given |sampling_rate|.
//
// @param sampling_rate Sampling rate in Hertz.
// @return Number of reverb octave bands.
size_t GetNumReverbOctaveBands(int sampling_rate);

// Converts the given |milliseconds| to number of samples with the given
// |sampling_rate|. This method should *not* be used when more precise
// (double-precission) value is desired.
//
// @param milliseconds Milliseconds in single-precission floating point.
// @param sampling_rate Sampling rate in Hertz.
// @return Number of samples.
size_t GetNumSamplesFromMilliseconds(float milliseconds, int sampling_rate);

// Ceils the given |size| to the next multiple of given |frames_per_buffer|.
//
// @param size Input size in frames.
// @param frames_per_buffer Frames per buffer.
// @return Ceiled size in frames.
size_t CeilToMultipleOfFramesPerBuffer(size_t size, size_t frames_per_buffer);

// Generates a Hann window (used for smooth onset and tapering of the generated
// reverb response tails).
//
// @param full_window True to generate a full window, false to generate a half.
// @param window_length Length of the window to be generated. Must be less than
//     or equal to the number of frames in the |buffer|.
// @param buffer AudioBuffer::Channel to which the window is written, the number
//     of frames will be the length in samples of the generated Hann window.
void GenerateHannWindow(bool full_window, size_t window_length,
                        AudioBuffer::Channel* buffer);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_DSP_UTILS_H_
