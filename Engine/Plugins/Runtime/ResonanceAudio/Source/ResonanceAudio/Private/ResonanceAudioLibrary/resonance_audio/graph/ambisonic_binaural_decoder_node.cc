/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/ambisonic_binaural_decoder_node.h"

#include "ambisonics/stereo_from_soundfield_converter.h"
#include "ambisonics/utils.h"
#include "base/constants_and_types.h"

#include "dsp/sh_hrir_creator.h"

namespace vraudio {

AmbisonicBinauralDecoderNode::AmbisonicBinauralDecoderNode(
    const SystemSettings& system_settings, int ambisonic_order,
    const std::string& sh_hrir_filename, FftManager* fft_manager,
    Resampler* resampler)
    : system_settings_(system_settings),
      num_ambisonic_channels_(GetNumPeriphonicComponents(ambisonic_order)),
      is_stereo_speaker_mode_(system_settings_.IsStereoSpeakerModeEnabled()),
      num_frames_processed_on_empty_input_(
          system_settings_.GetFramesPerBuffer()),
      stereo_output_buffer_(kNumStereoChannels,
                            system_settings.GetFramesPerBuffer()),
      silence_input_buffer_(num_ambisonic_channels_,
                            system_settings.GetFramesPerBuffer()),
      crossfader_(system_settings_.GetFramesPerBuffer()),
      crossfaded_output_buffer_(kNumStereoChannels,
                                system_settings.GetFramesPerBuffer()),
      temp_crossfade_buffer_(kNumStereoChannels,
                             system_settings.GetFramesPerBuffer()) {
  silence_input_buffer_.Clear();
  EnableProcessOnEmptyInput(true);
  std::unique_ptr<AudioBuffer> sh_hrirs = CreateShHrirsFromAssets(
      sh_hrir_filename, system_settings_.GetSampleRateHz(), resampler);
  CHECK_EQ(sh_hrirs->num_channels(), num_ambisonic_channels_);
  ambisonic_binaural_decoder_.reset(new AmbisonicBinauralDecoder(
      *sh_hrirs, system_settings_.GetFramesPerBuffer(), fft_manager));
}

AmbisonicBinauralDecoderNode::~AmbisonicBinauralDecoderNode() {}

const AudioBuffer* AmbisonicBinauralDecoderNode::AudioProcess(
    const NodeInput& input) {


  const bool was_stereo_speaker_mode_enabled = is_stereo_speaker_mode_;
  is_stereo_speaker_mode_ = system_settings_.IsStereoSpeakerModeEnabled();

  const size_t num_frames = system_settings_.GetFramesPerBuffer();
  const AudioBuffer* input_buffer = input.GetSingleInput();
  if (input_buffer == nullptr) {
    if (num_frames_processed_on_empty_input_ < num_frames &&
        !was_stereo_speaker_mode_enabled) {
      // If we have no input, generate a silent input buffer until the node
      // states are cleared.
      num_frames_processed_on_empty_input_ += num_frames;
      ambisonic_binaural_decoder_->Process(silence_input_buffer_,
                                           &stereo_output_buffer_);
      return &stereo_output_buffer_;
    } else {
      // Skip processing entirely when the states are fully cleared.
      return nullptr;
    }
  }

  num_frames_processed_on_empty_input_ = 0;

  DCHECK_EQ(input_buffer->num_channels(), num_ambisonic_channels_);
  DCHECK_EQ(input_buffer->num_frames(), num_frames);

  // If stereo speaker mode is enabled, perform M-S stereo decode. Otherwise,
  // perform binaural decode.
  if (is_stereo_speaker_mode_) {
    StereoFromSoundfield(*input_buffer, &stereo_output_buffer_);
  } else {
    ambisonic_binaural_decoder_->Process(*input_buffer, &stereo_output_buffer_);
  }

  if (is_stereo_speaker_mode_ != was_stereo_speaker_mode_enabled) {
    // Apply linear crossfade between binaural decode and stereo decode outputs.
    if (was_stereo_speaker_mode_enabled) {
      StereoFromSoundfield(*input_buffer, &temp_crossfade_buffer_);
    } else {
      ambisonic_binaural_decoder_->Process(*input_buffer,
                                           &temp_crossfade_buffer_);
    }
    crossfader_.ApplyLinearCrossfade(stereo_output_buffer_,
                                     temp_crossfade_buffer_,
                                     &crossfaded_output_buffer_);
    return &crossfaded_output_buffer_;
  }

  // Return the rendered output directly.
  return &stereo_output_buffer_;
}

}  // namespace vraudio
