/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_AMBISONIC_BINAURAL_DECODER_NODE_H_
#define RESONANCE_AUDIO_GRAPH_AMBISONIC_BINAURAL_DECODER_NODE_H_

#include <memory>
#include <string>

#include "ambisonics/ambisonic_binaural_decoder.h"
#include "base/audio_buffer.h"
#include "dsp/fft_manager.h"
#include "dsp/resampler.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"
#include "utils/buffer_crossfader.h"

namespace vraudio {

// Node that takes an ambisonic soundfield as input and renders a binaural
// stereo buffer as output.
class AmbisonicBinauralDecoderNode : public ProcessingNode {
 public:
  // Initializes AmbisonicBinauralDecoderNode class.
  //
  // @param system_settings Global system configuration.
  // @param ambisonic_order Ambisonic order.
  // @param sh_hrir_filename Filename to load the HRIR data from.
  // @param fft_manager Pointer to a manager to perform FFT transformations.
  // @resampler Pointer to a resampler used to convert HRIRs to the system rate.
  AmbisonicBinauralDecoderNode(const SystemSettings& system_settings,
                               int ambisonic_order,
                               const std::string& sh_hrir_filename,
                               FftManager* fft_manager, Resampler* resampler);

  ~AmbisonicBinauralDecoderNode() override;

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  const SystemSettings& system_settings_;

  // Number of Ambisonic channels.
  const size_t num_ambisonic_channels_;

  // Denotes if the stereo speaker mode is enabled.
  bool is_stereo_speaker_mode_;

  // Ambisonic decoder used to render binaural output.
  std::unique_ptr<AmbisonicBinauralDecoder> ambisonic_binaural_decoder_;

  size_t num_frames_processed_on_empty_input_;

  // Stereo output buffer.
  AudioBuffer stereo_output_buffer_;

  // Silence mono buffer to render reverb tails.
  AudioBuffer silence_input_buffer_;

  // Buffer crossfader to apply linear crossfade when the stereo speaker mode is
  // changed.
  BufferCrossfader crossfader_;

  // Stereo output buffer to store the crossfaded decode output when necessary.
  AudioBuffer crossfaded_output_buffer_;

  // Temporary crossfade buffer to store the intermediate stereo output.
  AudioBuffer temp_crossfade_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_AMBISONIC_BINAURAL_DECODER_NODE_H_
