/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/ambisonic_mixing_encoder_node.h"

#include "ambisonics/utils.h"
#include "base/constants_and_types.h"
#include "base/logging.h"


namespace vraudio {

AmbisonicMixingEncoderNode::AmbisonicMixingEncoderNode(
    const SystemSettings& system_settings,
    const AmbisonicLookupTable& lookup_table, int ambisonic_order)
    : system_settings_(system_settings),
      lookup_table_(lookup_table),
      ambisonic_order_(ambisonic_order),
      gain_mixer_(GetNumPeriphonicComponents(ambisonic_order_),
                  system_settings_.GetFramesPerBuffer()),
      coefficients_(GetNumPeriphonicComponents(ambisonic_order_)) {}

const AudioBuffer* AmbisonicMixingEncoderNode::AudioProcess(
    const NodeInput& input) {


  const WorldPosition& listener_position = system_settings_.GetHeadPosition();
  const WorldRotation& listener_rotation = system_settings_.GetHeadRotation();

  gain_mixer_.Reset();
  for (auto& input_buffer : input.GetInputBuffers()) {
    const int source_id = input_buffer->source_id();
    const auto source_parameters =
        system_settings_.GetSourceParameters(source_id);
    DCHECK_NE(source_id, kInvalidSourceId);
    DCHECK_EQ(input_buffer->num_channels(), 1U);

    // Compute the relative source direction in spherical angles.
    const ObjectTransform& source_transform =
        source_parameters->object_transform;
    WorldPosition relative_direction;
    GetRelativeDirection(listener_position, listener_rotation,
                         source_transform.position, &relative_direction);
    const SphericalAngle source_direction =
        SphericalAngle::FromWorldPosition(relative_direction);

    lookup_table_.GetEncodingCoeffs(ambisonic_order_, source_direction,
                                    source_parameters->spread_deg,
                                    &coefficients_);

    gain_mixer_.AddInputChannel((*input_buffer)[0], source_id, coefficients_);
  }
  return gain_mixer_.GetOutput();
}

}  // namespace vraudio
