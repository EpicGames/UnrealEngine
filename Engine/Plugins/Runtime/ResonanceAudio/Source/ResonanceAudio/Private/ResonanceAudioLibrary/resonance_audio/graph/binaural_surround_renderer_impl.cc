/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/binaural_surround_renderer_impl.h"

#include <algorithm>
#include <functional>

#include "base/misc_math.h"
#include "base/simd_utils.h"
#include "base/spherical_angle.h"
#include "graph/resonance_audio_api_impl.h"
#include "platforms/common/room_effects_utils.h"
#include "platforms/common/room_properties.h"
#include "utils/planar_interleaved_conversion.h"

namespace vraudio {

namespace {

// Maximum number of audio buffers in buffer queue.
const size_t kNumMaxBuffers = 64;

// Output gain, to avoid clipping of individual virtual speaker channels.
const float kGain = 0.5f;

}  // namespace

BinauralSurroundRendererImpl::BinauralSurroundRendererImpl(
    size_t frames_per_buffer, int sample_rate_hz)
    :
      resonance_audio_api_(nullptr),
      frames_per_buffer_(frames_per_buffer),
      sample_rate_hz_(sample_rate_hz),
      surround_format_(kInvalid),
      num_input_channels_(0),
      output_buffer_(kNumStereoChannels, frames_per_buffer),
      total_frames_buffered_(0),
      num_zero_padded_frames_(0),
      output_gain_(1.0f) {
}

bool BinauralSurroundRendererImpl::Init(SurroundFormat surround_format) {
  surround_format_ = surround_format;
  num_input_channels_ =
      GetExpectedNumChannelsFromSurroundFormat(surround_format);

  temp_planar_buffer_ptrs_.resize(num_input_channels_);

  input_audio_buffer_queue_.reset(new ThreadsafeFifo<AudioBuffer>(
      kNumMaxBuffers, AudioBuffer(num_input_channels_, frames_per_buffer_)));

  buffer_partitioner_.reset(new BufferPartitioner(
      num_input_channels_, frames_per_buffer_,
      std::bind(&BinauralSurroundRendererImpl::BufferPartitionerCallback, this,
                std::placeholders::_1)));

  buffer_unpartitioner_.reset(new BufferUnpartitioner(
      kNumStereoChannels, frames_per_buffer_,
      std::bind(&BinauralSurroundRendererImpl::ProcessBuffer, this)));

  resonance_audio_api_.reset(CreateResonanceAudioApi(
      kNumStereoChannels, frames_per_buffer_, sample_rate_hz_));

  if (surround_format == kSurroundMono || surround_format == kSurroundStereo ||
      surround_format == kSurroundFiveDotOne ||
      surround_format == kSurroundSevenDotOne) {
    InitializeRoomReverb();
  }
  // Initialize rendering mode.
  switch (surround_format) {
    case kSurroundMono:
      InitializeBinauralMono();
      break;
    case kSurroundStereo:
      InitializeBinauralStereo();
      break;
    case kSurroundFiveDotOne:
      InitializeBinauralSurround5dot1();
      break;
    case kSurroundSevenDotOne:
      InitializeBinauralSurround7dot1();
      break;
    case kFirstOrderAmbisonics:
    case kSecondOrderAmbisonics:
    case kThirdOrderAmbisonics:
      InitializeAmbisonics();
      break;
    case kFirstOrderAmbisonicsWithNonDiegeticStereo:
    case kSecondOrderAmbisonicsWithNonDiegeticStereo:
    case kThirdOrderAmbisonicsWithNonDiegeticStereo:
      InitializeAmbisonicsWithNonDiegeticStereo();
      break;
    default:
      LOG(FATAL) << "Undefined rendering mode";
      return false;
  }
  return true;
}

BinauralSurroundRendererImpl::BinauralSurroundRendererImpl()
    :
      resonance_audio_api_(nullptr),
      frames_per_buffer_(0),
      sample_rate_hz_(0),
	  surround_format_(SurroundFormat::kInvalid),
	  num_input_channels_(0),
      total_frames_buffered_(0),
      num_zero_padded_frames_(0),
      output_gain_(0.0f) {
}

AudioBuffer* BinauralSurroundRendererImpl::BufferPartitionerCallback(
    AudioBuffer* processed_buffer) {
  if (processed_buffer != nullptr) {
    input_audio_buffer_queue_->ReleaseInputObject(processed_buffer);
  }
  DCHECK(!input_audio_buffer_queue_->Full());
  return input_audio_buffer_queue_->AcquireInputObject();
}

void BinauralSurroundRendererImpl::SetStereoSpeakerMode(bool enabled) {
  resonance_audio_api_->SetStereoSpeakerMode(enabled);
}

size_t BinauralSurroundRendererImpl::GetExpectedNumChannelsFromSurroundFormat(
    SurroundFormat surround_format) {
  switch (surround_format) {
    case kSurroundMono:
      return kNumMonoChannels;
    case kSurroundStereo:
      return kNumStereoChannels;
    case kSurroundFiveDotOne:
      return kNumSurroundFiveDotOneChannels;
    case kSurroundSevenDotOne:
      return kNumSurroundSevenDotOneChannels;
    case kFirstOrderAmbisonics:
      return kNumFirstOrderAmbisonicChannels;
    case kSecondOrderAmbisonics:
      return kNumSecondOrderAmbisonicChannels;
    case kThirdOrderAmbisonics:
      return kNumThirdOrderAmbisonicChannels;
    case kFirstOrderAmbisonicsWithNonDiegeticStereo:
      return kNumFirstOrderAmbisonicChannels + kNumStereoChannels;
    case kSecondOrderAmbisonicsWithNonDiegeticStereo:
      return kNumSecondOrderAmbisonicChannels + kNumStereoChannels;
    case kThirdOrderAmbisonicsWithNonDiegeticStereo:
      return kNumThirdOrderAmbisonicChannels + kNumStereoChannels;
    default:
      LOG(FATAL) << "Undefined surround format mode";
      break;
  }

  return 0;
}

void BinauralSurroundRendererImpl::InitializeBinauralMono() {
  source_ids_.resize(kNumMonoChannels);
  // Front (0 degrees):
  source_ids_[0] = CreateSoundObject(0.0f);
  output_gain_ = kGain;
}

void BinauralSurroundRendererImpl::InitializeBinauralStereo() {

  source_ids_.resize(kNumStereoChannels);
  // Front left (30 degrees):
  source_ids_[0] = CreateSoundObject(30.0f);
  // Front right (-30 degrees):
  source_ids_[1] = CreateSoundObject(-30.0f);
  output_gain_ = kGain;
}

void BinauralSurroundRendererImpl::InitializeBinauralSurround5dot1() {
  source_ids_.resize(kNumSurroundFiveDotOneChannels);
  // Left (30 degrees):
  source_ids_[0] = CreateSoundObject(30.0f);
  // Right (-30 degrees):
  source_ids_[1] = CreateSoundObject(-30.0f);
  // Center (0 degrees):
  source_ids_[2] = CreateSoundObject(0.0f);
  // Low frequency effects at front center:
  source_ids_[3] = CreateSoundObject(0.0f);
  // Left surround (110 degrees):
  source_ids_[4] = CreateSoundObject(110.0f);
  // Right surround (-110 degrees):
  source_ids_[5] = CreateSoundObject(-110.0f);
  output_gain_ = kGain;
}

void BinauralSurroundRendererImpl::InitializeBinauralSurround7dot1() {
  source_ids_.resize(kNumSurroundSevenDotOneChannels);
  // Left (30 degrees):
  source_ids_[0] = CreateSoundObject(30.0f);
  // Right (-30 degrees):
  source_ids_[1] = CreateSoundObject(-30.0f);
  // Center (0 degrees):
  source_ids_[2] = CreateSoundObject(0.0f);
  // Low frequency effects at front center:
  source_ids_[3] = CreateSoundObject(0.0f);
  // Left surround 1 (90 degrees):
  source_ids_[4] = CreateSoundObject(90.0f);
  // Right surround 1 (-90 degrees):
  source_ids_[5] = CreateSoundObject(-90.0f);
  // Left surround 2 (150 degrees):
  source_ids_[6] = CreateSoundObject(150.0f);
  // Right surround 2 (-150 degrees):
  source_ids_[7] = CreateSoundObject(-150.0f);
  output_gain_ = kGain;
}

void BinauralSurroundRendererImpl::InitializeAmbisonics() {
  source_ids_.resize(1);
  source_ids_[0] =
      resonance_audio_api_->CreateAmbisonicSource(num_input_channels_);
}

void BinauralSurroundRendererImpl::InitializeAmbisonicsWithNonDiegeticStereo() {
  source_ids_.resize(2);
  CHECK_GT(num_input_channels_, kNumStereoChannels);
  source_ids_[0] = resonance_audio_api_->CreateAmbisonicSource(
      num_input_channels_ - kNumStereoChannels);
  source_ids_[1] = resonance_audio_api_->CreateStereoSource(kNumStereoChannels);
}

SourceId BinauralSurroundRendererImpl::CreateSoundObject(float azimuth_deg) {
  static const float kZeroElevation = 0.0f;
  auto speaker_position =
      vraudio::SphericalAngle::FromDegrees(azimuth_deg, kZeroElevation)
          .GetWorldPositionOnUnitSphere();
  const SourceId source_id = resonance_audio_api_->CreateSoundObjectSource(
      RenderingMode::kBinauralHighQuality);
  resonance_audio_api_->SetSourcePosition(
      source_id, speaker_position[0], speaker_position[1], speaker_position[2]);
  return source_id;
}

void BinauralSurroundRendererImpl::InitializeRoomReverb() {
  // The following settings has been applied based on AESTD1001.1.01-10.
  RoomProperties room_properties;
  room_properties.dimensions[0] = 9.54f;
  room_properties.dimensions[1] = 6.0f;
  room_properties.dimensions[2] = 15.12f;
  room_properties.reverb_brightness = 0.0f;
  room_properties.reflection_scalar = 1.0f;
  // Reduce reverb gain to compensate for virtual speakers gain.
  room_properties.reverb_gain = output_gain_;
  for (size_t i = 0; i < kNumRoomSurfaces; ++i) {
    room_properties.material_names[i] = MaterialName::kUniform;
  }
  resonance_audio_api_->SetReflectionProperties(
      ComputeReflectionProperties(room_properties));
  resonance_audio_api_->SetReverbProperties(
      ComputeReverbProperties(room_properties));
  resonance_audio_api_->EnableRoomEffects(true);
}

size_t BinauralSurroundRendererImpl::GetNumAvailableFramesInInputBuffer()
    const {
  DCHECK_NE(surround_format_, kInvalid);
  if (num_zero_padded_frames_ > 0) {
    // Zero padded output buffers must be consumed prior to
    // |AddInterleavedBuffer| calls;
    return 0;
  }
  if (input_audio_buffer_queue_->Full()) {
    return 0;
  }
  // Subtract two buffers from the available input slots to ensure the buffer
  // partitioner can be flushed at any time while keeping an extra buffer
  // available in the |buffer_partitioner_| callback for the next incoming data.
  const size_t num_frames_available_in_input_slots =
      (kNumMaxBuffers - input_audio_buffer_queue_->Size() - 2) *
      frames_per_buffer_;
  DCHECK_GT(frames_per_buffer_, buffer_partitioner_->GetNumBufferedFrames());
  const size_t num_frames_available_in_buffer_partitioner =
      frames_per_buffer_ - buffer_partitioner_->GetNumBufferedFrames();
  return num_frames_available_in_input_slots +
         num_frames_available_in_buffer_partitioner;
}

size_t BinauralSurroundRendererImpl::AddInterleavedInput(
    const int16* input_buffer_ptr, size_t num_channels, size_t num_frames) {
  return AddInputBufferTemplated<const int16*>(input_buffer_ptr, num_channels,
                                               num_frames);
}

size_t BinauralSurroundRendererImpl::AddInterleavedInput(
    const float* input_buffer_ptr, size_t num_channels, size_t num_frames) {
  return AddInputBufferTemplated<const float*>(input_buffer_ptr, num_channels,
                                               num_frames);
}

size_t BinauralSurroundRendererImpl::AddPlanarInput(
    const int16* const* input_buffer_ptrs, size_t num_channels,
    size_t num_frames) {
  return AddInputBufferTemplated<const int16* const*>(input_buffer_ptrs,
                                                      num_channels, num_frames);
}

size_t BinauralSurroundRendererImpl::AddPlanarInput(
    const float* const* input_buffer_ptrs, size_t num_channels,
    size_t num_frames) {
  return AddInputBufferTemplated<const float* const*>(input_buffer_ptrs,
                                                      num_channels, num_frames);
}

template <typename BufferType>
size_t BinauralSurroundRendererImpl::AddInputBufferTemplated(
    const BufferType input_buffer_ptr, size_t num_channels, size_t num_frames) {
  DCHECK_NE(surround_format_, kInvalid);
  if (num_channels != num_input_channels_) {
    LOG(WARNING) << "Invalid number of input channels";
    return 0;
  }

  if (num_zero_padded_frames_ > 0) {
    LOG(WARNING) << "Zero padded output buffers must be consumed prior to "
                    "|AddInterleavedBuffer| calls";
    return 0;
  }
  const size_t num_available_input_frames =
      std::min(num_frames, GetNumAvailableFramesInInputBuffer());

  buffer_partitioner_->AddBuffer(input_buffer_ptr, num_input_channels_,
                                 num_available_input_frames);
  total_frames_buffered_ += num_available_input_frames;
  return num_available_input_frames;
}

size_t BinauralSurroundRendererImpl::GetAvailableFramesInStereoOutputBuffer()
    const {
  const size_t num_available_samples_in_buffers =
      (input_audio_buffer_queue_->Size() * frames_per_buffer_) +
      buffer_unpartitioner_->GetNumBufferedFrames();
  return std::min(total_frames_buffered_, num_available_samples_in_buffers);
}

size_t BinauralSurroundRendererImpl::GetInterleavedStereoOutput(
    int16* output_buffer_ptr, size_t num_frames) {
  return GetStereoOutputBufferTemplated<int16*>(output_buffer_ptr, num_frames);
}

size_t BinauralSurroundRendererImpl::GetInterleavedStereoOutput(
    float* output_buffer_ptr, size_t num_frames) {
  return GetStereoOutputBufferTemplated<float*>(output_buffer_ptr, num_frames);
}

size_t BinauralSurroundRendererImpl::GetPlanarStereoOutput(
    int16** output_buffer_ptrs, size_t num_frames) {
  return GetStereoOutputBufferTemplated<int16**>(output_buffer_ptrs,
                                                 num_frames);
}

size_t BinauralSurroundRendererImpl::GetPlanarStereoOutput(
    float** output_buffer_ptrs, size_t num_frames) {
  return GetStereoOutputBufferTemplated<float**>(output_buffer_ptrs,
                                                 num_frames);
}

template <typename BufferType>
size_t BinauralSurroundRendererImpl::GetStereoOutputBufferTemplated(
    BufferType output_buffer_ptr, size_t num_frames) {
  DCHECK_NE(surround_format_, kInvalid);
  const size_t num_frames_available = GetAvailableFramesInStereoOutputBuffer();
  size_t num_frames_to_be_processed =
      std::min(num_frames_available, num_frames);
  if (num_frames_to_be_processed > total_frames_buffered_) {
    // Avoid outputting zero padded input frames from |TriggerProcessing|
    // calls.
    num_frames_to_be_processed = total_frames_buffered_;
  }

  const size_t num_frames_written = buffer_unpartitioner_->GetBuffer(
      output_buffer_ptr, kNumStereoChannels, num_frames_to_be_processed);

  DCHECK_GE(total_frames_buffered_, num_frames_written);
  total_frames_buffered_ -= num_frames_written;

  if (total_frames_buffered_ == 0) {
    // Clear zero padded frames from |TriggerProcessing| calls.
    buffer_unpartitioner_->Clear();
    num_zero_padded_frames_ = 0;
  }

  return num_frames_written;
}

void BinauralSurroundRendererImpl::Clear() {
  input_audio_buffer_queue_->Clear();
  buffer_partitioner_->Clear();
  buffer_unpartitioner_->Clear();
  total_frames_buffered_ = 0;
  num_zero_padded_frames_ = 0;
}

bool BinauralSurroundRendererImpl::TriggerProcessing() {
  if (num_zero_padded_frames_ > 0) {
    LOG(WARNING) << "Zero padded output buffers must be consumed prior to "
                    "|TriggerProcessing| calls";
    return false;
  }
  num_zero_padded_frames_ = buffer_partitioner_->Flush();
  return num_zero_padded_frames_ > 0;
}

void BinauralSurroundRendererImpl::SetHeadRotation(float w, float x, float y,
                                                   float z) {
  resonance_audio_api_->SetHeadRotation(x, y, z, w);
}

AudioBuffer* BinauralSurroundRendererImpl::ProcessBuffer() {
  if (input_audio_buffer_queue_->Size() == 0) {
    LOG(WARNING) << "Buffer underflow detected";
    return nullptr;
  }

  const AudioBuffer* input = input_audio_buffer_queue_->AcquireOutputObject();
  DCHECK_EQ(input->num_frames(), frames_per_buffer_);
  DCHECK_EQ(num_input_channels_, input->num_channels());
  GetRawChannelDataPointersFromAudioBuffer(*input, &temp_planar_buffer_ptrs_);
  // Initialize surround rendering.
  const float* planar_ptr;

  switch (surround_format_) {
    case kSurroundMono:
    case kSurroundStereo:
    case kSurroundFiveDotOne:
    case kSurroundSevenDotOne:
      DCHECK_EQ(input->num_channels(), source_ids_.size());
      for (size_t source_itr = 0; source_itr < source_ids_.size();
           ++source_itr) {
        planar_ptr = (*input)[source_itr].begin();
        resonance_audio_api_->SetPlanarBuffer(source_ids_[source_itr],
                                              &planar_ptr, kNumMonoChannels,
                                              input->num_frames());
      }
      break;
    case kFirstOrderAmbisonics:
    case kSecondOrderAmbisonics:
    case kThirdOrderAmbisonics:
      DCHECK_EQ(source_ids_.size(), 1U);
      resonance_audio_api_->SetPlanarBuffer(
          source_ids_[0], temp_planar_buffer_ptrs_.data(),
          input->num_channels(), input->num_frames());
      break;
    case kFirstOrderAmbisonicsWithNonDiegeticStereo:
    case kSecondOrderAmbisonicsWithNonDiegeticStereo:
    case kThirdOrderAmbisonicsWithNonDiegeticStereo:
      DCHECK_EQ(source_ids_.size(), 2U);
      DCHECK_GT(input->num_channels(), kNumStereoChannels);
      static_cast<ResonanceAudioApiImpl*>(resonance_audio_api_.get())
          ->SetPlanarBuffer(source_ids_[0], temp_planar_buffer_ptrs_.data(),
                            input->num_channels() - kNumStereoChannels,
                            input->num_frames());
      static_cast<ResonanceAudioApiImpl*>(resonance_audio_api_.get())
          ->SetPlanarBuffer(source_ids_[1],
                            temp_planar_buffer_ptrs_.data() +
                                (input->num_channels() - kNumStereoChannels),
                            kNumStereoChannels, input->num_frames());
      break;
    default:
      LOG(FATAL) << "Undefined surround format";
      break;
  }

  // Create a copy of the processed |AudioBuffer| to pass it to output buffer
  // queue.
  auto* const vraudio_api_impl =
      static_cast<ResonanceAudioApiImpl*>(resonance_audio_api_.get());
  vraudio_api_impl->ProcessNextBuffer();
  output_buffer_ = *vraudio_api_impl->GetStereoOutputBuffer();

  if (output_gain_ != 1.0f) {
    for (AudioBuffer::Channel& channel : output_buffer_) {
      ScalarMultiply(output_buffer_.num_frames(), output_gain_, channel.begin(),
                     channel.begin());
    }
  }
  input_audio_buffer_queue_->ReleaseOutputObject(input);
  return &output_buffer_;
}

}  // namespace vraudio
