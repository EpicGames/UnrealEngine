/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_BINAURAL_SURROUND_RENDERER_IMPL_H_
#define RESONANCE_AUDIO_GRAPH_BINAURAL_SURROUND_RENDERER_IMPL_H_

#include <memory>
#include <string>

#include "api/binaural_surround_renderer.h"
#include "api/resonance_audio_api.h"
#include "base/constants_and_types.h"
#include "utils/buffer_partitioner.h"
#include "utils/buffer_unpartitioner.h"
#include "utils/threadsafe_fifo.h"

namespace vraudio {

// Renders virtual surround sound as well as ambisonic soundfields to binaural
// stereo.
class BinauralSurroundRendererImpl : public BinauralSurroundRenderer {
 public:
  // Constructor.
  //
  // @param frames_per_buffer Number of frames in output buffer.
  // @param sample_rate_hz Sample rate of audio buffers.
  BinauralSurroundRendererImpl(size_t frames_per_buffer, int sample_rate_hz);

  ~BinauralSurroundRendererImpl() override= default;

  // Initializes surround sound decoding.
  //
  // @param surround_format Surround sound input format.
  // @return True on success.
  bool Init(SurroundFormat surround_format);

  // Implements |AudioRenderer| interface.
  void SetStereoSpeakerMode(bool enabled) override;
  size_t GetNumAvailableFramesInInputBuffer() const override;
  size_t AddInterleavedInput(const int16* input_buffer_ptr, size_t num_channels,
                             size_t num_frames) override;
  size_t AddInterleavedInput(const float* input_buffer_ptr, size_t num_channels,
                             size_t num_frames) override;
  size_t AddPlanarInput(const int16* const* input_buffer_ptrs,
                        size_t num_channels, size_t num_frames) override;
  size_t AddPlanarInput(const float* const* input_buffer_ptrs,
                        size_t num_channels, size_t num_frames) override;
  size_t GetAvailableFramesInStereoOutputBuffer() const override;
  size_t GetInterleavedStereoOutput(int16* output_buffer_ptr,
                                    size_t num_frames) override;
  size_t GetInterleavedStereoOutput(float* output_buffer_ptr,
                                    size_t num_frames) override;
  size_t GetPlanarStereoOutput(int16** output_buffer_ptrs,
                               size_t num_frames) override;
  size_t GetPlanarStereoOutput(float** output_buffer_ptrs,
                               size_t num_frames) override;
  bool TriggerProcessing() override;
  void Clear() override;
  void SetHeadRotation(float w, float x, float y, float z) override;

 protected:
  // Protected default constructor for mock tests.
  BinauralSurroundRendererImpl();

 private:
  // Callback triggered by |buffer_partitioner_| whenever a new |AudioBuffer|
  // has been generated.
  //
  // @param processed_buffer Pointer to processed buffer.
  // @return Pointer to next |AudioBuffer| to be filled up.
  AudioBuffer* BufferPartitionerCallback(AudioBuffer* processed_buffer);

  // Helper method to implement |AddInterleavedInput| independently from the
  // sample type.
  //
  // @tparam BufferType Input buffer type.
  // @param input_buffer_ptr Pointer to interleaved input data.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  // @return The number of consumed samples.
  template <typename BufferType>
  size_t AddInputBufferTemplated(const BufferType input_buffer_ptr,
                                 size_t num_channels, size_t num_frames);

  // Helper method to implement |GetInterleavedOutput| independently from the
  // sample type.
  //
  // @tparam BufferType Output buffer type.
  // @param output_buffer_ptr Pointer to allocated interleaved output buffer.
  // @param num_frames Size of output buffer in frames.
  // @return The number of consumed frames.
  template <typename BufferType>
  size_t GetStereoOutputBufferTemplated(BufferType output_buffer_ptr,
                                        size_t num_frames);

  // Helper method to obtain the expected number of audio channels for a given
  // surround format.
  //
  // @param surround_format Surround format query.
  // @return Number of audio channels.
  static size_t GetExpectedNumChannelsFromSurroundFormat(
      SurroundFormat surround_format);

  // Process method executed by |buffer_unpartitioner_|.
  AudioBuffer* ProcessBuffer();

  // Initializes binaural mono rendering.
  void InitializeBinauralMono();

  // Initializes binaural stereo rendering.
  void InitializeBinauralStereo();

  // Initializes binaural 5.1 rendering.
  void InitializeBinauralSurround5dot1();

  // Initializes binaural 7.1 rendering.
  void InitializeBinauralSurround7dot1();

  // Initializes binaural ambisonic rendering.
  void InitializeAmbisonics();

  // Initializes binaural ambisonic rendering with non-diegetic stereo.
  void InitializeAmbisonicsWithNonDiegeticStereo();

  // Creates a sound object at given angle within the horizontal listener plane.
  SourceId CreateSoundObject(float azimuth_deg);

  // Initializes room reverb for virtual surround sound rendering.
  void InitializeRoomReverb();

  // ResonanceAudioApi instance.
  std::unique_ptr<ResonanceAudioApi> resonance_audio_api_;

  // Frames per buffer.
  const size_t frames_per_buffer_;

  // System sample rate.
  const int sample_rate_hz_;

  // Selected surround sound format.
  SurroundFormat surround_format_;

  // Number of input channels.
  size_t num_input_channels_;

  // Partitions input buffers into |AudioBuffer|s.
  std::unique_ptr<BufferPartitioner> buffer_partitioner_;

  // Buffer queue containing partitioned input |AudioBuffer|s.
  std::unique_ptr<ThreadsafeFifo<AudioBuffer>> input_audio_buffer_queue_;

  // Binaural stereo output buffer.
  AudioBuffer output_buffer_;

  // Unpartitions processed |AudioBuffer|s into interleaved output buffers.
  std::unique_ptr<BufferUnpartitioner> buffer_unpartitioner_;

  // Vector containing the source ids of all rendered sound sources.
  std::vector<SourceId> source_ids_;

  // Total number of frames currently buffered.
  size_t total_frames_buffered_;

  // Total number of zero padded frames from |TriggerProcessing| calls.
  size_t num_zero_padded_frames_;

  // Temporary buffer to store pointers to planar ambisonic and stereo channels.
  std::vector<const float*> temp_planar_buffer_ptrs_;

  // Global output gain adjustment, to avoid clipping of individual channels
  // in virtual speaker modes.
  float output_gain_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_BINAURAL_SURROUND_RENDERER_IMPL_H_
