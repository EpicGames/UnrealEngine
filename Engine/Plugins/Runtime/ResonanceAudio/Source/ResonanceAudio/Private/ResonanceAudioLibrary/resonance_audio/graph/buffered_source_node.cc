/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/buffered_source_node.h"

#include "base/constants_and_types.h"
#include "base/logging.h"

namespace vraudio {

BufferedSourceNode::BufferedSourceNode(SourceId source_id, size_t num_channels,
                                       size_t frames_per_buffer)
    : source_id_(source_id),
      input_audio_buffer_(num_channels, frames_per_buffer),
      new_buffer_flag_(false) {
  input_audio_buffer_.Clear();
}

AudioBuffer* BufferedSourceNode::GetMutableAudioBufferAndSetNewBufferFlag() {
  new_buffer_flag_ = true;
  return &input_audio_buffer_;
}

const AudioBuffer* BufferedSourceNode::AudioProcess() {
  if (!new_buffer_flag_) {
    return nullptr;
  }
  new_buffer_flag_ = false;
  input_audio_buffer_.set_source_id(source_id_);
  return &input_audio_buffer_;
}

}  // namespace vraudio
