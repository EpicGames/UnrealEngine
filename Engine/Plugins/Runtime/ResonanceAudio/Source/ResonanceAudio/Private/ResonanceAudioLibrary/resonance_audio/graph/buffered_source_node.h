/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_BUFFERED_SOURCE_NODE_H_
#define RESONANCE_AUDIO_GRAPH_BUFFERED_SOURCE_NODE_H_

#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "node/source_node.h"

namespace vraudio {

// Node that sets the |AudioBuffer| of a source. This class is *not*
// thread-safe and calls to this class must be synchronized with the graph
// processing.
class BufferedSourceNode : public SourceNode {
 public:
  // Constructor.
  //
  // @param source_id Source id.
  // @param num_channel Number of channels in output buffers.
  BufferedSourceNode(SourceId source_id, size_t num_channels,
                     size_t frames_per_buffer);

  // Returns a mutable pointer to the internal |AudioBuffer| and sets a flag to
  // process the buffer in the next graph processing iteration. Calls to this
  // method must be synchronized with the audio graph processing.
  //
  // @return Mutable audio buffer pointer.
  AudioBuffer* GetMutableAudioBufferAndSetNewBufferFlag();

 protected:
  // Implements SourceNode.
  const AudioBuffer* AudioProcess() override;

  // Source id.
  const SourceId source_id_;

  // Input audio buffer.
  AudioBuffer input_audio_buffer_;

  // Flag indicating if an new audio buffer has been set via
  // |GetMutableAudioBufferAndSetNewBufferFlag|.
  bool new_buffer_flag_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_BUFFERED_SOURCE_NODE_H_
