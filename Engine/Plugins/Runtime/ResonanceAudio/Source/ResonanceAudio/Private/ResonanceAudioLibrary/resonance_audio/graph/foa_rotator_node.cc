/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/foa_rotator_node.h"

#include "base/logging.h"


namespace vraudio {

FoaRotatorNode::FoaRotatorNode(SourceId source_id,
                               const SystemSettings& system_settings)
    : system_settings_(system_settings),
      output_buffer_(kNumFirstOrderAmbisonicChannels,
                     system_settings.GetFramesPerBuffer()) {
  output_buffer_.Clear();
  output_buffer_.set_source_id(source_id);
}

const AudioBuffer* FoaRotatorNode::AudioProcess(const NodeInput& input) {


  // Get the soundfield input buffer.
  const AudioBuffer* input_buffer = input.GetSingleInput();
  DCHECK(input_buffer);
  DCHECK_GT(input_buffer->num_frames(), 0U);
  DCHECK_EQ(input_buffer->num_channels(), 4U);
  DCHECK_EQ(input_buffer->source_id(), output_buffer_.source_id());

  // Rotate soundfield buffer by the inverse head orientation.
  const auto source_parameters =
      system_settings_.GetSourceParameters(input_buffer->source_id());
  if (source_parameters == nullptr) {
    LOG(WARNING) << "Could not find source parameters";
    return nullptr;
  }

  const WorldRotation& source_rotation =
      source_parameters->object_transform.rotation;
  const WorldRotation inverse_head_rotation =
      system_settings_.GetHeadRotation().conjugate();
  const WorldRotation rotation = inverse_head_rotation * source_rotation;
  const bool rotation_applied =
      foa_rotator_.Process(rotation, *input_buffer, &output_buffer_);

  if (!rotation_applied) {
    return input_buffer;
  }

  // Copy buffer parameters.
  return &output_buffer_;
}

}  // namespace vraudio
