/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_FOA_ROTATOR_NODE_H_
#define RESONANCE_AUDIO_GRAPH_FOA_ROTATOR_NODE_H_

#include <memory>

#include "ambisonics/foa_rotator.h"
#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts a single first order PeriphonicSoundfieldBuffer as input
// and outputs a rotated PeriphonicSoundfieldBuffer of the corresponding
// soundfield input using head rotation information from the system settings.
class FoaRotatorNode : public ProcessingNode {
 public:
  FoaRotatorNode(SourceId source_id, const SystemSettings& system_settings);

 protected:
  // Implements ProcessingNode. Returns a null pointer if we are in stereo
  // loudspeaker or stereo pan mode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  const SystemSettings& system_settings_;

  // Soundfield rotator used to rotate first order soundfields.
  FoaRotator foa_rotator_;

  // Output buffer.
  AudioBuffer output_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_FOA_ROTATOR_NODE_H_
