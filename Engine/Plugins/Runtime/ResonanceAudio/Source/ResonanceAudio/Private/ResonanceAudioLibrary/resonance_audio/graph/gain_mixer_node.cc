/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/gain_mixer_node.h"

#include <vector>

#include "base/constants_and_types.h"


namespace vraudio {

GainMixerNode::GainMixerNode(const AttenuationType& attenuation_type,
                             const SystemSettings& system_settings,
                             size_t num_channels)
    : mute_enabled_(false),
      attenuation_type_(attenuation_type),
      gain_mixer_(num_channels, system_settings.GetFramesPerBuffer()),
      system_settings_(system_settings) {}

void GainMixerNode::SetMute(bool mute_enabled) { mute_enabled_ = mute_enabled; }

bool GainMixerNode::CleanUp() {
  CallCleanUpOnInputNodes();
  // Prevent node from being disconnected when all sources are removed.
  return false;
}

const AudioBuffer* GainMixerNode::AudioProcess(const NodeInput& input) {


  if (mute_enabled_) {
    // Skip processing and output nullptr audio buffer.
    return nullptr;
  }

  // Apply the gain to each input buffer channel.
  gain_mixer_.Reset();
  for (auto input_buffer : input.GetInputBuffers()) {
    const auto source_parameters =
        system_settings_.GetSourceParameters(input_buffer->source_id());
    if (source_parameters != nullptr) {
      const float target_gain =
          source_parameters->attenuations[attenuation_type_];
      const size_t num_channels = input_buffer->num_channels();
      gain_mixer_.AddInput(*input_buffer,
                           std::vector<float>(num_channels, target_gain));
    }
  }
  return gain_mixer_.GetOutput();
}

}  // namespace vraudio
