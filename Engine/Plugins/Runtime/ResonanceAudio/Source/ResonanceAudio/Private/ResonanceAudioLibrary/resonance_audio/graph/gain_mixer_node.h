/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_GAIN_MIXER_NODE_H_
#define RESONANCE_AUDIO_GRAPH_GAIN_MIXER_NODE_H_

#include "base/audio_buffer.h"
#include "base/source_parameters.h"
#include "dsp/gain_mixer.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts multiple input buffers, calculates and applies a gain
// value to each buffer based upon the given |AttenuationType| and then mixes
// the results together.
class GainMixerNode : public ProcessingNode {
 public:
  // Constructs |GainMixerNode| with given gain calculation method.
  //
  // @param attenuation_type Gain attenuation type to be used.
  // @param system_settings Global system settings.
  // @param num_channels Number of channels.
  GainMixerNode(const AttenuationType& attenuation_type,
                const SystemSettings& system_settings, size_t num_channels);

  // Mute the mixer node by skipping the audio processing and outputting nullptr
  // buffers.
  void SetMute(bool mute_enabled);

  // Node implementation.
  bool CleanUp() final;

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  // Flag indicating the mute status.
  bool mute_enabled_;

  // Gain attenuation type.
  const AttenuationType attenuation_type_;

  // Gain mixer.
  GainMixer gain_mixer_;

  // Global system settings.
  const SystemSettings& system_settings_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_GAIN_MIXER_NODE_H_
