/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/gain_node.h"

#include <cmath>


#include "dsp/gain.h"

namespace vraudio {

GainNode::GainNode(SourceId source_id, size_t num_channels,
                   const AttenuationType& attenuation_type,
                   const SystemSettings& system_settings)
    : num_channels_(num_channels),
      attenuation_type_(attenuation_type),
      gain_processors_(num_channels_),
      system_settings_(system_settings),
      output_buffer_(num_channels, system_settings.GetFramesPerBuffer()) {
  DCHECK_GT(num_channels, 0U);
  output_buffer_.set_source_id(source_id);
}

const AudioBuffer* GainNode::AudioProcess(const NodeInput& input) {


  const AudioBuffer* input_buffer = input.GetSingleInput();
  DCHECK(input_buffer);
  DCHECK_EQ(input_buffer->num_channels(), num_channels_);
  DCHECK_EQ(input_buffer->source_id(), output_buffer_.source_id());

  const auto source_parameters =
      system_settings_.GetSourceParameters(input_buffer->source_id());
  if (source_parameters == nullptr) {
    LOG(WARNING) << "Could not find source parameters";
    return nullptr;
  }

  const float current_gain = gain_processors_[0].GetGain();
  const float target_gain = source_parameters->attenuations[attenuation_type_];
  if (IsGainNearZero(target_gain) && IsGainNearZero(current_gain)) {
    // Make sure the gain processors are initialized.
    for (size_t i = 0; i < num_channels_; ++i) {
      gain_processors_[i].Reset(0.0f);
    }
    // Skip processing in case of zero gain.
    return nullptr;
  }
  if (IsGainNearUnity(target_gain) && IsGainNearUnity(current_gain)) {
    // Make sure the gain processors are initialized.
    for (size_t i = 0; i < num_channels_; ++i) {
      gain_processors_[i].Reset(1.0f);
    }
    // Skip processing in case of unity gain.
    return input_buffer;
  }

  // Apply the gain to each input buffer channel.
  for (size_t i = 0; i < num_channels_; ++i) {
    gain_processors_[i].ApplyGain(target_gain, (*input_buffer)[i],
                                  &output_buffer_[i],
                                  false /* accumulate_output */);
  }

  return &output_buffer_;
}

}  // namespace vraudio
