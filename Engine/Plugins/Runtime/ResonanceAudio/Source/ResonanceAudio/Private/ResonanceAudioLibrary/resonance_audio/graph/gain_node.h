/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_GAIN_NODE_H_
#define RESONANCE_AUDIO_GRAPH_GAIN_NODE_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "base/source_parameters.h"
#include "dsp/gain_processor.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that calculates and applies a gain value to each channel of an input
// buffer based upon the given |GainCalculator|.
class GainNode : public ProcessingNode {
 public:
  // Constructs |GainNode| with given gain attenuation method.
  //
  // @param source_id Output buffer source id.
  // @param num_channels Number of channels in the input buffer.
  // @param attenuation_type Gain attenuation type to be used.
  // @param system_settings Global system settings.
  GainNode(SourceId source_id, size_t num_channels,
           const AttenuationType& attenuation_type,
           const SystemSettings& system_settings);

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  // Number of channels of the audio buffer.
  const size_t num_channels_;

  // Gain attenuation type.
  const AttenuationType attenuation_type_;

  // Gain processors per each channel.
  std::vector<GainProcessor> gain_processors_;

  // Global system settings.
  const SystemSettings& system_settings_;

  // Output buffer.
  AudioBuffer output_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_GAIN_NODE_H_
