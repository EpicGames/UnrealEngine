/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/graph_manager.h"

#include <functional>

#include "ambisonics/utils.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "graph/foa_rotator_node.h"
#include "graph/gain_node.h"
#include "graph/hoa_rotator_node.h"
#include "graph/mono_from_soundfield_node.h"
#include "graph/near_field_effect_node.h"
#include "graph/occlusion_node.h"

namespace vraudio {

GraphManager::GraphManager(const SystemSettings& system_settings)
    :
      room_effects_enabled_(true),
      config_(GlobalConfig()),
      system_settings_(system_settings),
      fft_manager_(system_settings.GetFramesPerBuffer()),
      output_node_(std::make_shared<SinkNode>()) {
  CHECK_LE(system_settings.GetFramesPerBuffer(), kMaxSupportedNumFrames);

  stereo_mixer_node_ =
      std::make_shared<MixerNode>(system_settings_, kNumStereoChannels);
  output_node_->Connect(stereo_mixer_node_);

  /// Initialize the Ambisonic Lookup Table.
  lookup_table_.reset(new AmbisonicLookupTable(config_.max_ambisonic_order));
  // Initialize the Ambisonic Renderer subgraphs.
  for (const auto& sh_hrir_filename_itr : config_.sh_hrir_filenames) {
    const int ambisonic_order = sh_hrir_filename_itr.first;
    const auto& sh_hrir_filename = sh_hrir_filename_itr.second;
    InitializeAmbisonicRendererGraph(ambisonic_order, sh_hrir_filename);
    // Initialize the Ambisonic Mixing Encoders for HRTF sound object rendering.
    ambisonic_mixing_encoder_nodes_[ambisonic_order] =
        std::make_shared<AmbisonicMixingEncoderNode>(
            system_settings_, *lookup_table_, ambisonic_order);
    ambisonic_mixer_nodes_[ambisonic_order]->Connect(
        ambisonic_mixing_encoder_nodes_[ambisonic_order]);
  }

  // Stereo mixing panner node used in non-HRTF sound object rendering.
  stereo_mixing_panner_node_ =
      std::make_shared<StereoMixingPannerNode>(system_settings_);
  stereo_mixer_node_->Connect(stereo_mixing_panner_node_);

  // Initialize room effects graphs.
  InitializeReflectionsGraph();
  InitializeReverbGraph();
  // Initialize ambisonic output mixer.
  ambisonic_output_mixer_.reset(
      new Mixer(GetNumPeriphonicComponents(config_.max_ambisonic_order),
                system_settings.GetFramesPerBuffer()));
}

void GraphManager::CreateAmbisonicSource(SourceId ambisonic_source_id,
                                         size_t num_channels) {
  DCHECK(source_nodes_.find(ambisonic_source_id) == source_nodes_.end());
  // Create a new |ambisonic_source_node| and register to |source_nodes_|.
  auto ambisonic_source_node = std::make_shared<BufferedSourceNode>(
      ambisonic_source_id, num_channels, system_settings_.GetFramesPerBuffer());
  source_nodes_[ambisonic_source_id] = ambisonic_source_node;

  // Connect |ambisonic_source_node| to the ambisonic decoding pipeline.
  const int ambisonic_order = GetPeriphonicAmbisonicOrder(num_channels);
  auto direct_attenuation_node =
      std::make_shared<GainNode>(ambisonic_source_id, num_channels,
                                 AttenuationType::kDirect, system_settings_);
  direct_attenuation_node->Connect(ambisonic_source_node);
  if (ambisonic_order == 1) {
    // First order case.
    auto foa_rotator_node =
        std::make_shared<FoaRotatorNode>(ambisonic_source_id, system_settings_);
    foa_rotator_node->Connect(direct_attenuation_node);
    ambisonic_mixer_nodes_[ambisonic_order]->Connect(foa_rotator_node);
  } else {
    // Higher orders case.
    auto hoa_rotator_node = std::make_shared<HoaRotatorNode>(
        ambisonic_source_id, system_settings_, ambisonic_order);
    hoa_rotator_node->Connect(direct_attenuation_node);
    ambisonic_mixer_nodes_[ambisonic_order]->Connect(hoa_rotator_node);
  }
  // Connect to room effects rendering pipeline.
  auto mono_from_soundfield_node = std::make_shared<MonoFromSoundfieldNode>(
      ambisonic_source_id, system_settings_);
  mono_from_soundfield_node->Connect(ambisonic_source_node);
  reflections_gain_mixer_node_->Connect(mono_from_soundfield_node);
  reverb_gain_mixer_node_->Connect(mono_from_soundfield_node);
}

void GraphManager::CreateSoundObjectSource(SourceId sound_object_source_id,
                                           int ambisonic_order,
                                           bool enable_hrtf,
                                           bool enable_direct_rendering) {
  DCHECK(source_nodes_.find(sound_object_source_id) == source_nodes_.end());
  // Create a new |sound_object_source_node| and register to |source_nodes_|.
  auto sound_object_source_node = std::make_shared<BufferedSourceNode>(
      sound_object_source_id, kNumMonoChannels,
      system_settings_.GetFramesPerBuffer());
  source_nodes_[sound_object_source_id] = sound_object_source_node;

  // Create direct rendering pipeline.
  if (enable_direct_rendering) {
    auto direct_attenuation_node =
        std::make_shared<GainNode>(sound_object_source_id, kNumMonoChannels,
                                   AttenuationType::kDirect, system_settings_);
    direct_attenuation_node->Connect(sound_object_source_node);
    auto occlusion_node = std::make_shared<OcclusionNode>(
        sound_object_source_id, system_settings_);
    occlusion_node->Connect(direct_attenuation_node);
    auto near_field_effect_node = std::make_shared<NearFieldEffectNode>(
        sound_object_source_id, system_settings_);

    if (enable_hrtf) {
      ambisonic_mixing_encoder_nodes_[ambisonic_order]->Connect(occlusion_node);
    } else {
      stereo_mixing_panner_node_->Connect(occlusion_node);
    }

    near_field_effect_node->Connect(occlusion_node);
    stereo_mixer_node_->Connect(near_field_effect_node);
  }

  // Connect to room effects rendering pipeline.
  reflections_gain_mixer_node_->Connect(sound_object_source_node);
  reverb_gain_mixer_node_->Connect(sound_object_source_node);
}

void GraphManager::EnableRoomEffects(bool enable) {
  room_effects_enabled_ = enable;
  reflections_gain_mixer_node_->SetMute(!room_effects_enabled_);
  reverb_gain_mixer_node_->SetMute(!room_effects_enabled_);
}

const AudioBuffer* GraphManager::GetAmbisonicBuffer() const {
  ambisonic_output_mixer_->Reset();
  for (const auto& ambisonic_mixer_node_itr : ambisonic_mixer_nodes_) {
    const auto* ambisonic_buffer =
        ambisonic_mixer_node_itr.second->GetOutputBuffer();
    if (ambisonic_buffer != nullptr) {
      ambisonic_output_mixer_->AddInput(*ambisonic_buffer);
    }
  }
  return ambisonic_output_mixer_->GetOutput();
}

const AudioBuffer* GraphManager::GetStereoBuffer() const {
  return stereo_mixer_node_->GetOutputBuffer();
}

size_t GraphManager::GetNumMaxAmbisonicChannels() const {
  return GetNumPeriphonicComponents(config_.max_ambisonic_order);
}

bool GraphManager::GetRoomEffectsEnabled() const {
  return room_effects_enabled_;
}

void GraphManager::UpdateRoomReflections() { reflections_node_->Update(); }

void GraphManager::UpdateRoomReverb() { reverb_node_->Update(); }

void GraphManager::InitializeReverbGraph() {
  reverb_gain_mixer_node_ = std::make_shared<GainMixerNode>(
      AttenuationType::kReverb, system_settings_, kNumMonoChannels);
  reverb_node_ = std::make_shared<ReverbNode>(system_settings_, &fft_manager_);
  reverb_node_->Connect(reverb_gain_mixer_node_);
  stereo_mixer_node_->Connect(reverb_node_);
}

void GraphManager::InitializeReflectionsGraph() {
  reflections_gain_mixer_node_ = std::make_shared<GainMixerNode>(
      AttenuationType::kReflections, system_settings_, kNumMonoChannels);
  reflections_node_ = std::make_shared<ReflectionsNode>(system_settings_);
  reflections_node_->Connect(reflections_gain_mixer_node_);
  // Reflections are limited to First Order Ambisonics to reduce complexity.
  const int kAmbisonicOrder1 = 1;
  ambisonic_mixer_nodes_[kAmbisonicOrder1]->Connect(reflections_node_);
}

void GraphManager::CreateAmbisonicPannerSource(SourceId sound_object_source_id,
                                               bool enable_hrtf) {
  DCHECK(source_nodes_.find(sound_object_source_id) == source_nodes_.end());
  // Create a new |sound_object_source_node| and register to |source_nodes_|.
  auto sound_object_source_node = std::make_shared<BufferedSourceNode>(
      sound_object_source_id, kNumMonoChannels,
      system_settings_.GetFramesPerBuffer());
  source_nodes_[sound_object_source_id] = sound_object_source_node;

  if (enable_hrtf) {
    ambisonic_mixing_encoder_nodes_[config_.max_ambisonic_order]->Connect(
        sound_object_source_node);
  } else {
    stereo_mixing_panner_node_->Connect(sound_object_source_node);
  }
}

void GraphManager::CreateStereoSource(SourceId stereo_source_id) {
  DCHECK(source_nodes_.find(stereo_source_id) == source_nodes_.end());
  // Create a new |stereo_source_node| and register to |source_nodes_|.
  auto stereo_source_node = std::make_shared<BufferedSourceNode>(
      stereo_source_id, kNumStereoChannels,
      system_settings_.GetFramesPerBuffer());
  source_nodes_[stereo_source_id] = stereo_source_node;

  // Connect |stereo_source_node| to the stereo rendering pipeline.
  auto gain_node =
      std::make_shared<GainNode>(stereo_source_id, kNumStereoChannels,
                                 AttenuationType::kInput, system_settings_);
  gain_node->Connect(stereo_source_node);
  stereo_mixer_node_->Connect(gain_node);
}

void GraphManager::DestroySource(SourceId source_id) {
  auto source_node = LookupSourceNode(source_id);
  if (source_node != nullptr) {
    // Disconnect the source from the graph.
    source_node->MarkEndOfStream();
    output_node_->CleanUp();
    // Unregister the source from |source_nodes_|.
    source_nodes_.erase(source_id);
  }
}

std::shared_ptr<SinkNode> GraphManager::GetSinkNode() { return output_node_; }

void GraphManager::Process() {

  output_node_->ReadInputs();
}

AudioBuffer* GraphManager::GetMutableAudioBuffer(SourceId source_id) {
  auto source_node = LookupSourceNode(source_id);
  if (source_node == nullptr) {
    return nullptr;
  }
  return source_node->GetMutableAudioBufferAndSetNewBufferFlag();
}

void GraphManager::InitializeAmbisonicRendererGraph(
    int ambisonic_order, const std::string& sh_hrir_filename) {
  CHECK_LE(ambisonic_order, config_.max_ambisonic_order);
  const size_t num_channels = GetNumPeriphonicComponents(ambisonic_order);
  // Create binaural decoder pipeline.
  ambisonic_mixer_nodes_[ambisonic_order] = std::make_shared<MixerNode>(system_settings_, num_channels);
  auto ambisonic_binaural_decoder_node = std::make_shared<AmbisonicBinauralDecoderNode>(system_settings_, ambisonic_order, sh_hrir_filename, &fft_manager_, &resampler_);
  ambisonic_binaural_decoder_node->Connect(ambisonic_mixer_nodes_[ambisonic_order]);
  stereo_mixer_node_->Connect(ambisonic_binaural_decoder_node);
}

std::shared_ptr<BufferedSourceNode> GraphManager::LookupSourceNode(
    SourceId source_id) {
  auto source_node_itr = source_nodes_.find(source_id);
  if (source_node_itr == source_nodes_.end()) {
    LOG(WARNING) << "Source node " << source_id << " not found";
    return nullptr;
  }
  return source_node_itr->second;
}

}  // namespace vraudio
