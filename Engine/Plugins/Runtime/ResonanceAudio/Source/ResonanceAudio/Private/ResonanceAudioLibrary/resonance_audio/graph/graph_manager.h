/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_GRAPH_MANAGER_H_
#define RESONANCE_AUDIO_GRAPH_GRAPH_MANAGER_H_

#include <memory>
#include <string>
#include <unordered_map>

#include "ambisonics/ambisonic_lookup_table.h"
#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "config/global_config.h"
#include "dsp/fft_manager.h"
#include "dsp/resampler.h"
#include "graph/ambisonic_binaural_decoder_node.h"
#include "graph/ambisonic_mixing_encoder_node.h"
#include "graph/buffered_source_node.h"
#include "graph/gain_mixer_node.h"
#include "graph/mixer_node.h"
#include "graph/reflections_node.h"
#include "graph/reverb_node.h"
#include "graph/stereo_mixing_panner_node.h"
#include "graph/system_settings.h"
#include "node/sink_node.h"

namespace vraudio {

// The GraphManager class manages the construction and lifetime of the audio
// processing graph. It owns the output node that connects the audio processing
// graph to the audio hardware.
class GraphManager {
 public:
  // Initializes GraphManager class.
  //
  // @param system_settings Global system configuration.
  explicit GraphManager(const SystemSettings& system_settings);

  // Returns the sink node the audio graph is connected to.
  //
  // @return Shared pointer of the sink node.
  std::shared_ptr<SinkNode> GetSinkNode();

  // Triggers processing of the audio graph for all the connected nodes.
  void Process();

  // Returns a mutable pointer to the |AudioBuffer| of an audio source with
  // given |source_id|. Calls to this method must be synchronized with the audio
  // graph processing.
  //
  // @param source_id Source id.
  // @return Mutable audio buffer pointer. Nullptr if source_id not found.
  AudioBuffer* GetMutableAudioBuffer(SourceId source_id);

  // Creates an ambisonic panner source with given |sound_object_source_id|.
  //
  // Processing graph:
  //
  //                          +-------------------+
  //                          |                   |
  //             +------------+ SoundObjectSource +----------+
  //             |            |                   |          |
  //             |            +---------+---------+          |
  //             |                                           |
  // +-----------v-----------+                     +---------v----------+
  // |                       |                     |                    |
  // | AmbisonicMixingPanner |                     | StereoMixingPanner |
  // |                       |                     |                    |
  // +-----------+-----------+                     +---------+----------+
  //             |                                           |
  // +-----------v-----------+                     +---------v----------+
  // |                       |                     |                    |
  // |    AmbisonicMixer     |                     |    StereoMixer     |
  // |                       |                     |                    |
  // +-----------+-----------+                     +--------------------+
  //
  //
  // @param sound_object_source_id Id of sound object source.
  // @param enable_hrtf Flag to enable HRTF-based spatialization.
  void CreateAmbisonicPannerSource(SourceId sound_object_source_id,
                                   bool enable_hrtf);

  // Creates a new stereo non-spatialized source with given |stereo_source_id|.
  //
  // Processing graph:
  //
  //                            +--------------+
  //                            |              |
  //                            | StereoSource |
  //                            |              |
  //                            +-------+------+
  //                                    |
  //                            +-------v------+
  //                            |              |
  //                            |     Gain     |
  //                            |              |
  //                            +-------+------+
  //                                    |
  //                            +-------V------+
  //                            |              |
  //                            | StereoMixer  |
  //                            |              |
  //                            +--------------+
  //
  // @param stereo_source_id Id of new stereo source.
  void CreateStereoSource(SourceId stereo_source_id);

  // Destroys source with given |source_id|. Note that this call only sets a
  // flag to indicate that this source can be removed. The actual disconnect
  // happens from the audio processing thread the next time the processing graph
  // is triggered.
  //
  // @param source_id Id of source to be destroyed.
  void DestroySource(SourceId source_id);

  // Creates a new ambisonic source subgraph with given |ambisonic_source_id|.
  // Note: Ambisonic source subgraph is only created if the rendering mode is
  // HRTF.
  //
  // Processing graph (all the graphs created using http://asciiflow.com/):
  //
  //                           +-----------------+
  //                           |                 |
  //                   +-------+ AmbisonicSource +-------+
  //                   |       |                 |       |
  //                   |       +-----------------+       |
  //                   |                                 |
  //              +----v---+                  +----------v---------+
  //              |        |                  |                    |
  //              |  Gain  |               +--+ MonoFromSoundfield +--+
  //              |        |               |  |                    |  |
  //              +----+---+               |  +--------------------+  |
  //                   |                   |                          |
  //                   |                   |                          |
  //          +--------v-------+  +--------v---------+         +------v------+
  //          |                |  |                  |         |             |
  //          | Foa/HoaRotator |  | ReflectionsMixer |         | ReverbMixer |
  //          |                |  |                  |         |             |
  //          +--------+-------+  +--------+---------+         +------+------+
  //                   |
  //          +--------v-------+
  //          |                |
  //          | AmbisonicMixer |
  //          |                |
  //          +--------+-------+
  //
  // @param ambisonic_source_id Id of new ambisonic source.
  // @param num_channels Number of input channels of ambisonic source node.
  void CreateAmbisonicSource(SourceId ambisonic_source_id, size_t num_channels);

  // Creates a new sound object source with given |sound_object_source_id|.
  //
  // Processing graph:
  //
  //                          +-------------------+
  //                          |                   |
  //            +-------------+ SoundObjectSource +----------+
  //            |             |                   |          |
  //            |             +---------+---------+          |
  //            |                       |                    |
  // +----------v-----------+ +---------v---------+ +--------v--------+
  // |                      | |                   | |                 |
  // | ReflectionsGainMixer | | DirectAttenuation | | ReverbGainMixer |
  // |                      | |                   | |                 |
  // +----------+-----------+ +---------+---------+ +--------+--------+
  //                                    |
  //                          +---------v---------+
  //                  HRTF    |                   |    Stereo Panning
  //             +------------+     Occlusion     +----------+
  //             |            |                   |          |
  //             |            +---------+---------+          |
  //             |                      |                    |
  // +-----------v-----------+ +--------v--------+ +---------v----------+
  // |                       | |                 | |                    |
  // | AmbisonicMixingPanner | | NearFieldEffect | | StereoMixingPanner |
  // |                       | |                 | |                    |
  // +-----------+-----------+ +--------+--------+ +---------+----------+
  //             |                      |                    |
  // +-----------v-----------+ +--------v--------+           |
  // |                       | |                 |           |
  // |    AmbisonicMixer     | |   StereoMixer   <-----------+
  // |                       | |                 |
  // +-----------+-----------+ +-----------------+
  //
  //
  // @param sound_object_source_id Id of sound object source.
  // @param ambisonic_order Ambisonic order to encode the sound object source.
  // @param enable_hrtf Flag to enable HRTF-based rendering.
  // @param enable_direct_rendering Flag to enable direct source rendering.
  void CreateSoundObjectSource(SourceId sound_object_source_id,
                               int ambisonic_order, bool enable_hrtf,
                               bool enable_direct_rendering);

  // Mutes on/off the room effects mixers.
  //
  // @param Whether to enable room effects.
  void EnableRoomEffects(bool enable);

  // Returns the last processed output audio buffer of the ambisonic mix with
  // the highest possible ambisonic channel configuration. Note that, this
  // method will *not* trigger the processing of the audio graph.
  // |GraphManager::Process| must be called prior to this method call to ensure
  // that the output buffer is up-to-date.
  //
  // @return Output audio buffer of the ambisonic mix, or nullptr if no output.
  const AudioBuffer* GetAmbisonicBuffer() const;

  // Returns the last processed output audio buffer of the stereo (binaural)
  // mix. Note that, this method will *not* trigger the processing of the audio
  // graph. |GraphManager::Process| must be called prior to this method call to
  // ensure that the output buffer is up-to-date.
  //
  // @return Output audio buffer of the stereo mix, or nullptr if no output.
  const AudioBuffer* GetStereoBuffer() const;

  // Returns the maximum allowed number of ambisonic channels.
  //
  // @return Number of channels based on Ambisonic order in the global config.
  size_t GetNumMaxAmbisonicChannels() const;

  // Returns whether the room effects graph is enabled.
  //
  // @return True if room effects are enabled.
  bool GetRoomEffectsEnabled() const;

  // Updates the room reflections with the current properties for room effects
  // processing.
  void UpdateRoomReflections();

  // Updates the room reverb.
  void UpdateRoomReverb();

 private:
  // Initializes the Ambisonic renderer subgraph for the speficied Ambisonic
  // order and connects it to the |StereoMixerNode|.
  //
  // Processing graph:
  //
  //                         +------------------+
  //                         |                  |
  //                         |  AmbisonicMixer  |
  //                         |                  |
  //                         +--------+---------+
  //                                  |
  //                                  |
  //                     +------------v-------------+
  //                     |                          |
  //                     | AmbisonicBinauralDecoder |
  //                     |                          |
  //                     +------------+-------------+
  //                                  |
  //                                  |
  //                      +-----------v------------+
  //                      |                        |
  //                      |      StereoMixer       |
  //                      |                        |
  //                      +------------------------+
  //
  // @param ambisonic_order Ambisonic order.
  // @param sh_hrir_filename Filename to load the HRIR data from.
  void InitializeAmbisonicRendererGraph(int ambisonic_order,
                                        const std::string& sh_hrir_filename);

  // Helper method to lookup a source node with given |source_id|.
  //
  // @param source_id Source id.
  // @returns Shared pointer to source node instance, nullptr if not found.
  std::shared_ptr<BufferedSourceNode> LookupSourceNode(SourceId source_id);

  // Creates an audio subgraph that renders early reflections based on a room
  // model on a single mix.
  //
  // Processing graph:
  //
  //                       +---------------------------+
  //                       |                           |
  //                       |    ReflectionsGainMixer   |
  //                       |                           |
  //                       +-------------+-------------+
  //                                     |
  //                          +----------v----------+
  //                          |                     |
  //                          |     Reflections     |
  //                          |                     |
  //                          +----------+----------+
  //                                     |
  //                          +----------v----------+
  //                          |                     |
  //                          |   AmbisonicMixer    |
  //                          |                     |
  //                          +----------+----------+
  //
  void InitializeReflectionsGraph();

  // Creates an audio subgraph that renders a reverb from a mono mix of all the
  // sound objects based on a room model.
  //
  // Processing graph:
  //
  //                            +-----------------+
  //                            |                 |
  //                            | ReverbGainMixer |
  //                            |                 |
  //                            +--------+--------+
  //                                     |
  //                            +--------v--------+
  //                            |                 |
  //                            |     Reverb      |
  //                            |                 |
  //                            +--------+--------+
  //                                     |
  //                            +--------v--------+
  //                            |                 |
  //                            |   StereoMixer   |
  //                            |                 |
  //                            +-----------------+
  //
  void InitializeReverbGraph();

  // Flag indicating if room effects are enabled.
  bool room_effects_enabled_;

  // Mono mixer to accumulate all reverb sources.
  std::shared_ptr<GainMixerNode> reverb_gain_mixer_node_;

  // Reflections node.
  std::shared_ptr<ReflectionsNode> reflections_node_;

  // Mono mixer node to accumulate the early reflection sources.
  std::shared_ptr<GainMixerNode> reflections_gain_mixer_node_;

  // Reverb node.
  std::shared_ptr<ReverbNode> reverb_node_;

  // Ambisonic output mixer to accumulate incoming ambisonic inputs into a
  // single ambisonic output buffer.
  std::unique_ptr<Mixer> ambisonic_output_mixer_;

  // Global config passed in during construction.
  const GraphManagerConfig config_;

  // Manages system wide settings.
  const SystemSettings& system_settings_;

  // Provides Ambisonic encoding coefficients.
  std::unique_ptr<AmbisonicLookupTable> lookup_table_;

  // |FftManager| to be used in nodes that require FFT transformations.
  FftManager fft_manager_;

  // |Resampler| to be used to convert HRIRs to the system sample rate.
  Resampler resampler_;

  // Ambisonic mixer nodes per each ambisonic order to accumulate the
  // ambisonic sources for the corresponding binaural Ambisonic decoders.
  std::unordered_map<int, std::shared_ptr<MixerNode>> ambisonic_mixer_nodes_;

  // Stereo mixer to combine all the stereo and binaural output.
  std::shared_ptr<MixerNode> stereo_mixer_node_;

  // Ambisonic mixing encoder node to apply encoding coefficients and accumulate
  // the Ambisonic buffers.
  std::unordered_map<int, std::shared_ptr<AmbisonicMixingEncoderNode>>
      ambisonic_mixing_encoder_nodes_;

  // Stereo mixing panner node to apply stereo panning gains and accumulate the
  // buffers.
  std::shared_ptr<StereoMixingPannerNode> stereo_mixing_panner_node_;

  // Output node that enables audio playback of a single audio stream.
  std::shared_ptr<SinkNode> output_node_;

  // Holds all registered source nodes (independently of their type) and
  // allows look up by id.
  std::unordered_map<SourceId, std::shared_ptr<BufferedSourceNode>>
      source_nodes_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_GRAPH_MANAGER_H_
