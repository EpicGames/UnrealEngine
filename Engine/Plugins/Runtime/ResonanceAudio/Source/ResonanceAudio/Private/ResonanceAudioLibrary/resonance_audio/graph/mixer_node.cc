/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/mixer_node.h"



namespace vraudio {

MixerNode::MixerNode(const SystemSettings& system_settings, size_t num_channels)
    : num_channels_(num_channels),
      mixer_(num_channels_, system_settings.GetFramesPerBuffer()) {
  DCHECK_NE(num_channels_, 0U);
  EnableProcessOnEmptyInput(true);
}

const AudioBuffer* MixerNode::GetOutputBuffer() const {
  return mixer_.GetOutput();
}

bool MixerNode::CleanUp() {
  CallCleanUpOnInputNodes();
  // Prevent node from being disconnected when all sources are removed.
  return false;
}

const AudioBuffer* MixerNode::AudioProcess(const NodeInput& input) {


  mixer_.Reset();

  const auto& input_buffers = input.GetInputBuffers();
  if (input_buffers.empty()) {
    return nullptr;
  }

  for (auto input_buffer : input_buffers) {
    DCHECK_EQ(input_buffer->num_channels(), num_channels_);
    mixer_.AddInput(*input_buffer);
  }
  return mixer_.GetOutput();
}

}  // namespace vraudio
