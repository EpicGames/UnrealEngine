/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_MIXER_NODE_H_
#define RESONANCE_AUDIO_GRAPH_MIXER_NODE_H_

#include "base/audio_buffer.h"
#include "dsp/mixer.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Accepts multiple input buffers and outputs a downmix to a single output
// buffer. All input buffers must have the same number of channels and the same
// number of frames per buffer.
class MixerNode : public ProcessingNode {
 public:
  MixerNode(const SystemSettings& system_settings, size_t num_channels);

  // Returns the current output buffer of the mixer.
  //
  // @return Output audio buffer.
  const AudioBuffer* GetOutputBuffer() const;

  // Node implementation.
  bool CleanUp() final;

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  const size_t num_channels_;

  Mixer mixer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_MIXER_NODE_H_
