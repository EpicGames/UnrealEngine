/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/mono_from_soundfield_node.h"

#include "base/constants_and_types.h"

namespace vraudio {

MonoFromSoundfieldNode::MonoFromSoundfieldNode(
    SourceId source_id, const SystemSettings& system_settings)
    : output_buffer_(kNumMonoChannels, system_settings.GetFramesPerBuffer()) {
  output_buffer_.set_source_id(source_id);
  output_buffer_.Clear();
}

const AudioBuffer* MonoFromSoundfieldNode::AudioProcess(
    const NodeInput& input) {


  const AudioBuffer* input_buffer = input.GetSingleInput();
  DCHECK(input_buffer);
  DCHECK_EQ(input_buffer->source_id(), output_buffer_.source_id());
  DCHECK_NE(input_buffer->num_channels(), 0U);
  DCHECK_EQ(input_buffer->num_frames(), output_buffer_.num_frames());
  // Get W channel of the ambisonic input.
  output_buffer_[0] = (*input_buffer)[0];

  return &output_buffer_;
}

}  // namespace vraudio
