/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_MONO_FROM_SOUNDFIELD_NODE_H_
#define RESONANCE_AUDIO_GRAPH_MONO_FROM_SOUNDFIELD_NODE_H_

#include "base/audio_buffer.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts an ambisonic buffer as input and extracts its W channel
// onto a mono output buffer.
class MonoFromSoundfieldNode : public ProcessingNode {
 public:
  MonoFromSoundfieldNode(SourceId source_id,
                         const SystemSettings& system_settings);

 protected:
  // Implements |ProcessingNode|.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  // Mono audio buffer to store output data.
  AudioBuffer output_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_MONO_FROM_SOUNDFIELD_NODE_H_
