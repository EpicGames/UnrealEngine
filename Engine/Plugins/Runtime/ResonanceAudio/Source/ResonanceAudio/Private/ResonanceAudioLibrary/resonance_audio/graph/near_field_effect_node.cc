/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/near_field_effect_node.h"

#include <algorithm>
#include <cmath>

#include "base/logging.h"
#include "base/spherical_angle.h"

#include "dsp/distance_attenuation.h"
#include "dsp/gain.h"
#include "dsp/stereo_panner.h"

namespace vraudio {

NearFieldEffectNode::NearFieldEffectNode(SourceId source_id,
                                         const SystemSettings& system_settings)
    : pan_gains_({0.0f, 0.0f}),
      near_field_processor_(system_settings.GetSampleRateHz(),
                            system_settings.GetFramesPerBuffer()),
      system_settings_(system_settings),
      output_buffer_(kNumStereoChannels, system_settings.GetFramesPerBuffer()) {
  output_buffer_.set_source_id(source_id);
}

const AudioBuffer* NearFieldEffectNode::AudioProcess(const NodeInput& input) {


  const AudioBuffer* input_buffer = input.GetSingleInput();
  DCHECK(input_buffer);
  DCHECK_EQ(input_buffer->num_channels(), 1U);
  DCHECK_EQ(input_buffer->source_id(), output_buffer_.source_id());

  const auto source_parameters =
      system_settings_.GetSourceParameters(input_buffer->source_id());
  if (source_parameters == nullptr) {
    LOG(WARNING) << "Could not find source parameters";
    return nullptr;
  }


  DCHECK_EQ(pan_gains_.size(), kNumStereoChannels);
  const float near_field_gain = source_parameters->near_field_gain;
  if (near_field_gain > 0.0f) {
    const auto& listener_position = system_settings_.GetHeadPosition();
    const auto& listener_rotation = system_settings_.GetHeadRotation();
    const auto& source_transform = source_parameters->object_transform;
    // Compute the relative source direction in spherical angles to calculate
    // the left and right panner gains.
    WorldPosition relative_direction;
    GetRelativeDirection(listener_position, listener_rotation,
                         source_transform.position, &relative_direction);
    const auto source_direction =
        SphericalAngle::FromWorldPosition(relative_direction);
    CalculateStereoPanGains(source_direction, &pan_gains_);
    // Combine pan gains with per-source near field gain.
    const float total_near_field_gain =
        ComputeNearFieldEffectGain(listener_position,
                                   source_transform.position) *
        near_field_gain / kMaxNearFieldEffectGain;
    for (size_t i = 0; i < pan_gains_.size(); ++i) {
      pan_gains_[i] *= total_near_field_gain;
    }
  } else {
    // Disable near field effect if |near_field_gain| is zero.
    std::fill(pan_gains_.begin(), pan_gains_.end(), 0.0f);
  }

  const float left_current_gain = left_panner_.GetGain();
  const float right_current_gain = right_panner_.GetGain();
  const float left_target_gain = pan_gains_[0];
  const float right_target_gain = pan_gains_[1];
  const bool is_left_zero_gain =
      IsGainNearZero(left_current_gain) && IsGainNearZero(left_target_gain);
  const bool is_right_zero_gain =
      IsGainNearZero(right_current_gain) && IsGainNearZero(right_target_gain);

  if (is_left_zero_gain && is_right_zero_gain) {
    // Make sure gain processors are initialized.
    left_panner_.Reset(0.0f);
    right_panner_.Reset(0.0f);
    // Both channels go to zero, there is no need for further processing.
    return nullptr;
  }

  const auto& input_channel = (*input_buffer)[0];
  auto* left_output_channel = &output_buffer_[0];
  auto* right_output_channel = &output_buffer_[1];
  // Apply bass boost and delay compensation (if necessary) to the input signal
  // and place it temporarily in the right output channel. This way we avoid
  // allocating a temporary buffer.
  near_field_processor_.Process(input_channel, right_output_channel,
                                source_parameters->enable_hrtf);
  left_panner_.ApplyGain(left_target_gain, *right_output_channel,
                         left_output_channel, /*accumulate_output=*/false);
  right_panner_.ApplyGain(right_target_gain, *right_output_channel,
                          right_output_channel, /*accumulate_output=*/false);

  return &output_buffer_;
}

}  // namespace vraudio
