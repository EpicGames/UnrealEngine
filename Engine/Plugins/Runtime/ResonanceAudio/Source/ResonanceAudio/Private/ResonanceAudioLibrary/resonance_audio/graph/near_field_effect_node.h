/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_NEAR_FIELD_EFFECT_NODE_H_
#define RESONANCE_AUDIO_GRAPH_NEAR_FIELD_EFFECT_NODE_H_

#include <vector>

#include "base/audio_buffer.h"
#include "base/constants_and_types.h"
#include "dsp/gain_processor.h"
#include "dsp/near_field_processor.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts a single mono audio buffer as input, applies an appoximate
// near field effect and outputs a processed stereo audio buffer. The stereo
// output buffer can then be combined with a binaural output in order to
// simulate a sound source which is close (<1m) to the listener's head.
class NearFieldEffectNode : public ProcessingNode {
 public:
  // Constructor.
  //
  // @param source_id Output buffer source id.
  // @param system_settings Global system settings.
  NearFieldEffectNode(SourceId source_id,
                      const SystemSettings& system_settings);

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  // Left and right processors apply both near field gain and panner gains.
  GainProcessor left_panner_;
  GainProcessor right_panner_;

  // Left and right gains for near field and panning combined.
  std::vector<float> pan_gains_;

  // Near field processor used to apply approximate near field effect to the
  // mono source signal.
  NearFieldProcessor near_field_processor_;

  // Used to obtain head rotation.
  const SystemSettings& system_settings_;

  // Output buffer.
  AudioBuffer output_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_NEAR_FIELD_EFFECT_NODE_H_
