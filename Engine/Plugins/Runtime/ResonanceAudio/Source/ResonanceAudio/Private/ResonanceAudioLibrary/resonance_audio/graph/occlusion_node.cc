/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/occlusion_node.h"

#include <cmath>

#include "base/logging.h"
#include "base/spherical_angle.h"

#include "dsp/occlusion_calculator.h"

namespace vraudio {

namespace {

// Low pass filter coefficient for smoothing the applied occlusion. This avoids
// sudden unrealistic changes in the volume of a sound object. Range [0, 1].
// The value below has been calculated empirically.
const float kOcclusionSmoothingCoefficient = 0.75f;

// This function provides first order low-pass filtering. It is used to smooth
// the occlusion parameter.
float Interpolate(float coefficient, float previous_value, float target_value) {
  return target_value + coefficient * (previous_value - target_value);
}

}  // namespace

OcclusionNode::OcclusionNode(SourceId source_id,
                             const SystemSettings& system_settings)
    : system_settings_(system_settings),
      low_pass_filter_(0.0f),
      current_occlusion_(0.0f),
      output_buffer_(kNumMonoChannels, system_settings.GetFramesPerBuffer()) {
  output_buffer_.Clear();
  output_buffer_.set_source_id(source_id);
}

const AudioBuffer* OcclusionNode::AudioProcess(const NodeInput& input) {

  const AudioBuffer* input_buffer = input.GetSingleInput();
  DCHECK(input_buffer);
  DCHECK_EQ(input_buffer->source_id(), output_buffer_.source_id());

  const auto source_parameters =
      system_settings_.GetSourceParameters(input_buffer->source_id());
  if (source_parameters == nullptr) {
    LOG(WARNING) << "Could not find source parameters";
    return nullptr;
  }

  const WorldPosition& listener_position = system_settings_.GetHeadPosition();
  const WorldRotation& listener_rotation = system_settings_.GetHeadRotation();
  const ObjectTransform& source_transform = source_parameters->object_transform;
  // Compute the relative listener/source direction in spherical angles.
  WorldPosition relative_direction;
  GetRelativeDirection(listener_position, listener_rotation,
                       source_transform.position, &relative_direction);
  const SphericalAngle listener_direction =
      SphericalAngle::FromWorldPosition(relative_direction);

  GetRelativeDirection(source_transform.position, source_transform.rotation,
                       listener_position, &relative_direction);
  const SphericalAngle source_direction =
      SphericalAngle::FromWorldPosition(relative_direction);
  // Calculate low-pass filter coefficient based on listener/source directivity
  // and occlusion values.
  const float listener_directivity = CalculateDirectivity(
      source_parameters->listener_directivity_alpha,
      source_parameters->listener_directivity_order, listener_direction);
  const float source_directivity = CalculateDirectivity(
      source_parameters->directivity_alpha,
      source_parameters->directivity_order, source_direction);
  current_occlusion_ =
      Interpolate(kOcclusionSmoothingCoefficient, current_occlusion_,
                  source_parameters->occlusion_intensity);
  const float filter_coefficient = CalculateOcclusionFilterCoefficient(
      listener_directivity * source_directivity, current_occlusion_);
  low_pass_filter_.SetCoefficient(filter_coefficient);
  if (!low_pass_filter_.Filter((*input_buffer)[0], &output_buffer_[0])) {
    return input_buffer;
  }
  // Copy buffer parameters.
  return &output_buffer_;
}

}  // namespace vraudio
