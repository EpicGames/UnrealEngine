/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_OCCLUSION_NODE_H_
#define RESONANCE_AUDIO_GRAPH_OCCLUSION_NODE_H_

#include "base/audio_buffer.h"
#include "dsp/mono_pole_filter.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts a single audio buffer as input and outputs the input buffer
// with its cuttoff frequency scaled by listener/source directivity and
// occlusion intensity.
class OcclusionNode : public ProcessingNode {
 public:
  // Constructor.
  //
  // @param source_id Output buffer source id.
  // @param system_settings Global system settings.
  OcclusionNode(SourceId source_id, const SystemSettings& system_settings);

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  friend class OcclusionNodeTest;

  const SystemSettings& system_settings_;

  // Used to low-pass input audio when a source is occluded or self-occluded.
  MonoPoleFilter low_pass_filter_;

  // Occlusion intensity value for the current input buffer.
  float current_occlusion_;

  // Output buffer.
  AudioBuffer output_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_OCCLUSION_NODE_H_
