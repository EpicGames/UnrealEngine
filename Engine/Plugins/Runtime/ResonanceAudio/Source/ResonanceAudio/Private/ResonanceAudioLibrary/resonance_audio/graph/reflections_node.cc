/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/reflections_node.h"

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"


namespace vraudio {

ReflectionsNode::ReflectionsNode(const SystemSettings& system_settings)
    : system_settings_(system_settings),
      reflections_processor_(system_settings_.GetSampleRateHz(),
                             system_settings_.GetFramesPerBuffer()),
      num_frames_processed_on_empty_input_(
          system_settings_.GetFramesPerBuffer()),
      output_buffer_(kNumFirstOrderAmbisonicChannels,
                     system_settings_.GetFramesPerBuffer()),
      silence_mono_buffer_(kNumMonoChannels,
                           system_settings_.GetFramesPerBuffer()) {
  silence_mono_buffer_.Clear();
  EnableProcessOnEmptyInput(true);
}

void ReflectionsNode::Update() {

  const auto& current_reflection_properties = reflection_properties_;
  const auto& new_reflection_properties =
      system_settings_.GetReflectionProperties();
  const bool room_position_changed =
      !EqualSafe(std::begin(current_reflection_properties.room_position),
                 std::end(current_reflection_properties.room_position),
                 std::begin(new_reflection_properties.room_position),
                 std::end(new_reflection_properties.room_position));
  const bool room_rotation_changed =
      !EqualSafe(std::begin(current_reflection_properties.room_rotation),
                 std::end(current_reflection_properties.room_rotation),
                 std::begin(new_reflection_properties.room_rotation),
                 std::end(new_reflection_properties.room_rotation));
  const bool room_dimensions_changed =
      !EqualSafe(std::begin(current_reflection_properties.room_dimensions),
                 std::end(current_reflection_properties.room_dimensions),
                 std::begin(new_reflection_properties.room_dimensions),
                 std::end(new_reflection_properties.room_dimensions));
  const bool cutoff_frequency_changed =
      current_reflection_properties.cutoff_frequency !=
      new_reflection_properties.cutoff_frequency;
  const bool coefficients_changed =
      !EqualSafe(std::begin(current_reflection_properties.coefficients),
                 std::end(current_reflection_properties.coefficients),
                 std::begin(new_reflection_properties.coefficients),
                 std::end(new_reflection_properties.coefficients));
  const auto& current_listener_position = listener_position_;
  const auto& new_listener_position = system_settings_.GetHeadPosition();
  const bool listener_position_changed =
      current_listener_position != new_listener_position;
  if (room_position_changed || room_rotation_changed ||
      room_dimensions_changed || cutoff_frequency_changed ||
      coefficients_changed || listener_position_changed) {
    // Update reflections processor if necessary.
    reflection_properties_ = new_reflection_properties;
    listener_position_ = new_listener_position;
    reflections_processor_.Update(reflection_properties_, listener_position_);
  }
}

const AudioBuffer* ReflectionsNode::AudioProcess(const NodeInput& input) {

  const AudioBuffer* input_buffer = input.GetSingleInput();
  const size_t num_frames = system_settings_.GetFramesPerBuffer();
  if (input_buffer == nullptr) {
    // If we have no input, generate a silent input buffer until the node states
    // are cleared.
    if (num_frames_processed_on_empty_input_ <
        reflections_processor_.num_frames_to_process_on_empty_input()) {
      num_frames_processed_on_empty_input_ += num_frames;
      input_buffer = &silence_mono_buffer_;
    } else {
      // Skip processing entirely when the states are fully cleared.
      return nullptr;
    }
  } else {
    num_frames_processed_on_empty_input_ = 0;
    DCHECK_EQ(input_buffer->num_channels(), kNumMonoChannels);
  }
  output_buffer_.Clear();
  reflections_processor_.Process(*input_buffer, &output_buffer_);

  // Rotate the reflections with respect to listener's orientation.
  const WorldRotation inverse_head_rotation =
      system_settings_.GetHeadRotation().conjugate();
  foa_rotator_.Process(inverse_head_rotation, output_buffer_, &output_buffer_);

  // Copy buffer parameters.
  return &output_buffer_;
}

}  // namespace vraudio
