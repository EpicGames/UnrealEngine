/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_REFLECTIONS_NODE_H_
#define RESONANCE_AUDIO_GRAPH_REFLECTIONS_NODE_H_

#include <vector>

#include "ambisonics/foa_rotator.h"
#include "api/resonance_audio_api.h"
#include "base/audio_buffer.h"
#include "base/misc_math.h"
#include "dsp/reflections_processor.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts a single mono buffer as input and outputs an ambisonically
// encoded sound field buffer of the mix of all the early room reflections.
class ReflectionsNode : public ProcessingNode {
 public:
  // Initializes |ReflectionsNode| class.
  //
  // @param system_settings Global system configuration.
  explicit ReflectionsNode(const SystemSettings& system_settings);

  // Updates the reflections. Depending on whether to use RT60s for reverb
  // according to the global system settings, the reflections are calculated
  // either by the current room properties or the proxy room properties.
  void Update();

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  const SystemSettings& system_settings_;

  // First-order-ambisonics rotator to be used to rotate the reflections with
  // respect to the listener's orientation.
  FoaRotator foa_rotator_;

  // Processes and encodes reflections into an ambisonic buffer.
  ReflectionsProcessor reflections_processor_;

  // Most recently updated reflection properties.
  ReflectionProperties reflection_properties_;

  // Most recently updated listener position.
  WorldPosition listener_position_;

  size_t num_frames_processed_on_empty_input_;

  // Ambisonic output buffer.
  AudioBuffer output_buffer_;

  // Silence mono buffer to render reflection tails during the absence of input
  // buffers.
  AudioBuffer silence_mono_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_REFLECTIONS_NODE_H_
