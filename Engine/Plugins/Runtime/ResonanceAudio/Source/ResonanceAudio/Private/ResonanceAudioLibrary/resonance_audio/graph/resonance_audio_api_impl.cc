/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/resonance_audio_api_impl.h"

#include <algorithm>
#include <numeric>

#include "ambisonics/utils.h"
#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/misc_math.h"
#include "base/source_parameters.h"

#include "base/unique_ptr_wrapper.h"
#include "config/source_config.h"
#include "dsp/channel_converter.h"
#include "dsp/distance_attenuation.h"
#include "graph/source_parameters_manager.h"
#include "utils/planar_interleaved_conversion.h"
#include "utils/sample_type_conversion.h"

namespace vraudio {

namespace {

// Support 50 setter calls for 512 sources.
const size_t kMaxNumTasksOnTaskQueue = 50 * 512;

// User warning/notification messages.
static const char* kBadInputPointerMessage = "Ignoring nullptr buffer";
static const char* kBufferSizeMustMatchNumFramesMessage =
    "Number of frames must match the frames per buffer specified during "
    "construction - ignoring buffer";

// Helper method to fetch |SourceGraphConfig| from |RenderingMode|.
SourceGraphConfig GetSourceGraphConfigFromRenderingMode(
    RenderingMode rendering_mode) {
  switch (rendering_mode) {
    case RenderingMode::kStereoPanning:
      return StereoPanningConfig();
    case RenderingMode::kBinauralLowQuality:
      return BinauralLowQualityConfig();
    case RenderingMode::kBinauralMediumQuality:
      return BinauralMediumQualityConfig();
    case RenderingMode::kBinauralHighQuality:
      return BinauralHighQualityConfig();
    case RenderingMode::kRoomEffectsOnly:
      return RoomEffectsOnlyConfig();
    default:
      LOG(FATAL) << "Unknown rendering mode";
      break;
  }
  return BinauralHighQualityConfig();
}

}  // namespace

ResonanceAudioApiImpl::ResonanceAudioApiImpl(size_t num_channels,
                                             size_t frames_per_buffer,
                                             int sample_rate_hz)
    : system_settings_(num_channels, frames_per_buffer, sample_rate_hz),
      task_queue_(kMaxNumTasksOnTaskQueue),
      source_id_counter_(0) {
  if (num_channels != kNumStereoChannels) {
    LOG(FATAL) << "Only stereo output is supported";
    return;
  }

  if (frames_per_buffer > kMaxSupportedNumFrames) {
    LOG(FATAL) << "Only frame lengths up to " << kMaxSupportedNumFrames
               << " are supported.";
    return;
  }

  // The pffft library requires a minimum buffer size of 32 samples.
  if (frames_per_buffer < FftManager::kMinFftSize) {
    LOG(FATAL) << "The minimum number of frames per buffer is "
               << FftManager::kMinFftSize << " samples";
    return;
  }
  graph_manager_.reset(new GraphManager(system_settings_));
}

ResonanceAudioApiImpl::~ResonanceAudioApiImpl() {
  // Clear task queue before shutting down.
  task_queue_.Execute();
}

bool ResonanceAudioApiImpl::FillInterleavedOutputBuffer(size_t num_channels,
                                                        size_t num_frames,
                                                        float* buffer_ptr) {
  DCHECK(buffer_ptr);
  return FillOutputBuffer<float*>(num_channels, num_frames, buffer_ptr);
}

bool ResonanceAudioApiImpl::FillInterleavedOutputBuffer(size_t num_channels,
                                                        size_t num_frames,
                                                        int16* buffer_ptr) {
  DCHECK(buffer_ptr);
  return FillOutputBuffer<int16*>(num_channels, num_frames, buffer_ptr);
}

bool ResonanceAudioApiImpl::FillPlanarOutputBuffer(size_t num_channels,
                                                   size_t num_frames,
                                                   float* const* buffer_ptr) {
  DCHECK(buffer_ptr);
  return FillOutputBuffer<float* const*>(num_channels, num_frames, buffer_ptr);
}

bool ResonanceAudioApiImpl::FillPlanarOutputBuffer(size_t num_channels,
                                                   size_t num_frames,
                                                   int16* const* buffer_ptr) {
  DCHECK(buffer_ptr);
  return FillOutputBuffer<int16* const*>(num_channels, num_frames, buffer_ptr);
}

void ResonanceAudioApiImpl::SetHeadPosition(float x, float y, float z) {
  auto task = [this, x, y, z]() {
    const WorldPosition head_position(x, y, z);
    system_settings_.SetHeadPosition(head_position);
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetHeadRotation(float x, float y, float z,
                                            float w) {
  auto task = [this, w, x, y, z]() {
    const WorldRotation head_rotation(w, x, y, z);
    system_settings_.SetHeadRotation(head_rotation);
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetPrimaryVolume(float volume) {
  auto task = [this, volume]() { system_settings_.SetPrimaryGain(volume); };
  task_queue_.Post(task);
}

int ResonanceAudioApiImpl::CreateAmbisonicSource(size_t num_channels) {
  if (num_channels < kNumFirstOrderAmbisonicChannels ||
      !IsValidAmbisonicOrder(num_channels)) {
    // Invalid number of input channels, don't create the ambisonic source.
    LOG(ERROR) << "Invalid number of channels for the ambisonic source: "
               << num_channels;
    return kInvalidSourceId;
  }

  const int ambisonic_source_id = source_id_counter_.fetch_add(1);

  const size_t num_valid_channels =
      std::min(num_channels, graph_manager_->GetNumMaxAmbisonicChannels());
  if (num_valid_channels < num_channels) {
    LOG(WARNING) << "Number of ambisonic channels will be diminished to "
                 << num_valid_channels;
  }

  auto task = [this, ambisonic_source_id, num_valid_channels]() {
    graph_manager_->CreateAmbisonicSource(ambisonic_source_id,
                                          num_valid_channels);
    system_settings_.GetSourceParametersManager()->Register(
        ambisonic_source_id);
    // Overwrite default source parameters for ambisonic source.
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            ambisonic_source_id);
    source_parameters->room_effects_gain = 0.0f;
    source_parameters->distance_rolloff_model = DistanceRolloffModel::kNone;
    source_parameters->distance_attenuation = 1.0f;
  };
  task_queue_.Post(task);
  return ambisonic_source_id;
}

int ResonanceAudioApiImpl::CreateStereoSource(size_t num_channels) {
  if (num_channels > kNumStereoChannels) {
    LOG(ERROR) << "Unsupported number of input channels";
    return kInvalidSourceId;
  }
  const int stereo_source_id = source_id_counter_.fetch_add(1);

  auto task = [this, stereo_source_id]() {
    graph_manager_->CreateStereoSource(stereo_source_id);
    system_settings_.GetSourceParametersManager()->Register(stereo_source_id);
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            stereo_source_id);
    source_parameters->enable_hrtf = false;
  };
  task_queue_.Post(task);
  return stereo_source_id;
}

int ResonanceAudioApiImpl::CreateSoundObjectSource(
    RenderingMode rendering_mode) {
  const int sound_object_source_id = source_id_counter_.fetch_add(1);

  const auto config = GetSourceGraphConfigFromRenderingMode(rendering_mode);
  auto task = [this, sound_object_source_id, config]() {
    graph_manager_->CreateSoundObjectSource(
        sound_object_source_id, config.ambisonic_order, config.enable_hrtf,
        config.enable_direct_rendering);
    system_settings_.GetSourceParametersManager()->Register(
        sound_object_source_id);
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    source_parameters->enable_hrtf = config.enable_hrtf;
  };
  task_queue_.Post(task);
  return sound_object_source_id;
}

void ResonanceAudioApiImpl::DestroySource(SourceId source_id) {
  auto task = [this, source_id]() {
    graph_manager_->DestroySource(source_id);
    system_settings_.GetSourceParametersManager()->Unregister(source_id);
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetInterleavedBuffer(SourceId source_id,
                                                 const float* audio_buffer_ptr,
                                                 size_t num_channels,
                                                 size_t num_frames) {
  SetSourceBuffer<const float*>(source_id, audio_buffer_ptr, num_channels,
                                num_frames);
}

void ResonanceAudioApiImpl::SetInterleavedBuffer(SourceId source_id,
                                                 const int16* audio_buffer_ptr,
                                                 size_t num_channels,
                                                 size_t num_frames) {
  SetSourceBuffer<const int16*>(source_id, audio_buffer_ptr, num_channels,
                                num_frames);
}

void ResonanceAudioApiImpl::SetPlanarBuffer(
    SourceId source_id, const float* const* audio_buffer_ptr,
    size_t num_channels, size_t num_frames) {
  SetSourceBuffer<const float* const*>(source_id, audio_buffer_ptr,
                                       num_channels, num_frames);
}

void ResonanceAudioApiImpl::SetPlanarBuffer(
    SourceId source_id, const int16* const* audio_buffer_ptr,
    size_t num_channels, size_t num_frames) {
  SetSourceBuffer<const int16* const*>(source_id, audio_buffer_ptr,
                                       num_channels, num_frames);
}

void ResonanceAudioApiImpl::SetSourceDistanceAttenuation(
    SourceId source_id, float distance_attenuation) {
  auto task = [this, source_id, distance_attenuation]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      const auto& rolloff_model = source_parameters->distance_rolloff_model;
      DCHECK_EQ(rolloff_model, DistanceRolloffModel::kNone);
      if (rolloff_model != DistanceRolloffModel::kNone) {
        LOG(WARNING) << "Implicit distance rolloff model is set. The value "
                        "will be overwritten.";
      }
      source_parameters->distance_attenuation = distance_attenuation;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSourceDistanceModel(SourceId source_id,
                                                   DistanceRolloffModel rolloff,
                                                   float min_distance,
                                                   float max_distance) {
  if (max_distance < min_distance && rolloff != DistanceRolloffModel::kNone) {
    LOG(WARNING) << "max_distance must be larger than min_distance";
    return;
  }
  auto task = [this, source_id, rolloff, min_distance, max_distance]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      source_parameters->distance_rolloff_model = rolloff;
      source_parameters->minimum_distance = min_distance;
      source_parameters->maximum_distance = max_distance;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSourcePosition(SourceId source_id, float x,
                                              float y, float z) {
  const WorldPosition position(x, y, z);
  auto task = [this, source_id, position]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      source_parameters->object_transform.position = position;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSourceRoomEffectsGain(SourceId source_id,
                                                     float room_effects_gain) {
  auto task = [this, source_id, room_effects_gain]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      source_parameters->room_effects_gain = room_effects_gain;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSourceRotation(SourceId source_id, float x,
                                              float y, float z, float w) {
  const WorldRotation rotation(w, x, y, z);
  auto task = [this, source_id, rotation]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      source_parameters->object_transform.rotation = rotation;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSourceVolume(SourceId source_id, float volume) {
  auto task = [this, source_id, volume]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            source_id);
    if (source_parameters != nullptr) {
      source_parameters->gain = volume;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSoundObjectDirectivity(
    SourceId sound_object_source_id, float alpha, float order) {
  auto task = [this, sound_object_source_id, alpha, order]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    if (source_parameters != nullptr) {
      source_parameters->directivity_alpha = alpha;
      source_parameters->directivity_order = order;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSoundObjectListenerDirectivity(
    SourceId sound_object_source_id, float alpha, float order) {
  auto task = [this, sound_object_source_id, alpha, order]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    if (source_parameters != nullptr) {
      source_parameters->listener_directivity_alpha = alpha;
      source_parameters->listener_directivity_order = order;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSoundObjectNearFieldEffectGain(
    SourceId sound_object_source_id, float gain) {
  auto task = [this, sound_object_source_id, gain]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    if (source_parameters != nullptr) {
      source_parameters->near_field_gain = gain;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSoundObjectOcclusionIntensity(
    SourceId sound_object_source_id, float intensity) {
  auto task = [this, sound_object_source_id, intensity]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    if (source_parameters != nullptr) {
      source_parameters->occlusion_intensity = intensity;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetSoundObjectSpread(
    SourceId sound_object_source_id, float spread_deg) {
  auto task = [this, sound_object_source_id, spread_deg]() {
    auto source_parameters =
        system_settings_.GetSourceParametersManager()->GetMutableParameters(
            sound_object_source_id);
    if (source_parameters != nullptr) {
      source_parameters->spread_deg = spread_deg;
    }
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::EnableRoomEffects(bool enable) {
  auto task = [this, enable]() { graph_manager_->EnableRoomEffects(enable); };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetReflectionProperties(
    const ReflectionProperties& reflection_properties) {
  auto task = [this, reflection_properties]() {
    system_settings_.SetReflectionProperties(reflection_properties);
  };
  task_queue_.Post(task);
}

void ResonanceAudioApiImpl::SetReverbProperties(
    const ReverbProperties& reverb_properties) {
  auto task = [this, reverb_properties]() {
    system_settings_.SetReverbProperties(reverb_properties);
  };
  task_queue_.Post(task);
}

const AudioBuffer* ResonanceAudioApiImpl::GetAmbisonicOutputBuffer() const {
  return graph_manager_->GetAmbisonicBuffer();
}

const AudioBuffer* ResonanceAudioApiImpl::GetStereoOutputBuffer() const {
  return graph_manager_->GetStereoBuffer();
}

void ResonanceAudioApiImpl::ProcessNextBuffer() {
#if defined(ENABLE_TRACING) && !ION_PRODUCTION
  // This enables tracing on the audio thread.
  auto task = []() { ENABLE_TRACING_ON_CURRENT_THREAD("AudioThread"); };
  task_queue_.Post(task);
#endif  // defined(ENABLE_TRACING) && !ION_PRODUCTION


  task_queue_.Execute();

  // Update room effects only if the pipeline is initialized.
  if (graph_manager_->GetRoomEffectsEnabled()) {
    graph_manager_->UpdateRoomReflections();
    graph_manager_->UpdateRoomReverb();
  }
  // Update source attenuation parameters.
  const auto process = [this](SourceParameters* parameters) {
    const float primary_gain = system_settings_.GetPrimaryGain();
    const auto& listener_position = system_settings_.GetHeadPosition();
    const auto& reflection_properties =
        system_settings_.GetReflectionProperties();
    const auto& reverb_properties = system_settings_.GetReverbProperties();
    UpdateAttenuationParameters(primary_gain, reflection_properties.gain,
                                reverb_properties.gain, listener_position,
                                parameters);
  };
  system_settings_.GetSourceParametersManager()->ProcessAllParameters(process);

  graph_manager_->Process();
}

void ResonanceAudioApiImpl::SetStereoSpeakerMode(bool enabled) {
  auto task = [this, enabled]() {
    system_settings_.SetStereoSpeakerMode(enabled);
  };
  task_queue_.Post(task);
}

template <typename OutputType>
bool ResonanceAudioApiImpl::FillOutputBuffer(size_t num_channels,
                                             size_t num_frames,
                                             OutputType buffer_ptr) {


  if (buffer_ptr == nullptr) {
    LOG(WARNING) << kBadInputPointerMessage;
    return false;
  }
  if (num_channels != kNumStereoChannels) {
    LOG(WARNING) << "Output buffer must be stereo";
    return false;
  }
  const size_t num_input_samples = num_frames * num_channels;
  const size_t num_expected_output_samples =
      system_settings_.GetFramesPerBuffer() * system_settings_.GetNumChannels();
  if (num_input_samples != num_expected_output_samples) {
    LOG(WARNING) << "Output buffer size must be " << num_expected_output_samples
                 << " samples";
    return false;
  }

  // Get the processed output buffer.
  ProcessNextBuffer();
  const AudioBuffer* output_buffer = GetStereoOutputBuffer();
  if (output_buffer == nullptr) {
    // This indicates that the graph processing is triggered without having any
    // connected sources.
    return false;
  }

  FillExternalBuffer(*output_buffer, buffer_ptr, num_frames, num_channels);
  return true;
}

template <typename SampleType>
void ResonanceAudioApiImpl::SetSourceBuffer(SourceId source_id,
                                            SampleType audio_buffer_ptr,
                                            size_t num_input_channels,
                                            size_t num_frames) {
  // Execute task queue to ensure newly created sound sources are initialized.
  task_queue_.Execute();

  if (audio_buffer_ptr == nullptr) {
    LOG(WARNING) << kBadInputPointerMessage;
    return;
  }
  if (num_frames != system_settings_.GetFramesPerBuffer()) {
    LOG(WARNING) << kBufferSizeMustMatchNumFramesMessage;
    return;
  }

  AudioBuffer* const output_buffer =
      graph_manager_->GetMutableAudioBuffer(source_id);
  if (output_buffer == nullptr) {
    LOG(WARNING) << "Source audio buffer not found";
    return;
  }
  const size_t num_output_channels = output_buffer->num_channels();

  if (num_input_channels == num_output_channels) {
    FillAudioBuffer(audio_buffer_ptr, num_frames, num_input_channels,
                    output_buffer);

    return;
  }

  if ((num_input_channels == kNumMonoChannels) &&
      (num_output_channels == kNumStereoChannels)) {
    FillAudioBufferWithChannelRemapping(
        audio_buffer_ptr, num_frames, num_input_channels,
        {0, 0} /* channel_map */, output_buffer);
    return;
  }

  if (num_input_channels > num_output_channels) {
    std::vector<size_t> channel_map(num_output_channels);
    // Fill channel map with increasing indices.
    std::iota(std::begin(channel_map), std::end(channel_map), 0);
    FillAudioBufferWithChannelRemapping(audio_buffer_ptr, num_frames,
                                        num_input_channels, channel_map,
                                        output_buffer);
    return;
  }

  LOG(WARNING) << "Number of input channels does not match the number of "
                  "output channels";
}

}  // namespace vraudio
