/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_RESONANCE_AUDIO_API_IMPL_H_
#define RESONANCE_AUDIO_GRAPH_RESONANCE_AUDIO_API_IMPL_H_

#include <atomic>
#include <memory>
#include <vector>

#include "base/integral_types.h"
#include "api/resonance_audio_api.h"
#include "base/audio_buffer.h"
#include "graph/graph_manager.h"
#include "graph/system_settings.h"
#include "utils/lockless_task_queue.h"

namespace vraudio {

// Implementation of ResonanceAudioApi interface.
class ResonanceAudioApiImpl : public ResonanceAudioApi {
 public:
  // Constructor that initializes |ResonanceAudioApi| with system configuration.
  //
  // @param num_channels Number of channels of audio output.
  // @param frames_per_buffer Number of frames per buffer.
  // @param sample_rate_hz System sample rate.
  ResonanceAudioApiImpl(size_t num_channels, size_t frames_per_buffer,
                        int sample_rate_hz);

  ~ResonanceAudioApiImpl() override;

  //////////////////////////////////
  // ResonanceAudioApi implementation.
  //////////////////////////////////

  // Obtain processed output buffers.
  bool FillInterleavedOutputBuffer(size_t num_channels, size_t num_frames,
                                   float* buffer_ptr) override;
  bool FillInterleavedOutputBuffer(size_t num_channels, size_t num_frames,
                                   int16* buffer_ptr) override;
  bool FillPlanarOutputBuffer(size_t num_channels, size_t num_frames,
                              float* const* buffer_ptr) override;
  bool FillPlanarOutputBuffer(size_t num_channels, size_t num_frames,
                              int16* const* buffer_ptr) override;

  // Listener configuration.
  void SetHeadPosition(float x, float y, float z) override;
  void SetHeadRotation(float x, float y, float z, float w) override;
  void SetPrimaryVolume(float volume) override;
  void SetStereoSpeakerMode(bool enabled) override;

  // Create and destroy sources.
  SourceId CreateAmbisonicSource(size_t num_channels) override;
  SourceId CreateStereoSource(size_t num_channels) override;
  SourceId CreateSoundObjectSource(RenderingMode rendering_mode) override;
  void DestroySource(SourceId source_id) override;

  // Set source data.
  void SetInterleavedBuffer(SourceId source_id, const float* audio_buffer_ptr,
                            size_t num_channels, size_t num_frames) override;
  void SetInterleavedBuffer(SourceId source_id, const int16* audio_buffer_ptr,
                            size_t num_channels, size_t num_frames) override;
  void SetPlanarBuffer(SourceId source_id, const float* const* audio_buffer_ptr,
                       size_t num_channels, size_t num_frames) override;
  void SetPlanarBuffer(SourceId source_id, const int16* const* audio_buffer_ptr,
                       size_t num_channels, size_t num_frames) override;

  // Source configuration.
  void SetSourceDistanceAttenuation(SourceId source_id,
                                    float distance_attenuation) override;
  void SetSourceDistanceModel(SourceId source_id, DistanceRolloffModel rolloff,
                              float min_distance, float max_distance) override;
  void SetSourcePosition(SourceId source_id, float x, float y,
                         float z) override;
  void SetSourceRoomEffectsGain(SourceId source_id,
                                float room_effects_gain) override;
  void SetSourceRotation(SourceId source_id, float x, float y, float z,
                         float w) override;
  void SetSourceVolume(SourceId source_id, float volume) override;

  // Sound object configuration.
  void SetSoundObjectDirectivity(SourceId sound_object_source_id, float alpha,
                                 float order) override;
  void SetSoundObjectListenerDirectivity(SourceId sound_object_source_id,
                                         float alpha, float order) override;
  void SetSoundObjectNearFieldEffectGain(SourceId sound_object_source_id,
                                         float gain) override;
  void SetSoundObjectOcclusionIntensity(SourceId sound_object_source_id,
                                        float intensity) override;
  void SetSoundObjectSpread(SourceId sound_object_source_id,
                            float spread_deg) override;

  // Room effects configuration.
  void EnableRoomEffects(bool enable) override;
  void SetReflectionProperties(
      const ReflectionProperties& reflection_properties) override;
  void SetReverbProperties(const ReverbProperties& reverb_properties) override;

  //////////////////////////////////
  // Internal API methods.
  //////////////////////////////////

  // Returns the last processed output buffer of the ambisonic mix.
  //
  // @return Pointer to ambisonic output buffer.
  const AudioBuffer* GetAmbisonicOutputBuffer() const;

  // Returns the last processed output buffer of the stereo (binaural) mix.
  //
  // @return Pointer to stereo output buffer.
  const AudioBuffer* GetStereoOutputBuffer() const;

  // Triggers processing of the audio graph with the updated system properties.
  void ProcessNextBuffer();

 private:
  // This method triggers the processing of the audio graph and outputs a
  // binaural stereo output buffer.
  //
  // @tparam OutputType Output sample format, only float and int16 are
  //     supported.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Size of buffer in frames.
  // @param buffer_ptr Raw pointer to audio buffer.
  // @return True if a valid output was successfully rendered, false otherwise.
  template <typename OutputType>
  bool FillOutputBuffer(size_t num_channels, size_t num_frames,
                        OutputType buffer_ptr);

  // Sets the next audio buffer to a sound source.
  //
  // @param source_id Id of sound source.
  // @param audio_buffer_ptr Pointer to planar or interleaved audio buffer.
  // @param num_input_channels Number of input channels.
  // @param num_frames Number of frames per channel / audio buffer.
  template <typename SampleType>
  void SetSourceBuffer(SourceId source_id, SampleType audio_buffer_ptr,
                       size_t num_input_channels, size_t num_frames);

  // Graph manager used to create and destroy sound objects.
  std::unique_ptr<GraphManager> graph_manager_;

  // Manages system wide settings.
  SystemSettings system_settings_;

  // Task queue to cache manipulation of all the entities in the system. All
  // tasks are executed from the audio thread.
  LocklessTaskQueue task_queue_;

  // Incremental source id counter.
  std::atomic<int> source_id_counter_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_RESONANCE_AUDIO_API_IMPL_H_
