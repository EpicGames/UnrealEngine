/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/reverb_node.h"

#include <algorithm>
#include <cmath>

#include "base/constants_and_types.h"
#include "base/logging.h"


namespace vraudio {

namespace {

// Default time in seconds to update the rt60s over.
const float kUpdateTimeSeconds = 1.0f;

// Interpolates between the current and target values in steps of |update_step|,
// will set |current| to |target| when the diff between them is less than
// |update_step|.
inline void InterpolateFloatParam(float update_step, float target,
                                  float* current) {
  if (std::abs(target - *current) <= std::abs(update_step)) {
    *current = target;
  } else {
    *current += update_step;
  }
}

}  // namespace

ReverbNode::ReverbNode(const SystemSettings& system_settings,
                       FftManager* fft_manager)
    : system_settings_(system_settings),
      rt60_band_update_steps_(kNumReverbOctaveBands, 0.0f),
      gain_update_step_(0.0f),
      rt60_updating_(false),
      gain_updating_(false),
      buffers_to_update_(
          static_cast<float>(system_settings_.GetSampleRateHz()) *
          kUpdateTimeSeconds /
          static_cast<float>(system_settings_.GetFramesPerBuffer())),
      spectral_reverb_(system_settings_.GetSampleRateHz(),
                       system_settings_.GetFramesPerBuffer()),
      onset_compensator_(system_settings_.GetSampleRateHz(),
                         system_settings_.GetFramesPerBuffer(), fft_manager),
      num_frames_processed_on_empty_input_(0),
      reverb_length_frames_(0),
      output_buffer_(kNumStereoChannels, system_settings_.GetFramesPerBuffer()),
      compensator_output_buffer_(kNumStereoChannels,
                                 system_settings_.GetFramesPerBuffer()),
      silence_mono_buffer_(kNumMonoChannels,
                           system_settings_.GetFramesPerBuffer()) {
  EnableProcessOnEmptyInput(true);
  output_buffer_.Clear();
  silence_mono_buffer_.Clear();
  Update();
}

void ReverbNode::Update() {
  new_reverb_properties_ = system_settings_.GetReverbProperties();

  rt60_updating_ = !EqualSafe(std::begin(reverb_properties_.rt60_values),
                              std::end(reverb_properties_.rt60_values),
                              std::begin(new_reverb_properties_.rt60_values),
                              std::end(new_reverb_properties_.rt60_values));
  if (rt60_updating_) {
    for (size_t i = 0; i < kNumReverbOctaveBands; ++i) {
      rt60_band_update_steps_[i] = (new_reverb_properties_.rt60_values[i] -
                                    reverb_properties_.rt60_values[i]) /
                                   buffers_to_update_;
    }
  }
  // Update the reverb gain if necessary.
  gain_updating_ = reverb_properties_.gain != new_reverb_properties_.gain;
  if (gain_updating_) {
    gain_update_step_ =
        (new_reverb_properties_.gain - reverb_properties_.gain) /
        buffers_to_update_;
  }
}

const AudioBuffer* ReverbNode::AudioProcess(const NodeInput& input) {
  if (rt60_updating_) {
    for (size_t i = 0; i < kNumReverbOctaveBands; ++i) {
      InterpolateFloatParam(rt60_band_update_steps_[i],
                            new_reverb_properties_.rt60_values[i],
                            &reverb_properties_.rt60_values[i]);
    }
    spectral_reverb_.SetRt60PerOctaveBand(reverb_properties_.rt60_values);
    const auto max_rt_it =
        std::max_element(std::begin(reverb_properties_.rt60_values),
                         std::end(reverb_properties_.rt60_values));
    reverb_length_frames_ = static_cast<size_t>(
        *max_rt_it * static_cast<float>(system_settings_.GetSampleRateHz()));
    onset_compensator_.Update(reverb_properties_.rt60_values,
                              reverb_properties_.gain);
    // |InterpolateFloatParam| will set the two values below to be equal on
    // completion of interpolation.
    rt60_updating_ = !EqualSafe(std::begin(reverb_properties_.rt60_values),
                                std::end(reverb_properties_.rt60_values),
                                std::begin(new_reverb_properties_.rt60_values),
                                std::end(new_reverb_properties_.rt60_values));
  }

  if (gain_updating_) {
    InterpolateFloatParam(gain_update_step_, new_reverb_properties_.gain,
                          &reverb_properties_.gain);
    spectral_reverb_.SetGain(reverb_properties_.gain);
    onset_compensator_.Update(reverb_properties_.rt60_values,
                              reverb_properties_.gain);
    // |InterpolateFloatParam| will set the two values below to be equal on
    // completion of interpolation.
    gain_updating_ = reverb_properties_.gain != new_reverb_properties_.gain;
  }

  const AudioBuffer* input_buffer = input.GetSingleInput();
  if (input_buffer == nullptr) {
    // If we have no input, generate a silent input buffer until the node states
    // are cleared.
    if (num_frames_processed_on_empty_input_ < reverb_length_frames_) {
      const size_t num_frames = system_settings_.GetFramesPerBuffer();
      num_frames_processed_on_empty_input_ += num_frames;
      spectral_reverb_.Process(silence_mono_buffer_[0], &output_buffer_[0],
                               &output_buffer_[1]);
      return &output_buffer_;
    } else {
      // Skip processing entirely when the states are fully cleared.
      return nullptr;
    }
  }
  DCHECK_EQ(input_buffer->num_channels(), kNumMonoChannels);
  num_frames_processed_on_empty_input_ = 0;
  spectral_reverb_.Process((*input_buffer)[0], &output_buffer_[0],
                           &output_buffer_[1]);
  onset_compensator_.Process(*input_buffer, &compensator_output_buffer_);
  output_buffer_[0] += compensator_output_buffer_[0];
  output_buffer_[1] += compensator_output_buffer_[1];
  return &output_buffer_;
}

}  // namespace vraudio
