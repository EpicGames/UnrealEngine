/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_REVERB_NODE_H_
#define RESONANCE_AUDIO_GRAPH_REVERB_NODE_H_

#include "api/resonance_audio_api.h"
#include "base/audio_buffer.h"
#include "dsp/fft_manager.h"
#include "dsp/reverb_onset_compensator.h"
#include "dsp/spectral_reverb.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Implements a spectral reverb producing a decorrelated stereo output with
// onset compensated by a pair of convolution filters.
class ReverbNode : public ProcessingNode {
 public:
  // Constructs a |ReverbNode|.
  //
  // @param system_settings Global system configuration.
  // @param fft_manager Pointer to a manager to perform FFT transformations.
  ReverbNode(const SystemSettings& system_settings, FftManager* fft_manager);

  // Updates the |SpectralReverb| using the current room properties or RT60
  // values depending on the system settings.
  void Update();

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  // Global system configuration.
  const SystemSettings& system_settings_;

  // Current reverb properties.
  ReverbProperties reverb_properties_;

  // New reverb properties.
  ReverbProperties new_reverb_properties_;

  // Per band reverb time update step sizes.
  std::vector<float> rt60_band_update_steps_;

  // Update step size for the gain parameter.
  float gain_update_step_;

  // Denotes whether the rt60s are currently being updated.
  bool rt60_updating_;

  // Denotes whether the gain is currently being updated.
  bool gain_updating_;

  // Number of buffers to updae rt60s over.
  float buffers_to_update_;

  // DSP class to perform filtering associated with the reverb.
  SpectralReverb spectral_reverb_;

  // DSP class to perform spectral reverb onset compensation.
  ReverbOnsetCompensator onset_compensator_;

  // Number of frames of zeroed out data to be processed by the node to ensure
  // the entire tail is rendered after input has ceased.
  size_t num_frames_processed_on_empty_input_;

  // Longest current reverb time, across all bands, in frames.
  size_t reverb_length_frames_;

  // Output buffers for mixing spectral reverb and compensator output.
  AudioBuffer output_buffer_;
  AudioBuffer compensator_output_buffer_;

  // Silence mono buffer to render reverb tails during the absence of input
  // buffers.
  AudioBuffer silence_mono_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_REVERB_NODE_H_
