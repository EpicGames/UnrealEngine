/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/source_parameters_manager.h"

#include "base/logging.h"

namespace vraudio {

void SourceParametersManager::Register(SourceId source_id) {
  DCHECK(parameters_.find(source_id) == parameters_.end());
  parameters_[source_id] = SourceParameters();
}

void SourceParametersManager::Unregister(SourceId source_id) {
  parameters_.erase(source_id);
}

const SourceParameters* SourceParametersManager::GetParameters(
    SourceId source_id) const {
  const auto source_parameters_itr = parameters_.find(source_id);
  if (source_parameters_itr == parameters_.end()) {
    LOG(ERROR) << "Source " << source_id << " not found";
    return nullptr;
  }
  return &source_parameters_itr->second;
}

SourceParameters* SourceParametersManager::GetMutableParameters(
    SourceId source_id) {
  auto source_parameters_itr = parameters_.find(source_id);
  if (source_parameters_itr == parameters_.end()) {
    LOG(ERROR) << "Source " << source_id << " not found";
    return nullptr;
  }
  return &source_parameters_itr->second;
}

void SourceParametersManager::ProcessAllParameters(const Process& process) {
  for (auto& source_parameters_itr : parameters_) {
    process(&source_parameters_itr.second);
  }
}

}  // namespace vraudio
