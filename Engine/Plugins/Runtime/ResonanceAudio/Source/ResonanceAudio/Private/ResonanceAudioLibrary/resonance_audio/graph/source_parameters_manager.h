/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_SOURCE_PARAMETERS_MANAGER_H_
#define RESONANCE_AUDIO_GRAPH_SOURCE_PARAMETERS_MANAGER_H_

#include <functional>
#include <unordered_map>

#include "base/constants_and_types.h"
#include "base/source_parameters.h"

namespace vraudio {

// Class that manages the corresponding parameters of each registered source.
class SourceParametersManager {
 public:
  // Alias for the parameters process closure type.
  using Process = std::function<void(SourceParameters*)>;

  // Registers new source parameters for given |source_id|.
  //
  // @param source_id Source id.
  void Register(SourceId source_id);

  // Unregisters the source parameters for given |source_id|.
  //
  // @param source_id Source id.
  void Unregister(SourceId source_id);

  // Returns read-only source parameters for given |source_id|.
  //
  // @param source_id Source id.
  // @return Read-only source parameters, nullptr if |source_id| not found.
  const SourceParameters* GetParameters(SourceId source_id) const;

  // Returns mutable source parameters for given |source_id|.
  //
  // @param source_id Source id.
  // @return Mutable source parameters, nullptr if |source_id| not found.
  SourceParameters* GetMutableParameters(SourceId source_id);

  // Executes given |process| for the parameters of each registered source.
  //
  // @param process Parameters processing method.
  void ProcessAllParameters(const Process& process);

 private:
  // Registered source parameters.
  std::unordered_map<SourceId, SourceParameters> parameters_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_SOURCE_PARAMETERS_MANAGER_H_
