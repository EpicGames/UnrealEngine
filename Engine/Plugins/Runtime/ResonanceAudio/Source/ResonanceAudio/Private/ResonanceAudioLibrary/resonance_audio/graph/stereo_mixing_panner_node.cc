/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "graph/stereo_mixing_panner_node.h"

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/spherical_angle.h"

#include "dsp/stereo_panner.h"

namespace vraudio {

StereoMixingPannerNode::StereoMixingPannerNode(
    const SystemSettings& system_settings)
    : system_settings_(system_settings),
      gain_mixer_(kNumStereoChannels, system_settings_.GetFramesPerBuffer()),
      coefficients_(kNumStereoChannels) {}

const AudioBuffer* StereoMixingPannerNode::AudioProcess(
    const NodeInput& input) {


  const WorldPosition& listener_position = system_settings_.GetHeadPosition();
  const WorldRotation& listener_rotation = system_settings_.GetHeadRotation();

  gain_mixer_.Reset();
  for (auto& input_buffer : input.GetInputBuffers()) {
    const int source_id = input_buffer->source_id();
    const auto source_parameters =
        system_settings_.GetSourceParameters(source_id);
    DCHECK_NE(source_id, kInvalidSourceId);
    DCHECK_EQ(input_buffer->num_channels(), 1U);

    // Compute the relative source direction in spherical angles.
    const ObjectTransform& source_transform =
        source_parameters->object_transform;
    WorldPosition relative_direction;
    GetRelativeDirection(listener_position, listener_rotation,
                         source_transform.position, &relative_direction);
    const SphericalAngle source_direction =
        SphericalAngle::FromWorldPosition(relative_direction);


    CalculateStereoPanGains(source_direction, &coefficients_);

    gain_mixer_.AddInputChannel((*input_buffer)[0], source_id, coefficients_);
  }
  return gain_mixer_.GetOutput();
}

}  // namespace vraudio
