/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_STEREO_MIXING_PANNER_NODE_H_
#define RESONANCE_AUDIO_GRAPH_STEREO_MIXING_PANNER_NODE_H_

#include <vector>

#include "base/audio_buffer.h"
#include "dsp/gain_mixer.h"
#include "graph/system_settings.h"
#include "node/processing_node.h"

namespace vraudio {

// Node that accepts single mono sound object buffer as input and pans into a
// stereo panorama.
class StereoMixingPannerNode : public ProcessingNode {
 public:
  // Initializes StereoMixingPannerNode class.
  //
  // @param system_settings Global system configuration.
  explicit StereoMixingPannerNode(const SystemSettings& system_settings);

  // Node implementation.
  bool CleanUp() final {
    CallCleanUpOnInputNodes();
    // Prevent node from being disconnected when all sources are removed.
    return false;
  }

 protected:
  // Implements ProcessingNode.
  const AudioBuffer* AudioProcess(const NodeInput& input) override;

 private:
  const SystemSettings& system_settings_;

  // |GainMixer| instance.
  GainMixer gain_mixer_;

  // Panning coefficients to be applied the input.
  std::vector<float> coefficients_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_STEREO_MIXING_PANNER_NODE_H_
