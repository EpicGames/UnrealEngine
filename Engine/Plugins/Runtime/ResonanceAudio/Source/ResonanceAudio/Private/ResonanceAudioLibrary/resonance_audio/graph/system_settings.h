/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_GRAPH_SYSTEM_SETTINGS_H_
#define RESONANCE_AUDIO_GRAPH_SYSTEM_SETTINGS_H_

#include "api/resonance_audio_api.h"
#include "base/constants_and_types.h"
#include "base/misc_math.h"
#include "graph/source_parameters_manager.h"

namespace vraudio {

// Contains system-wide settings and parameters. Note that this class is not
// thread-safe. Updating system parameters must be avoided during the audio
// graph processing.
class SystemSettings {
 public:
  // Constructor initializes the system configuration.
  //
  // @param num_output_channels Number of output channels.
  // @param frames_per_buffer Buffer size in frames.
  // @param sample_rate_hz Sample rate.
  SystemSettings(size_t num_output_channels, size_t frames_per_buffer,
                 int sample_rate_hz)
      : sample_rate_hz_(sample_rate_hz),
        frames_per_buffer_(frames_per_buffer),
        num_channels_(num_output_channels),
        head_rotation_(WorldRotation::Identity()),
        head_position_(WorldPosition::Zero()),
        primary_gain_(1.0f),
        stereo_speaker_mode_(false) {}

  // Sets the listener head orientation.
  //
  // @param head_rotation Listener head orientation.
  void SetHeadRotation(const WorldRotation& head_rotation) {
    head_rotation_ = head_rotation;
  }

  // Sets the listener head position.
  //
  // @param head_position Listener head position.
  void SetHeadPosition(const WorldPosition& head_position) {
    head_position_ = head_position;
  }

  // Sets the global stereo speaker mode flag. This flag enforces stereo panning
  // and disables HRTF-based binauralization. The stereo speaker mode is
  // disabled by default.
  //
  // @param enabled Defines the stereo speaker mode state.
  void SetStereoSpeakerMode(bool enabled) { stereo_speaker_mode_ = enabled; }

  // Returns the source parameters manager.
  //
  // @return Mutable source parameters manager.
  SourceParametersManager* GetSourceParametersManager() {
    return &source_parameters_manager_;
  }

  // Returns the parameters of source with given |source_id|.
  //
  // @param source_id Source id.
  // @return Pointer to source parameters, nullptr if |source_id| not found.
  const SourceParameters* GetSourceParameters(SourceId source_id) const {
    return source_parameters_manager_.GetParameters(source_id);
  }

  // Returns the sample rate.
  //
  // @return Sample rate in Hertz.
  int GetSampleRateHz() const { return sample_rate_hz_; }

  // Returns the frames per buffer.
  //
  // @return Buffer size in frames.
  size_t GetFramesPerBuffer() const { return frames_per_buffer_; }

  // Returns the number of output channels.
  //
  // @return Number of output channels.
  size_t GetNumChannels() const { return num_channels_; }

  // Returns the head rotation.
  //
  // @return Head orientation.
  const WorldRotation& GetHeadRotation() const { return head_rotation_; }

  // Returns the head position.
  //
  // @return Head position.
  const WorldPosition& GetHeadPosition() const { return head_position_; }

  // Returns the stereo speaker mode state.
  //
  // @return Current stereo speaker mode state.
  bool IsStereoSpeakerModeEnabled() const { return stereo_speaker_mode_; }

  // Sets the primary gain.
  //
  // @param primary_gain primary output gain.
  void SetPrimaryGain(float primary_gain) { primary_gain_ = primary_gain; }

  // Sets current reflection properties.
  //
  // @param reflection_properties Reflection properties.
  void SetReflectionProperties(
      const ReflectionProperties& reflection_properties) {
    reflection_properties_ = reflection_properties;
  }

  // Sets current reverb properties.
  //
  // @param reverb_properties Reflection properties.
  void SetReverbProperties(const ReverbProperties& reverb_properties) {
    reverb_properties_ = reverb_properties;
  }

  // Returns the primary gain.
  //
  // @return primary output gain.
  float GetPrimaryGain() const { return primary_gain_; }

  // Returns the current reflection properties of the environment.
  //
  // @return Current reflection properties.
  const ReflectionProperties& GetReflectionProperties() const {
    return reflection_properties_;
  }

  // Returns the current reverb properties of the environment.
  //
  // @return Current reverb properties.
  const ReverbProperties& GetReverbProperties() const {
    return reverb_properties_;
  }

  // Disable copy and assignment operator. Since |SystemSettings| serves as a
  // global parameter storage, it should never be copied.
  SystemSettings& operator=(const SystemSettings&) = delete;
  SystemSettings(const SystemSettings&) = delete;

 private:
  // Sampling rate.
  const int sample_rate_hz_;

  // Frames per buffer.
  const size_t frames_per_buffer_;

  // Number of channels per buffer.
  const size_t num_channels_;

  // The most recently updated head rotation and position.
  WorldRotation head_rotation_;
  WorldPosition head_position_;

  // Source parameters manager.
  SourceParametersManager source_parameters_manager_;

  // primary gain in amplitude.
  float primary_gain_;

  // Current reflection properties of the environment.
  ReflectionProperties reflection_properties_;

  // Current reverb properties of the environment.
  ReverbProperties reverb_properties_;

  // Defines the state of the global speaker mode.
  bool stereo_speaker_mode_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_GRAPH_SYSTEM_SETTINGS_H_
