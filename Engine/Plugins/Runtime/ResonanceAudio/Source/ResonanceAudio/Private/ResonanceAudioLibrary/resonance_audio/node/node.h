/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_NODE_H_
#define RESONANCE_AUDIO_NODE_NODE_H_

#include <memory>
#include <set>
#include <unordered_map>
#include <utility>
#include <vector>

#include "base/logging.h"

namespace vraudio {

// Implements a processing node in a synchronous processing graph.
// This processing graph is expected to be directed, acyclic, and
// accessed from a single thread.
//
// Subclasses are expected to implement Process(), which will read
// from all of the instance's inputs, process the data as necessary,
// and then write to all of its outputs.
//
// Data is passed through unique_ptrs, so nodes are expected to
// modify data in place whenever it suits their purposes. If an
// outputs is connected to more than one input, copies will be made for
// each input.
//
// Graphs are managed through shared_ptrs. Orphaned nodes are kept
// alive as long as they output to a living input. Ownership is
// unidirectional -- from input to output -- in order to avoid
// circular dependencies.
class Node : public std::enable_shared_from_this<Node> {
 public:
  virtual ~Node() = default;
  virtual void Process() = 0;

  // Disconnects from input nodes that are marked to be at the end of data
  // stream.
  //
  // @return True if node is does not have any inputs and can be removed, false
  // otherwise.
  virtual bool CleanUp() = 0;

  template <class T>
  class Output;

  // An endpoint for a node, this object consumes data from any connected
  // outputs. Because an input may be connected to more than one output, it
  // returns a vector of read data. All outputs must be of the same type.
  template <typename T>
  class Input {
   public:
    // Unordered map that stores pairs of input Node instances and their
    // |Output| member.
    typedef std::unordered_map<Output<T>*, std::shared_ptr<Node>> OutputNodeMap;

    Input() {}
    ~Input();

    // Returns a vector of computed data, one for each connected output.
    const std::vector<T>& Read();

    // Connects this input to the specified output.
    //
    // @param node The parent of the output.
    // @param output The output to connect to.
    void Connect(const std::shared_ptr<Node>& node, Output<T>* output);

    // Disconnects this input from the specified output.
    //
    // @param output The output to be disconnected.
    void Disconnect(Output<T>* output);

    // Returns the number of connected outputs.
    //
    // @return Number of connected outputs.
    size_t GetNumConnections() const;

    // Returns reference to OutputNodeMap map to obtain all connected nodes and
    // their outputs.
    const OutputNodeMap& GetConnectedNodeOutputPairs();

    // Disable copy constructor.
    Input(const Input& that) = delete;

   private:
    friend class Node::Output<T>;

    void AddOutput(const std::shared_ptr<Node>& node, Output<T>* output);
    void RemoveOutput(Output<T>* output);

    OutputNodeMap outputs_;
    std::vector<T> read_data_;
  };

  // An endpoint for a node, this object produces data for any connected inputs.
  // Because an output may have more than one input, this object will duplicate
  // any computed data, once for each connected input. All inputs must be of the
  // same type.
  //
  // If an output does not have any data to deliver, it will ask its parent node
  // to process more data. It is assumed that after processing, some new data
  // will be written to this output.
  template <typename T>
  class Output {
   public:
    explicit Output(Node* node) : parent_(node) {}

    // Parent nodes should call this function to push new data to any connected
    // inputs. This data will be copied once for each connected input.
    //
    // @param data New data to pass to all connected inputs.
    void Write(T data);

    // Disable copy constructor.
    Output(const Output& that) = delete;

   private:
    friend class Node::Input<T>;

    // Signature of copy operator.
    typedef T (*CopyOperator)(const T&);

    // Returns a single piece of stored processed data. If no data exists,
    // the parent node is processed to produce more data.
    T PullData();

    void AddInput(Input<T>* input);
    bool RemoveInput(Input<T>* input);

    std::set<Input<T>*> inputs_;
    std::vector<T> written_data_;
    Node* parent_;
  };
};

template <class T>
Node::Input<T>::~Input() {
  for (auto& o : outputs_) {
    CHECK(o.first->RemoveInput(this));
  }
}

template <class T>
const std::vector<T>& Node::Input<T>::Read() {
  read_data_.clear();

  for (auto& o : outputs_) {
    // Obtain processed data.
    T processed_data = o.first->PullData();
    if (processed_data != nullptr) {
      read_data_.emplace_back(std::move(processed_data));
    }
  }

  return read_data_;
}

template <class T>
void Node::Input<T>::Connect(const std::shared_ptr<Node>& node,
                             Output<T>* output) {
  output->AddInput(this);
  AddOutput(node, output);
}

// RemoveOutput(output) may trigger *output be destructed,
// so we need to call output->RemoveInput(this) first.
template <class T>
void Node::Input<T>::Disconnect(Output<T>* output) {
  output->RemoveInput(this);
  RemoveOutput(output);
}

template <class T>
size_t Node::Input<T>::GetNumConnections() const {
  return outputs_.size();
}

template <class T>
const typename Node::Input<T>::OutputNodeMap&
Node::Input<T>::GetConnectedNodeOutputPairs() {
  return outputs_;
}

template <class T>
void Node::Input<T>::AddOutput(const std::shared_ptr<Node>& node,
                               Output<T>* output) {
  outputs_[output] = node;

  DCHECK(outputs_.find(output) != outputs_.end());
}

template <class T>
void Node::Input<T>::RemoveOutput(Output<T>* output) {
  outputs_.erase(output);
}

template <class T>
T Node::Output<T>::PullData() {
  if (written_data_.empty()) {
    parent_->Process();
  }

  DCHECK(!written_data_.empty());

  T return_value = std::move(written_data_.back());
  written_data_.pop_back();
  return return_value;
}

template <class T>
void Node::Output<T>::Write(T data) {
  DCHECK(written_data_.empty());
  written_data_.clear();
  written_data_.emplace_back(std::move(data));

  // If we have more than one connected input, copy the data for each input.
  for (size_t i = 1; i < inputs_.size(); i++) {
    written_data_.push_back(written_data_[0]);
  }

  DCHECK(written_data_.size() == inputs_.size());
}

template <class T>
void Node::Output<T>::AddInput(Input<T>* input) {
  inputs_.insert(input);
}

template <class T>
bool Node::Output<T>::RemoveInput(Input<T>* input) {
  auto it = inputs_.find(input);
  if (it == inputs_.end()) {
    return false;
  }

  inputs_.erase(it);
  return true;
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_NODE_H_
