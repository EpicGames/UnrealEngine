/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "node/processing_node.h"

namespace vraudio {

ProcessingNode::NodeInput::NodeInput(
    const std::vector<const AudioBuffer*>& input_vector)
    : input_vector_(input_vector) {}

const AudioBuffer* ProcessingNode::NodeInput::GetSingleInput() const {
  if (input_vector_.size() == 1) {
    return input_vector_[0];
  }
  if (input_vector_.size() > 1) {
    LOG(WARNING) << "GetSingleInput() called on multi buffer input";
  }
  return nullptr;
}

const std::vector<const AudioBuffer*>&
ProcessingNode::NodeInput::GetInputBuffers() const {
  return input_vector_;
}

ProcessingNode::ProcessingNode()
    : Node(), output_stream_(this), process_on_no_input_(false) {}

void ProcessingNode::Connect(
    const std::shared_ptr<PublisherNodeType>& publisher_node) {
  input_stream_.Connect(publisher_node->GetSharedNodePtr(),
                        publisher_node->GetOutput());
}

void ProcessingNode::Process() {
  NodeInput input(input_stream_.Read());
  const AudioBuffer* output = nullptr;
  // Only call AudioProcess if input data is available.
  if (process_on_no_input_ || !input.GetInputBuffers().empty()) {
    output = AudioProcess(input);
  }
  output_stream_.Write(output);
}

bool ProcessingNode::CleanUp() {
  CallCleanUpOnInputNodes();
  return (input_stream_.GetNumConnections() == 0);
}

void ProcessingNode::EnableProcessOnEmptyInput(bool enable) {
  process_on_no_input_ = enable;
}

void ProcessingNode::CallCleanUpOnInputNodes() {
  // We need to make a copy of the OutputNodeMap map since it changes due to
  // Disconnect() calls.
  const auto connected_nodes = input_stream_.GetConnectedNodeOutputPairs();
  for (const auto& input_node : connected_nodes) {
    Output<const AudioBuffer*>* output = input_node.first;
    std::shared_ptr<Node> node = input_node.second;
    const bool is_ready_to_be_disconnected = node->CleanUp();
    if (is_ready_to_be_disconnected) {
      input_stream_.Disconnect(output);
    }
  }
}

std::shared_ptr<Node> ProcessingNode::GetSharedNodePtr() {
  return shared_from_this();
}
Node::Output<const AudioBuffer*>* ProcessingNode::GetOutput() {
  return &output_stream_;
}

}  // namespace vraudio
