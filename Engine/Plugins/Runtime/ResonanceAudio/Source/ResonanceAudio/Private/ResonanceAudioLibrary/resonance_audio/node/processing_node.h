/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_PROCESSING_NODE_H_
#define RESONANCE_AUDIO_NODE_PROCESSING_NODE_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"
#include "node/publisher_node.h"
#include "node/subscriber_node.h"

namespace vraudio {

// Audio processing node that reads from multiple inputs, processes the
// received data and outputs its result.
class ProcessingNode : public Node,
                       public SubscriberNode<const AudioBuffer*>,
                       public PublisherNode<const AudioBuffer*> {
 public:
  typedef SubscriberNode<const AudioBuffer*> SubscriberNodeType;
  typedef PublisherNode<const AudioBuffer*> PublisherNodeType;

  // Helper class to manage incoming |AudioBuffer|s.
  class NodeInput {
   public:
    // Constructor.
    //
    // @param input_vector Vector containing pointers to incoming
    //     |AudioBuffer|s.
    explicit NodeInput(const std::vector<const AudioBuffer*>& input_vector);

    // Returns a nullptr if zero or more than one input buffers are available.
    // Otherwise a pointer to the single input |AudioBuffer| is returned. This
    // method should be used if only a single input |AudioBuffer| is expected.
    //
    // @return Pointer to single input |AudioBuffer|.
    const AudioBuffer* GetSingleInput() const;

    // Returns vector with input |AudioBuffer|s.
    //
    // @return Pointer to single input |AudioBuffer|.
    const std::vector<const AudioBuffer*>& GetInputBuffers() const;

    // Delete copy constructor.
    NodeInput(const NodeInput& that) = delete;

   private:
    // Const reference to vector of input |AudioBuffer|s.
    const std::vector<const AudioBuffer*>& input_vector_;
  };

  ProcessingNode();

  // SubscriberNode<InputType> implementation.
  void Connect(
      const std::shared_ptr<PublisherNodeType>& publisher_node) override;

  // Node implementation.
  void Process() final;
  bool CleanUp() override;

  // By default, calls to AudioProcess() are skipped in case of empty input
  // buffers. This enables this node to process audio buffers in the absence of
  // input data (which is needed for instance for a reverberation effect).
  void EnableProcessOnEmptyInput(bool enable);

  // Disable copy constructor.
  ProcessingNode(const ProcessingNode& that) = delete;

 protected:
  // Calls |CleanUp| on all connected input nodes.
  void CallCleanUpOnInputNodes();

  // Pure virtual method to implement the audio processing method. This method
  // receives a vector of all input arguments to be processed and requires to
  // output a single output buffer.
  //
  // @param input Input instance to receive pointers to input |AudioBuffer|s.
  // @return Returns output data.
  virtual const AudioBuffer* AudioProcess(const NodeInput& input) = 0;

 private:
  // PublisherNode<OutputType> implementation.
  std::shared_ptr<Node> GetSharedNodePtr() final;
  Node::Output<const AudioBuffer*>* GetOutput() final;

  // Input stream to poll for incoming data.
  Node::Input<const AudioBuffer*> input_stream_;

  // Output stream to write processed data to.
  Node::Output<const AudioBuffer*> output_stream_;

  // Flag that indicates if |AudioProcess| should be called in case no input
  // data is available.
  bool process_on_no_input_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_PROCESSING_NODE_H_
