/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_PUBLISHER_NODE_H_
#define RESONANCE_AUDIO_NODE_PUBLISHER_NODE_H_

#include <memory>

#include "base/logging.h"
#include "node/node.h"

namespace vraudio {

// Interface for publisher nodes that declares helper methods required to
// connect to a publisher node. All publishing nodes need to implement this
// interface.
//
// @tparam OutputType Type of the output container being streamed.
// @interface
template <typename OutputType>
class PublisherNode {
 public:
  virtual ~PublisherNode() = default;

  // Creates a shared pointer of the Node instance.
  //
  // @return Returns a shared pointer the of Node instance.
  virtual std::shared_ptr<Node> GetSharedNodePtr() = 0;

  // Get internal Node::Output instance.
  //
  // @return Returns a pointer to the internal Node::Output instance.
  virtual Node::Output<OutputType>* GetOutput() = 0;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_PUBLISHER_NODE_H_
