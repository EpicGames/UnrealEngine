/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "node/sink_node.h"

#include "base/logging.h"

namespace vraudio {

SinkNode::SinkNode() : Node() {}

const std::vector<const AudioBuffer*>& SinkNode::ReadInputs() {
  return input_stream_.Read();
}

void SinkNode::Connect(
    const std::shared_ptr<PublisherNodeType>& publisher_node) {
  input_stream_.Connect(publisher_node->GetSharedNodePtr(),
                        publisher_node->GetOutput());
}

void SinkNode::Process() {
  LOG(FATAL) << "Process should not be called on audio sink node.";
}

bool SinkNode::CleanUp() {
  // We need to make a copy of the OutputNodeMap map since it might change due
  // to Disconnect() calls.
  const auto connected_nodes = input_stream_.GetConnectedNodeOutputPairs();
  for (const auto& input_node : connected_nodes) {
    Output<const AudioBuffer*>* output = input_node.first;
    std::shared_ptr<Node> node = input_node.second;
    const bool is_orphaned = node->CleanUp();
    if (is_orphaned) {
      input_stream_.Disconnect(output);
    }
  }
  return false;
}

}  // namespace vraudio
