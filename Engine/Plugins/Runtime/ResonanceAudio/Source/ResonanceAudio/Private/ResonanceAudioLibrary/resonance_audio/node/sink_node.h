/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_SINK_NODE_H_
#define RESONANCE_AUDIO_NODE_SINK_NODE_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"
#include "node/publisher_node.h"
#include "node/subscriber_node.h"

namespace vraudio {

// Audio sink node that reads from multiple inputs.
class SinkNode : public Node, public SubscriberNode<const AudioBuffer*> {
 public:
  typedef PublisherNode<const AudioBuffer*> PublisherNodeType;

  SinkNode();

  // Polls for data on all inputs of the sink node.
  //
  // @return A vector of input data.
  const std::vector<const AudioBuffer*>& ReadInputs();

  // SubscriberNode<AudioBuffer> implementation.
  void Connect(
      const std::shared_ptr<PublisherNodeType>& publisher_node) override;

  // Node implementation.
  void Process() final;
  bool CleanUp() final;

  // Disable copy constructor.
  SinkNode(const SinkNode& that) = delete;

 private:
  // Input stream to poll for incoming data.
  Node::Input<const AudioBuffer*> input_stream_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_SINK_NODE_H_
