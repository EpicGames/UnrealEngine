/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "node/source_node.h"

namespace vraudio {

SourceNode::SourceNode()
    : Node(), output_stream_(this), end_of_stream_(false) {}

void SourceNode::Process() {
  const AudioBuffer* output = AudioProcess();
  output_stream_.Write(output);
}

bool SourceNode::CleanUp() { return end_of_stream_; }

std::shared_ptr<Node> SourceNode::GetSharedNodePtr() {
  return shared_from_this();
}

Node::Output<const AudioBuffer*>* SourceNode::GetOutput() {
  return &output_stream_;
}

void SourceNode::MarkEndOfStream() { end_of_stream_ = true; }

}  // namespace vraudio
