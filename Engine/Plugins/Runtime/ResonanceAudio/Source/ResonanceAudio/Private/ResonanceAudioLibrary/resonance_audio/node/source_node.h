/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_SOURCE_NODE_H_
#define RESONANCE_AUDIO_NODE_SOURCE_NODE_H_

#include <atomic>
#include <memory>

#include "base/audio_buffer.h"
#include "node/publisher_node.h"

namespace vraudio {

// Audio source node that outputs data via the AudioProcess() method.
class SourceNode : public Node, public PublisherNode<const AudioBuffer*> {
 public:
  typedef PublisherNode<const AudioBuffer*> PublisherNodeType;

  SourceNode();

  // Node implementation.
  void Process() final;
  bool CleanUp() final;

  // PublisherNode<OutputType> implementation.
  std::shared_ptr<Node> GetSharedNodePtr() final;
  Node::Output<const AudioBuffer*>* GetOutput() final;

  // Marks this node as being out of data and to be removed during the next
  // clean-up cycle.
  void MarkEndOfStream();

  // Disable copy constructor.
  SourceNode(const SourceNode& that) = delete;

 protected:
  // Pure virtual method to implement the audio processing method. This method
  // requires to return a single output buffer that can be processed by the node
  // subscribers.
  //
  // @return Returns output data.
  virtual const AudioBuffer* AudioProcess() = 0;

 private:
  // Output stream to write processed data to.
  Node::Output<const AudioBuffer*> output_stream_;

  // Flag indicating if this source node can be removed.
  std::atomic<bool> end_of_stream_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_SOURCE_NODE_H_
