/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_NODE_SUBSCRIBER_NODE_H_
#define RESONANCE_AUDIO_NODE_SUBSCRIBER_NODE_H_

#include <memory>

#include "base/logging.h"
#include "node/publisher_node.h"

namespace vraudio {

// Interface for subscriber nodes that declares the connection and
// disconnection methods. All subscribing nodes need to implement this
// interface.
//
// @tparam InputType Input data type, i. e., the output data type of nodes to
//     connect to.
// @interface
template <typename InputType>
class SubscriberNode {
 public:
  virtual ~SubscriberNode() = default;

  // Connects this node to |publisher_node|.
  //
  // @param publisher_node Publisher node to connect to.
  virtual void Connect(
      const std::shared_ptr<PublisherNode<InputType>>& publisher_node) = 0;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_NODE_SUBSCRIBER_NODE_H_
