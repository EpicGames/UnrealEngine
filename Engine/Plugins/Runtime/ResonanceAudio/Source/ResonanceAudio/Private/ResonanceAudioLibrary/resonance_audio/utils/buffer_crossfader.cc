/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "utils/buffer_crossfader.h"

#include "base/constants_and_types.h"
#include "base/simd_utils.h"

namespace vraudio {

BufferCrossfader::BufferCrossfader(size_t num_frames)
    : crossfade_buffer_(kNumStereoChannels, num_frames) {
  DCHECK_NE(num_frames, 0);
  // Initialize the |crossfade_buffer_|.
  auto* fade_in_channel = &crossfade_buffer_[0];
  auto* fade_out_channel = &crossfade_buffer_[1];
  for (size_t frame = 0; frame < num_frames; ++frame) {
    const float crossfade_factor =
        static_cast<float>(frame) / static_cast<float>(num_frames);
    (*fade_in_channel)[frame] = crossfade_factor;
    (*fade_out_channel)[frame] = 1.0f - crossfade_factor;
  }
}

void BufferCrossfader::ApplyLinearCrossfade(const AudioBuffer& input_fade_in,
                                            const AudioBuffer& input_fade_out,
                                            AudioBuffer* output) const {
  DCHECK(output);
  DCHECK_NE(output, &input_fade_in);
  DCHECK_NE(output, &input_fade_out);

  const size_t num_channels = input_fade_in.num_channels();
  const size_t num_frames = input_fade_in.num_frames();
  DCHECK_EQ(num_channels, input_fade_out.num_channels());
  DCHECK_EQ(num_channels, output->num_channels());
  DCHECK_EQ(num_frames, input_fade_out.num_frames());
  DCHECK_EQ(num_frames, output->num_frames());
  DCHECK_EQ(num_frames, crossfade_buffer_.num_frames());

  const auto* gain_fade_in_channel = crossfade_buffer_[0].begin();
  const auto* gain_fade_out_channel = crossfade_buffer_[1].begin();
  for (size_t channel = 0; channel < num_channels; ++channel) {
    const auto* input_fade_in_channel = input_fade_in[channel].begin();
    const auto* input_fade_out_channel = input_fade_out[channel].begin();
    auto* output_channel = ((*output)[channel]).begin();
    MultiplyPointwise(num_frames, gain_fade_in_channel, input_fade_in_channel,
                      output_channel);
    MultiplyAndAccumulatePointwise(num_frames, gain_fade_out_channel,
                                   input_fade_out_channel, output_channel);
  }
}

}  // namespace vraudio
