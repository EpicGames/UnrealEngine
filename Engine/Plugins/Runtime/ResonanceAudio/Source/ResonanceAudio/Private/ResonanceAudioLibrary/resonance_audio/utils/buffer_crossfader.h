/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_BUFFER_CROSSFADER_H_
#define RESONANCE_AUDIO_UTILS_BUFFER_CROSSFADER_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Class that takes two input buffers and produces an output buffer by applying
// linear crossfade between the inputs.
class BufferCrossfader {
 public:
  explicit BufferCrossfader(size_t num_frames);

  // Applies linear crossfade for given input buffers and stores the result in
  // |output| buffer. Note that, in-place processing is *not* supported by this
  // method, the output buffer must be different than the input buffers.
  //
  // @param input_fade_in Input buffer to fade-in to.
  // @param input_fade_out Input buffer to fade-out from.
  // @param output Output buffer to store the crossfaded result.
  void ApplyLinearCrossfade(const AudioBuffer& input_fade_in,
                            const AudioBuffer& input_fade_out,
                            AudioBuffer* output) const;

 private:
  // Stereo audio buffer to store crossfade decay and growth multipliers.
  AudioBuffer crossfade_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_BUFFER_CROSSFADER_H_
