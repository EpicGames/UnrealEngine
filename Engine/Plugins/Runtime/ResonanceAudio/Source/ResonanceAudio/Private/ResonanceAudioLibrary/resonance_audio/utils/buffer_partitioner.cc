/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "utils/buffer_partitioner.h"

#include "utils/planar_interleaved_conversion.h"

namespace vraudio {

BufferPartitioner::BufferPartitioner(size_t num_channels,
                                     size_t frames_per_buffer,
                                     NewBufferCallback buffer_callback)
    : num_channels_(num_channels),
      frames_per_buffer_(frames_per_buffer),
      buffer_callback_(std::move(buffer_callback)),
      current_buffer_ptr_(nullptr),
      current_buffer_write_position_frames_(0),
      planar_channel_ptrs_(num_channels) {}

size_t BufferPartitioner::GetNumGeneratedBuffersForNumInputFrames(
    size_t num_input_frames) const {
  return (current_buffer_write_position_frames_ + num_input_frames) /
         frames_per_buffer_;
}

void BufferPartitioner::AddBuffer(const int16* interleaved_buffer,
                                  size_t num_channels, size_t num_frames) {
  AddBufferTemplated<const int16*>(interleaved_buffer, num_channels,
                                   num_frames);
}

void BufferPartitioner::AddBuffer(const float* interleaved_buffer,
                                  size_t num_channels, size_t num_frames) {
  AddBufferTemplated<const float*>(interleaved_buffer, num_channels,
                                   num_frames);
}

void BufferPartitioner::AddBuffer(const float* const* planar_buffer,
                                  size_t num_channels, size_t num_frames) {
  AddBufferTemplated<const float* const*>(planar_buffer, num_channels,
                                          num_frames);
}

void BufferPartitioner::AddBuffer(const int16* const* planar_buffer,
                                  size_t num_channels, size_t num_frames) {
  AddBufferTemplated<const int16* const*>(planar_buffer, num_channels,
                                          num_frames);
}

void BufferPartitioner::AddBuffer(const AudioBuffer& audio_buffer) {
  AddBuffer(audio_buffer.num_frames(), audio_buffer);
}

void BufferPartitioner::AddBuffer(size_t num_valid_frames,
                                  const AudioBuffer& audio_buffer) {
  DCHECK_EQ(audio_buffer.num_channels(), num_channels_);
  DCHECK_LE(num_valid_frames, audio_buffer.num_frames());
  for (size_t channel = 0; channel < num_channels_; ++channel) {
    planar_channel_ptrs_[channel] = &audio_buffer[channel][0];
  }
  AddBuffer(planar_channel_ptrs_.data(), audio_buffer.num_channels(),
            num_valid_frames);
}

size_t BufferPartitioner::GetNumBufferedFrames() const {
  return current_buffer_write_position_frames_;
}

size_t BufferPartitioner::Flush() {
  if (current_buffer_write_position_frames_ == 0 ||
      current_buffer_ptr_ == nullptr) {
    return 0;
  }
  DCHECK_LE(current_buffer_write_position_frames_,
            current_buffer_ptr_->num_frames());
  const size_t num_zeropadded_frames =
      current_buffer_ptr_->num_frames() - current_buffer_write_position_frames_;
  for (AudioBuffer::Channel& channel : *current_buffer_ptr_) {
    DCHECK_LE(current_buffer_write_position_frames_, channel.size());
    std::fill(channel.begin() + current_buffer_write_position_frames_,
              channel.end(), 0.0f);
  }
  current_buffer_ptr_ = buffer_callback_(current_buffer_ptr_);
  current_buffer_write_position_frames_ = 0;
  return num_zeropadded_frames;
}

void BufferPartitioner::Clear() {
  current_buffer_ptr_ = nullptr;
  current_buffer_write_position_frames_ = 0;
}

template <typename BufferType>
void BufferPartitioner::AddBufferTemplated(BufferType buffer,
                                           size_t num_channels,
                                           size_t num_frames) {
  DCHECK_EQ(num_channels, num_channels_);

  size_t input_read_frame = 0;
  while (input_read_frame < num_frames) {
    if (current_buffer_ptr_ == nullptr) {
      current_buffer_ptr_ = buffer_callback_(nullptr);
      if (current_buffer_ptr_ == nullptr) {
        LOG(WARNING) << "No input buffer received";
        return;
      }
      DCHECK_EQ(current_buffer_ptr_->num_frames(), frames_per_buffer_);
      DCHECK_EQ(current_buffer_ptr_->num_channels(), num_channels_);
      current_buffer_write_position_frames_ = 0;
    }
    DCHECK_GT(frames_per_buffer_, current_buffer_write_position_frames_);
    const size_t remaining_frames_in_temp_buffer =
        frames_per_buffer_ - current_buffer_write_position_frames_;
    const size_t remaining_frames_in_input_buffer =
        num_frames - input_read_frame;
    const size_t num_frames_to_process = std::min(
        remaining_frames_in_temp_buffer, remaining_frames_in_input_buffer);

    FillAudioBufferWithOffset(buffer, num_frames, num_channels_,
                              input_read_frame,
                              current_buffer_write_position_frames_,
                              num_frames_to_process, current_buffer_ptr_);

    input_read_frame += num_frames_to_process;

    // Update write pointers in temporary buffer.
    current_buffer_write_position_frames_ += num_frames_to_process;
    if (current_buffer_write_position_frames_ == frames_per_buffer_) {
      // Current buffer is filled with data -> pass it to callback.
      current_buffer_ptr_ = buffer_callback_(current_buffer_ptr_);
      current_buffer_write_position_frames_ = 0;
      if (current_buffer_ptr_ == nullptr) {
        LOG(WARNING) << "No input buffer received";
        return;
      }
      DCHECK_EQ(current_buffer_ptr_->num_frames(), frames_per_buffer_);
      DCHECK_EQ(current_buffer_ptr_->num_channels(), num_channels_);
    }
  }
}

}  // namespace vraudio
