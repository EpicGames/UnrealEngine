/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_BUFFER_PARTITIONER_H_
#define RESONANCE_AUDIO_UTILS_BUFFER_PARTITIONER_H_

#include <functional>
#include <memory>

#include "base/audio_buffer.h"
#include "base/logging.h"

namespace vraudio {

// Packages input buffers of arbitrary sizes into fixed size |AudioBuffer|s.
class BufferPartitioner {
 public:
  // Callback to receive processed and assign empty |AudioBuffer|s while
  // processing input buffers.
  //
  // @param output Pointer to partitioned |AudioBuffer| with input audio data.
  // @return Pointer to the next |AudioBuffer| to be filled.
  typedef std::function<AudioBuffer*(AudioBuffer* output)> NewBufferCallback;

  // Constructor.
  //
  // @param num_channels Number of audio channels in input and output buffers.
  // @param frames_per_buffer Number of frames in output |AudioBuffer|s.
  // @param buffer_callback Callback to receive output |AudioBuffer|s.
  BufferPartitioner(size_t num_channels, size_t frames_per_buffer,
                    NewBufferCallback buffer_callback);

  // Predicts the number of generated buffers for a given number of input frames
  // and based on the current fill state of the internal |temp_buffer_|.
  //
  // @param num_input_frames Number of input frames.
  // @return Number of generated output buffers.
  size_t GetNumGeneratedBuffersForNumInputFrames(size_t num_input_frames) const;

  // Returns the number of buffered frames in internal |temp_buffer_|.
  size_t GetNumBufferedFrames() const;

  // Adds an interleaved int16 input buffer. This method triggers
  // |NewBufferCallback| whenever a new |AudioBuffer| has been generated.
  //
  // @param interleaved_buffer Interleaved input buffer.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  void AddBuffer(const int16* interleaved_buffer, size_t num_channels,
                 size_t num_frames);

  // Adds an interleaved float input buffer. This method triggers
  // |NewBufferCallback| whenever a new |AudioBuffer| has been generated.
  //
  // @param interleaved_buffer Interleaved input buffer.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  void AddBuffer(const float* interleaved_buffer, size_t num_channels,
                 size_t num_frames);

  // Adds a planar float input buffer. This method triggers
  // |NewBufferCallback| whenever a new |AudioBuffer| has been generated.
  //
  // @param planar_buffer Pointer to array of pointers for each audio channel.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  void AddBuffer(const float* const* planar_buffer, size_t num_channels,
                 size_t num_frames);

  // Adds a planar int16 input buffer. This method triggers
  // |NewBufferCallback| whenever a new |AudioBuffer| has been generated.
  //
  // @param planar_buffer Pointer to array of pointers for each audio channel.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  void AddBuffer(const int16* const* planar_buffer, size_t num_channels,
                 size_t num_frames);

  // Adds an |AudioBuffer|. This method triggers |NewBufferCallback| whenever a
  // new |AudioBuffer| has been generated.
  //
  // @param audio_buffer Planar |AudioBuffer| input.
  void AddBuffer(const AudioBuffer& audio_buffer);

  // Adds an |AudioBuffer| and specifies how many of that audio buffers frames
  // are valid. This method triggers |NewBufferCallback| whenever a new
  // |AudioBuffer| has been generated.
  //
  // @param num_valid_frames Indicates the number of frames which are actually
  //     valid in the audio buffer passed. Frames after this will be ignored.
  // @param audio_buffer Planar |AudioBuffer| input.
  void AddBuffer(size_t num_valid_frames, const AudioBuffer& audio_buffer);

  // Flushes the internal temporary |AudioBuffer| by filling the remaining
  // audio frames with silence.
  //
  // @return Number of zero padded audio frames. Zero if the internal buffer
  //    is empty.
  size_t Flush();

  // Clears internal temporary buffer that holds remaining audio frames.
  void Clear();

 private:
  // Adds an interleaved and planar float and int16 input buffer as well as
  // planar |AudioBuffer| input. This method triggers |NewBufferCallback|
  // whenever a new |AudioBuffer| has been generated.
  //
  // @tparam BufferType Input buffer type.
  // @param buffer Input buffer.
  // @param num_channels Number of channels in input buffer.
  // @param num_frames Number of frames in input buffer.
  template <typename BufferType>
  void AddBufferTemplated(BufferType buffer, size_t num_channels,
                          size_t num_frames);

  // Number of channels in input buffers.
  const size_t num_channels_;

  // Number of frames per buffer in output buffers.
  const size_t frames_per_buffer_;

  // Callback to output generated |AudioBuffer|s.
  const NewBufferCallback buffer_callback_;

  // Temporary buffer to remaining samples from input buffers.
  AudioBuffer* current_buffer_ptr_;  // Not owned.

  // Current write position in frames in temporary buffer.
  size_t current_buffer_write_position_frames_;

  // Helper vector to obtain an array of planar channel pointers from an
  // |AudioBuffer|.
  std::vector<const float*> planar_channel_ptrs_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_BUFFER_PARTITIONER_H_
