/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "utils/buffer_unpartitioner.h"

#include "utils/planar_interleaved_conversion.h"
#include "utils/sample_type_conversion.h"

namespace vraudio {

BufferUnpartitioner::BufferUnpartitioner(size_t num_channels,
                                         size_t frames_per_buffer,
                                         GetBufferCallback buffer_callback)
    :
#if !DISABLE_CHECKS
	num_channels_(num_channels),
#endif
      frames_per_buffer_(frames_per_buffer),
      buffer_callback_(std::move(buffer_callback)),
      current_input_buffer_(nullptr),
      current_buffer_read_offset_frames_(0) {
  DCHECK_GT(frames_per_buffer_, 0);
  DCHECK_GT(num_channels, 0);
}

size_t BufferUnpartitioner::GetNumBuffersRequestedForNumInputFrames(
    size_t num_output_frames) const {
  if (num_output_frames == 0) {
    return 0;
  }
  return (num_output_frames - GetNumFramesAvailableInBuffer() +
          frames_per_buffer_ - 1) /
         frames_per_buffer_;
}

size_t BufferUnpartitioner::GetBuffer(int16* output_buffer, size_t num_channels,
                                      size_t num_frames) {
  return GetBufferTemplated<int16*>(output_buffer, num_channels, num_frames);
}

size_t BufferUnpartitioner::GetBuffer(float* output_buffer, size_t num_channels,
                                      size_t num_frames) {
  return GetBufferTemplated<float*>(output_buffer, num_channels, num_frames);
}

size_t BufferUnpartitioner::GetBuffer(int16** output_buffer,
                                      size_t num_channels, size_t num_frames) {
  return GetBufferTemplated<int16**>(output_buffer, num_channels, num_frames);
}

size_t BufferUnpartitioner::GetBuffer(float** output_buffer,
                                      size_t num_channels, size_t num_frames) {
  return GetBufferTemplated<float**>(output_buffer, num_channels, num_frames);
}

size_t BufferUnpartitioner::GetNumBufferedFrames() const {
  return current_buffer_read_offset_frames_;
}

size_t BufferUnpartitioner::GetNumFramesAvailableInBuffer() const {
  if (current_input_buffer_ == nullptr) {
    return 0;
  }
  DCHECK_GE(current_input_buffer_->num_frames(),
            current_buffer_read_offset_frames_);
  return current_input_buffer_->num_frames() -
         current_buffer_read_offset_frames_;
}

void BufferUnpartitioner::Clear() {
  current_input_buffer_ = nullptr;
  current_buffer_read_offset_frames_ = 0;
}

template <typename BufferType>
size_t BufferUnpartitioner::GetBufferTemplated(BufferType buffer,
                                               size_t num_channels,
                                               size_t num_frames) {
  DCHECK_EQ(num_channels, num_channels_);

  size_t num_copied_frames = 0;
  while (num_copied_frames < num_frames) {
    if (current_input_buffer_ == nullptr) {
      current_input_buffer_ = buffer_callback_();
      if (current_input_buffer_ == nullptr) {
        // No more input |AudioBuffer|s are available.
        return num_copied_frames;
      }
      DCHECK_EQ(frames_per_buffer_, current_input_buffer_->num_frames());
      current_buffer_read_offset_frames_ = 0;
    }
    DCHECK_GE(frames_per_buffer_, current_buffer_read_offset_frames_);
    const size_t remaining_frames_in_input_buffer =
        num_frames - num_copied_frames;
    DCHECK_GE(current_input_buffer_->num_frames(),
              current_buffer_read_offset_frames_);
    const size_t num_frames_to_process =
        std::min(current_input_buffer_->num_frames() -
                     current_buffer_read_offset_frames_,
                 remaining_frames_in_input_buffer);

    FillExternalBufferWithOffset(
        *current_input_buffer_, current_buffer_read_offset_frames_, buffer,
        num_frames, num_channels, num_copied_frames, num_frames_to_process);

    num_copied_frames += num_frames_to_process;

    current_buffer_read_offset_frames_ += num_frames_to_process;
    if (current_buffer_read_offset_frames_ ==
        current_input_buffer_->num_frames()) {
      current_input_buffer_ = nullptr;
    }
  }
  return num_copied_frames;
}

}  // namespace vraudio
