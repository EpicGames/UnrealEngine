/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_BUFFER_UNPARTITIONER_H_
#define RESONANCE_AUDIO_UTILS_BUFFER_UNPARTITIONER_H_

#include <functional>
#include <memory>

#include "base/audio_buffer.h"
#include "base/logging.h"

namespace vraudio {

// Unpackages |AudioBuffer| into output buffers of arbitrary sizes.
class BufferUnpartitioner {
 public:
  // Callback to obtain input |AudioBuffer|s. Returns a nullptr if no buffers
  // are available.
  typedef std::function<const AudioBuffer*()> GetBufferCallback;

  // Constructor.
  //
  // @param num_channels Number of audio channels in input and output buffers.
  // @param frames_per_buffer Number of frames per input buffer.
  // @param buffer_callback Callback to receive output |AudioBuffer|s.
  BufferUnpartitioner(size_t num_channels, size_t frames_per_buffer,
                      GetBufferCallback buffer_callback);

  // Returns the number of input buffers required for a given number of output
  // frames and based on the current fill state of the internal |temp_buffer_|.
  //
  // @param num_output_frames Number of output frames.
  // @return Number of required input |AudioBuffer|s.
  size_t GetNumBuffersRequestedForNumInputFrames(
      size_t num_output_frames) const;

  // Returns the number of buffered frames in internal |temp_buffer_|.
  size_t GetNumBufferedFrames() const;

  // Requests an interleaved int16 output buffer. This method triggers
  // |GetBufferCallback|.
  //
  // @param output_buffer Interleaved output buffer to write to.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Number of frames in output buffer.
  // @return Number of frames actually written.
  size_t GetBuffer(int16* output_buffer, size_t num_channels,
                   size_t num_frames);

  // Requests an interleaved float output buffer. This method triggers
  // |GetBufferCallback|.
  //
  // @param output_buffer Interleaved output buffer to write to.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Number of frames in output buffer.
  // @return Number of frames actually written.
  size_t GetBuffer(float* output_buffer, size_t num_channels,
                   size_t num_frames);

  // Requests a planar int16 output buffer. This method triggers
  // |GetBufferCallback|.
  //
  // @param output_buffer Planar output buffer to write to.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Number of frames in output buffer.
  // @return Number of frames actually written.
  size_t GetBuffer(int16** output_buffer, size_t num_channels,
                   size_t num_frames);

  // Requests a planar float output buffer. This method triggers
  // |GetBufferCallback|.
  //
  // @param output_buffer Planar output buffer to write to.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Number of frames in output buffer.
  // @return Number of frames actually written.
  size_t GetBuffer(float** output_buffer, size_t num_channels,
                   size_t num_frames);

  // Clears internal temporary buffer that holds remaining audio frames.
  void Clear();

 private:
  // Returns the number of frames that are buffered in |current_input_buffer_|.
  //
  // @return Number of frames in |current_input_buffer_|. If
  //     |current_input_buffer_| is undefined, it returns 0.
  size_t GetNumFramesAvailableInBuffer() const;

  // Writes output buffer into target buffer. Supported buffer types are planar
  // and interleaved floating point abd interleaved int16 output. This method
  // triggers |GetBufferCallback|.
  //
  // @tparam BufferType Output buffer type.
  // @param buffer Output buffer to write to.
  // @param num_channels Number of channels in output buffer.
  // @param num_frames Number of frames in output buffer.
  // @return Number of frames actually written.
  template <typename BufferType>
  size_t GetBufferTemplated(BufferType buffer, size_t num_channels,
                            size_t num_frames);

#if !DISABLE_CHECKS
  // Number of channels in output buffers.
  const size_t num_channels_;
#endif

  // Number of frames per buffer in output buffers.
  const size_t frames_per_buffer_;

  // Callback to request input |AudioBuffer|s.
  const GetBufferCallback buffer_callback_;

  // Temporary buffer containing remaining audio frames.
  const AudioBuffer* current_input_buffer_;  // Not owned.

  // Current read position in |current_input_buffer_|.
  size_t current_buffer_read_offset_frames_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_BUFFER_UNPARTITIONER_H_
