/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4702) /* unreachable code */
#pragma warning(disable:4100) /* unreferenced formal parameter */
#endif

#include "utils/lockless_task_queue.h"

#include "base/logging.h"

#include <mutex>

namespace vraudio {

LocklessTaskQueue::LocklessTaskQueue(size_t max_tasks) : max_tasks_(max_tasks) {
  CHECK_GT(max_tasks, 0U);
  Init(max_tasks);
}

LocklessTaskQueue::~LocklessTaskQueue() { Clear(); }

void LocklessTaskQueue::Post(Task&& task) {
  std::lock_guard<std::mutex> scope_lock(tasks_mutex_);
  Node* const free_node = PopNodeFromList(&free_list_head_);
  if (free_node == nullptr) {
    LOG(WARNING) << "Queue capacity reached - dropping task";
    return;
  }
  free_node->task = std::move(task);
  PushNodeToList(&task_list_head_, free_node);
}

void LocklessTaskQueue::Execute() {
  Node* const old_task_list_head_ptr = task_list_head_.exchange(nullptr);
  ProcessTaskList(old_task_list_head_ptr, true /*execute_tasks*/);
}

void LocklessTaskQueue::Clear() {
  Node* const old_task_list_head_ptr = task_list_head_.exchange(nullptr);
  ProcessTaskList(old_task_list_head_ptr, false /*execute_tasks*/);
}

void LocklessTaskQueue::PushNodeToList(std::atomic<Node*>* list_head,
                                       Node* node) {
  DCHECK(list_head);
  DCHECK(node);
  Node* list_head_ptr;
  do {
    list_head_ptr = list_head->load();
    node->next = list_head_ptr;
  } while (!std::atomic_compare_exchange_strong_explicit(
      list_head, &list_head_ptr, node, std::memory_order_release,
      std::memory_order_relaxed));
}

LocklessTaskQueue::Node* LocklessTaskQueue::PopNodeFromList(
    std::atomic<Node*>* list_head) {
  DCHECK(list_head);
  Node* list_head_ptr;
  Node* list_head_next_ptr;
  do {
    list_head_ptr = list_head->load();
    if (list_head_ptr == nullptr) {
      // End of list reached.
      return nullptr;
    }
    list_head_next_ptr = list_head_ptr->next.load();
  } while (!std::atomic_compare_exchange_strong_explicit(
      list_head, &list_head_ptr, list_head_next_ptr, std::memory_order_relaxed,
      std::memory_order_relaxed));
  return list_head_ptr;
}

void LocklessTaskQueue::ProcessTaskList(Node* list_head, bool execute) {
  std::lock_guard<std::mutex> scope_lock(tasks_mutex_);

  Node* node_itr = list_head;
  while (node_itr != nullptr) {
    Node* next_node_ptr = node_itr->next;
    if (temp_tasks_.size() < max_tasks_) {
      temp_tasks_.emplace_back(std::move(node_itr->task));
    } else{
      LOG(WARNING) << "temp_tasks_ vector size is larger than max_tasks_, dropping all remaining tasks";
    }

    node_itr->task = Task();
    PushNodeToList(&free_list_head_, node_itr);
    node_itr = next_node_ptr;
  }

  if (execute) {
    // Execute tasks in reverse order.
    for (std::vector<Task>::reverse_iterator task_itr = temp_tasks_.rbegin();
         task_itr != temp_tasks_.rend(); ++task_itr) {
      if (*task_itr != nullptr) {
        (*task_itr)();
      }
    }
  }
  temp_tasks_.clear();
  static bool bHasWarned = false;
  if (bHasWarned && max_tasks_ && !temp_tasks_.capacity()) {
    LOG(WARNING) << "std::vector::clear() on this platform has zero'd out the temp_tasks_ capacity, (this will cause re-allocation in every ProcessTaskList call)";
    bHasWarned = false;
  }
}

void LocklessTaskQueue::Init(size_t num_nodes) {
  std::lock_guard<std::mutex> scope_lock(tasks_mutex_);
  nodes_.resize(num_nodes);
  temp_tasks_.reserve(num_nodes);

  // Initialize free list.
  free_list_head_ = &nodes_[0];
  for (size_t i = 0; i < num_nodes - 1; ++i) {
    nodes_[i].next = &nodes_[i + 1];
  }
  nodes_[num_nodes - 1].next = nullptr;

  // Initialize task list.
  task_list_head_ = nullptr;
}

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif