/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_LOCKLESS_TASK_QUEUE_H_
#define RESONANCE_AUDIO_UTILS_LOCKLESS_TASK_QUEUE_H_

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4702) /* unreachable code */
#pragma warning(disable:4100) /* unreferenced formal parameter */
#endif

#include <stddef.h>

#include <atomic>
#include <mutex>
#include <functional>
#include <vector>

namespace vraudio {

// Lock-less task queue which is thread safe for cocurrent task producers and a
// single task executor thread.
class LocklessTaskQueue {
 public:
  // Alias for the task closure type.
  typedef std::function<void()> Task;

  // Constructor. Preallocates nodes on the task queue list.
  //
  // @param max_tasks Maximum number of tasks on the task queue.
  explicit LocklessTaskQueue(size_t max_tasks);

  ~LocklessTaskQueue();

  // Posts a new task to task queue.
  //
  // @param task Task to process.
  void Post(Task&& task);

  // Executes all tasks on the task queue.
  void Execute();

  // Removes all tasks on the task queue.
  void Clear();

 private:
  // Node to model a single-linked list.
  struct Node {
    Node() = default;

    // Dummy copy constructor to enable vector::resize allocation.
    Node(const Node& /*node*/) : next() {}

    // User task.
    LocklessTaskQueue::Task task;

    // Pointer to next node.
    std::atomic<Node*> next;
  };

  // Pushes a node to the front of a list.
  //
  // @param list_head Pointer to list head.
  // @param node Node to be pushed to the front of the list.
  void PushNodeToList(std::atomic<Node*>* list_head, Node* node);

  // Pops a node from the front of a list.
  //
  // @param list_head Pointer to list head.
  // @return Front node, nullptr if list is empty.
  Node* PopNodeFromList(std::atomic<Node*>* list_head);

  // Iterates over list and moves all tasks to |temp_tasks_| to be executed in
  // FIFO order. All processed nodes are pushed back to the free list.
  //
  // @param list_head Head node of list to be processed.
  // @param execute If true, tasks on task list are executed.
  void ProcessTaskList(Node* list_head, bool execute);

  // Initializes task queue structures and preallocates task queue nodes.
  //
  // @param num_nodes Number of nodes to be initialized on free list.
  void Init(size_t num_nodes);

  // Pointer to head node of free list.
  std::atomic<Node*> free_list_head_;

  // Pointer to head node of task list.
  std::atomic<Node*> task_list_head_;

  // Holds preallocated nodes.
  std::vector<Node> nodes_;

  // Temporary vector to hold |Task|s in order to execute them in reverse order
  // (FIFO, instead of LIFO).
  std::vector<Task> temp_tasks_;

  // this mutex guarantees that temp_tasks_ is not mutated while being iterated on.
  std::mutex tasks_mutex_;
  size_t max_tasks_;
};

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif

#endif  // RESONANCE_AUDIO_UTILS_LOCKLESS_TASK_QUEUE_H_
