/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "utils/ogg_vorbis_recorder.h"

#include "base/logging.h"
#include "utils/planar_interleaved_conversion.h"

namespace vraudio {

OggVorbisRecorder::OggVorbisRecorder(int sample_rate_hz, size_t num_channels,
                                     size_t num_frames, size_t max_num_buffers)
    : sample_rate_hz_(sample_rate_hz),
      num_channels_(num_channels),
      num_frames_(num_frames),
      max_num_buffers_(max_num_buffers),
      temp_data_channel_ptrs_(num_channels),
      crossfader_(num_frames_),
      crossfade_buffer_(num_channels_, num_frames_) {
  DCHECK_GT(sample_rate_hz_, 0);
  DCHECK_NE(num_channels_, 0U);
  DCHECK_NE(num_frames_, 0U);
  CHECK_NE(max_num_buffers_, 0U);
}

void OggVorbisRecorder::AddInput(std::unique_ptr<AudioBuffer> input_buffer) {
  DCHECK(input_buffer);
  DCHECK_EQ(input_buffer->num_channels(), num_channels_);
  DCHECK_EQ(input_buffer->num_frames(), num_frames_);

  if (data_.size() == max_num_buffers_) {
    LOG(WARNING) << "Maximum input buffer limit reached, overwriting data";
    data_.erase(data_.begin());
  }
  data_.push_back(std::move(input_buffer));
}

void OggVorbisRecorder::Reset() { data_.clear(); }

bool OggVorbisRecorder::WriteToFile(const std::string& file_path, float quality,
                                    bool seamless) {
  if (data_.empty()) {
    LOG(WARNING) << "No recorded data";
    return false;
  }

  if (seamless) {
    MakeSeamless();
  }

  if (!encoder_.InitializeForFile(
          file_path, num_channels_, sample_rate_hz_,
          VorbisStreamEncoder::EncodingMode::kVariableBitRate, 0 /* bitrate */,
          quality)) {
    LOG(WARNING) << "Cannot initialize file to record: " << file_path;
    Reset();
    return false;
  }
  for (const auto& audio_buffer : data_) {
    GetRawChannelDataPointersFromAudioBuffer(*audio_buffer,
                                             &temp_data_channel_ptrs_);
    if (!encoder_.AddPlanarBuffer(temp_data_channel_ptrs_.data(), num_channels_,
                                  num_frames_)) {
      LOG(WARNING) << "Failed to write buffer into file: " << file_path;
      Reset();
      return false;
    }
  }
  if (!encoder_.FlushAndClose()) {
    LOG(WARNING) << "Failed to safely close file: " << file_path;
    Reset();
    return false;
  }

  Reset();
  return true;
}

void OggVorbisRecorder::MakeSeamless() {
  if (data_.size() == 1) {
    LOG(WARNING) << "Not enough data to make seamless file";
    return;
  }
  // Apply crossfade between the beginning and the end buffers of |data_|.
  auto* front_buffer = data_.front().get();
  const auto& back_buffer = *data_.back();
  crossfader_.ApplyLinearCrossfade(*front_buffer, back_buffer,
                                   &crossfade_buffer_);

  *front_buffer = crossfade_buffer_;
  data_.pop_back();
}

}  // namespace vraudio
