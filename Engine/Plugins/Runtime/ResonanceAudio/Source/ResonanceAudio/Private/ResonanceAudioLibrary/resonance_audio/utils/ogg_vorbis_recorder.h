/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_OGG_VORBIS_RECORDER_H_
#define RESONANCE_AUDIO_UTILS_OGG_VORBIS_RECORDER_H_

#include <memory>
#include <vector>

#include "base/audio_buffer.h"
#include "utils/buffer_crossfader.h"
#include "utils/vorbis_stream_encoder.h"

namespace vraudio {

// Class that takes audio buffers as input and writes them into a compressed OGG
// Vorbis file.

class OggVorbisRecorder {
 public:
  // Constructs a new recorder with given sampling rate and number of channels.

  OggVorbisRecorder(int sample_rate_hz, size_t num_channels, size_t num_frames,
                    size_t max_num_buffers);

  // Adds next input buffer at the end of the record data.
  //
  // @param input_buffer Next audio buffer to be recorded.
  void AddInput(std::unique_ptr<AudioBuffer> input_buffer);

  // Flushes the record data.
  void Reset();

  // Writes the current record data into a file and resets the recorder.
  //
  // @param file_path Full path of the file to be recorded into.
  // @param quality Compression quality of the record. The usable range is from
  //     -0.1 (lowest quality, smallest file) to 1.0 (highest quality, largest
  //     file).
  // @param seamless True to record seamlessly for looping. Note that this
  //     option will truncate the record length by |num_frames_| samples.
  // @return False if fails to successfully write data into |file_path|.
  bool WriteToFile(const std::string& file_path, float quality, bool seamless);

 private:
  // Helper method to make record data seamless.
  void MakeSeamless();

  // Record sampling rate.
  const int sample_rate_hz_;

  // Record number of channels.
  const size_t num_channels_;

  // Record number of frames per buffer.
  const size_t num_frames_;

  // Maximum number of input buffers allowed to record.
  const size_t max_num_buffers_;

  // Record data that is stored as a list of planar audio buffers.
  std::vector<std::unique_ptr<AudioBuffer>> data_;

  // Temporary vector to extract pointers to planar channels in an
  // |AudioBuffer|.
  std::vector<const float*> temp_data_channel_ptrs_;

  // Buffer crossfader that is used to create seamless loop.
  BufferCrossfader crossfader_;

  // Temporary buffer to store the crossfaded output.

  AudioBuffer crossfade_buffer_;

  // OGG Vorbis encoder to write record data into file in compressed format.
  VorbisStreamEncoder encoder_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_OGG_VORBIS_RECORDER_H_
