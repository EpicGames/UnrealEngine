/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "utils/planar_interleaved_conversion.h"

#include "base/constants_and_types.h"
#include "base/logging.h"
#include "base/simd_utils.h"

#include "utils/sample_type_conversion.h"

namespace vraudio {

namespace {

template <typename InputType, typename OutputType>
void ConvertInterleavedToPlanarTemplated(
    InputType interleaved_buffer, size_t num_input_frames,
    size_t num_input_channels, size_t input_offset_frames,
    const std::vector<size_t>* channel_map, OutputType output_buffer,
    size_t num_output_frames, size_t num_output_channels,
    size_t output_offset_frames, size_t num_frames_to_copy) {

#if !DISABLE_CHECKS
  DCHECK_GE(num_input_frames, input_offset_frames);
  const size_t max_num_input_frames = num_input_frames - input_offset_frames;
  DCHECK_GE(num_output_frames, output_offset_frames);
  const size_t max_num_output_frames = num_output_frames - output_offset_frames;
  DCHECK_GE(max_num_input_frames, num_frames_to_copy);
  DCHECK_GE(max_num_output_frames, num_frames_to_copy);

  if (channel_map == nullptr) {
    DCHECK_EQ(num_input_channels, num_output_channels);
  } else {
    DCHECK_GE(channel_map->size(), num_output_channels);
  }
#endif

  InputType interleaved_buffer_with_offset =
      interleaved_buffer + input_offset_frames * num_input_channels;

  if (num_input_channels == kNumStereoChannels &&
      num_output_channels == kNumStereoChannels) {
    if (channel_map == nullptr) {
      DeinterleaveStereo(num_frames_to_copy, interleaved_buffer_with_offset,
                         &output_buffer[0][output_offset_frames],
                         &output_buffer[1][output_offset_frames]);
    } else {
      DCHECK_LT((*channel_map)[0], kNumStereoChannels);
      DCHECK_LT((*channel_map)[1], kNumStereoChannels);
      DeinterleaveStereo(
          num_input_frames, interleaved_buffer_with_offset,
          &output_buffer[(*channel_map)[0]][output_offset_frames],
          &output_buffer[(*channel_map)[1]][output_offset_frames]);
    }
  } else {
    for (size_t channel_idx = 0; channel_idx < num_output_channels;
         ++channel_idx) {
      const size_t input_channel =
          channel_map != nullptr ? (*channel_map)[channel_idx] : channel_idx;
      DCHECK_LT(input_channel, num_input_channels);
      InputType input_ptr = &interleaved_buffer_with_offset[input_channel];
      float* output_ptr = &output_buffer[channel_idx][output_offset_frames];
      for (size_t frame = 0; frame < num_frames_to_copy; ++frame) {
        ConvertSampleToFloatFormat(*input_ptr, output_ptr);
        input_ptr += num_input_channels;
        ++output_ptr;
      }
    }
  }
}

template <typename PlanarInputType, typename PlanarOutputType>
void ConvertPlanarToPlanarTemplated(
    PlanarInputType input, size_t num_input_frames, size_t num_input_channels,
    size_t input_offset_frames, const std::vector<size_t>* channel_map,
    PlanarOutputType planar_output_ptrs, size_t num_output_frames,
    size_t num_output_channels, size_t output_offset_frames,
    size_t num_frames_convert_and_copy) {

#if !DISABLE_CHECKS
  DCHECK_GE(num_input_frames, input_offset_frames);
  const size_t max_num_input_frames = num_input_frames - input_offset_frames;
  DCHECK_GE(num_output_frames, output_offset_frames);
  const size_t max_num_output_frames = num_output_frames - output_offset_frames;
  DCHECK_GE(max_num_input_frames, num_frames_convert_and_copy);
  DCHECK_GE(max_num_output_frames, num_frames_convert_and_copy);

  if (channel_map == nullptr) {
    DCHECK_EQ(num_input_channels, num_output_channels);
  } else {
    DCHECK_GE(channel_map->size(), num_output_channels);
  }
#endif

  for (size_t channel = 0; channel < num_output_channels; ++channel) {
    const size_t input_channel =
        channel_map != nullptr ? (*channel_map)[channel] : channel;
    DCHECK_LT(input_channel, num_input_channels);
    ConvertPlanarSamples(num_frames_convert_and_copy,
                         &input[input_channel][input_offset_frames],
                         &planar_output_ptrs[channel][output_offset_frames]);
  }
}

template <typename PlanarInputType, typename InterleavedOutputType>
void ConvertPlanarToInterleavedTemplated(
    PlanarInputType input, size_t num_input_frames, size_t num_input_channels,
    size_t input_offset_frames, InterleavedOutputType interleaved_output_ptr,
    size_t num_output_frames, size_t num_output_channels,
    size_t output_offset_frames, size_t num_frames_convert_and_copy) {

#if !DISABLE_CHECKS
  DCHECK(interleaved_output_ptr);
  DCHECK_GE(num_input_frames, input_offset_frames);
  const size_t max_num_input_frames = num_input_frames - input_offset_frames;
  DCHECK_GE(num_output_frames, output_offset_frames);
  const size_t max_num_output_frames = num_output_frames - output_offset_frames;
  DCHECK_GE(max_num_input_frames, num_frames_convert_and_copy);
  DCHECK_GE(max_num_output_frames, num_frames_convert_and_copy);
  DCHECK_EQ(num_input_channels, num_output_channels);
#endif

  InterleavedOutputType interleaved_output_ptr_with_offset =
      interleaved_output_ptr + output_offset_frames * num_output_channels;

  if (num_input_channels == kNumStereoChannels &&
      num_output_channels == kNumStereoChannels) {
    const float* left_ptr = &input[0][input_offset_frames];
    const float* right_ptr = &input[1][input_offset_frames];
    InterleaveStereo(num_frames_convert_and_copy, left_ptr, right_ptr,
                     interleaved_output_ptr_with_offset);
  } else {
    for (size_t channel = 0; channel < num_output_channels; ++channel) {
      const float* input_channel_ptr = &input[channel][input_offset_frames];
      size_t interleaved_index = channel;
      for (size_t frame = 0; frame < num_frames_convert_and_copy; ++frame) {
        ConvertSampleFromFloatFormat(
            input_channel_ptr[frame],
            &interleaved_output_ptr_with_offset[interleaved_index]);
        interleaved_index += num_output_channels;
      }
    }
  }
}

}  // namespace

void PlanarFromInterleaved(const float* interleaved_buffer,
                           size_t num_input_frames, size_t num_input_channels,
                           const std::vector<float*>& planar_buffer_ptr,
                           size_t num_output_frames) {
  DCHECK(interleaved_buffer);
  DCHECK_GT(planar_buffer_ptr.size(), 0);

  const size_t num_frames_to_copy =
      std::min(num_input_frames, num_output_frames);
  ConvertInterleavedToPlanarTemplated<const float*, float* const*>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/,
      planar_buffer_ptr.data(), num_output_frames, planar_buffer_ptr.size(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void PlanarFromInterleaved(const int16* interleaved_buffer,
                           size_t num_input_frames, size_t num_input_channels,
                           const std::vector<float*>& planar_buffer_ptr,
                           size_t num_output_frames) {
  DCHECK(interleaved_buffer);
  DCHECK_GT(planar_buffer_ptr.size(), 0);

  const size_t num_frames_to_copy =
      std::min(num_input_frames, num_output_frames);
  ConvertInterleavedToPlanarTemplated<const int16*, float* const*>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/,
      planar_buffer_ptr.data(), num_output_frames, planar_buffer_ptr.size(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void FillAudioBuffer(const float* interleaved_buffer, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertInterleavedToPlanarTemplated<const float*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void FillAudioBuffer(const int16* interleaved_buffer, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());

  ConvertInterleavedToPlanarTemplated<const int16*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void FillAudioBuffer(const std::vector<float>& interleaved_buffer,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(output);
  DCHECK_EQ(interleaved_buffer.size() % num_input_channels, 0);
  const size_t num_frames_to_copy = std::min(
      interleaved_buffer.size() / num_input_channels, output->num_frames());
  FillAudioBuffer(&interleaved_buffer[0], num_frames_to_copy,
                  num_input_channels, output);
}

void FillAudioBuffer(const std::vector<int16>& interleaved_buffer,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(output);
  DCHECK_EQ(interleaved_buffer.size() % num_input_channels, 0);
  const size_t num_frames_to_copy = std::min(
      interleaved_buffer.size() / num_input_channels, output->num_frames());
  FillAudioBuffer(&interleaved_buffer[0], num_frames_to_copy,
                  num_input_channels, output);
}

void FillAudioBuffer(const float* const* planar_ptrs, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(planar_ptrs);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertPlanarToPlanarTemplated<const float* const*, AudioBuffer&>(
      planar_ptrs, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void FillAudioBuffer(const int16* const* planar_ptrs, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output) {
  DCHECK(planar_ptrs);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertPlanarToPlanarTemplated<const int16* const*, AudioBuffer&>(
      planar_ptrs, num_input_frames, num_input_channels,
      0 /* input_offset_frames */, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(),
      0 /* output_offset_frames */, num_frames_to_copy);
}

void FillAudioBufferWithOffset(const float* interleaved_buffer,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  ConvertInterleavedToPlanarTemplated<const float*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      input_frame_offset, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(), output_frame_offset,
      num_frames_to_copy);
}

void FillAudioBufferWithOffset(const int16* interleaved_buffer,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  ConvertInterleavedToPlanarTemplated<const int16*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      input_frame_offset, nullptr /* channel_map*/, *output,
      output->num_frames(), output->num_channels(), output_frame_offset,
      num_frames_to_copy);
}

void FillAudioBufferWithOffset(const float* const* planar_ptrs,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output) {
  DCHECK(planar_ptrs);
  DCHECK(output);
  ConvertPlanarToPlanarTemplated<const float* const*, AudioBuffer&>(
      planar_ptrs, num_input_frames, num_input_channels, input_frame_offset,
      nullptr /* channel_map*/, *output, output->num_frames(),
      output->num_channels(), output_frame_offset, num_frames_to_copy);
}

void FillAudioBufferWithOffset(const int16* const* planar_ptrs,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output) {
  DCHECK(planar_ptrs);
  DCHECK(output);
  ConvertPlanarToPlanarTemplated<const int16* const*, AudioBuffer&>(
      planar_ptrs, num_input_frames, num_input_channels, input_frame_offset,
      nullptr /* channel_map*/, *output, output->num_frames(),
      output->num_channels(), output_frame_offset, num_frames_to_copy);
}

void FillAudioBufferWithChannelRemapping(const int16* interleaved_buffer,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertInterleavedToPlanarTemplated<const int16*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /*input_frame_offset*/, &channel_map, *output, output->num_frames(),
      output->num_channels(), 0 /*output_frame_offset*/, num_frames_to_copy);
}

void FillAudioBufferWithChannelRemapping(const float* interleaved_buffer,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output) {
  DCHECK(interleaved_buffer);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertInterleavedToPlanarTemplated<const float*, AudioBuffer&>(
      interleaved_buffer, num_input_frames, num_input_channels,
      0 /*input_frame_offset*/, &channel_map, *output, output->num_frames(),
      output->num_channels(), 0 /*output_frame_offset*/, num_frames_to_copy);
}

void FillAudioBufferWithChannelRemapping(const float* const* planar_ptrs,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output) {
  DCHECK(planar_ptrs);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertPlanarToPlanarTemplated<const float* const*, AudioBuffer&>(
      planar_ptrs, num_input_frames, num_input_channels,
      0 /*input_offset_frames*/, &channel_map, *output, output->num_frames(),
      output->num_channels(), 0 /* output_offset_frames*/, num_frames_to_copy);
}

void FillAudioBufferWithChannelRemapping(const int16* const* planar_ptr,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output) {
  DCHECK(planar_ptr);
  DCHECK(output);
  const size_t num_frames_to_copy =
      std::min(num_input_frames, output->num_frames());
  ConvertPlanarToPlanarTemplated<const int16* const*, AudioBuffer&>(
      planar_ptr, num_input_frames, num_input_channels,
      0 /*input_offset_frames*/, &channel_map, *output, output->num_frames(),
      output->num_channels(), 0 /* output_offset_frames*/, num_frames_to_copy);
}

void FillExternalBuffer(const AudioBuffer& input, std::vector<float>* output) {
  DCHECK(output);
  output->resize(input.num_frames() * input.num_channels());
  FillExternalBuffer(input, output->data(), input.num_frames(),
                     input.num_channels());
}

void FillExternalBuffer(const AudioBuffer& input, std::vector<int16>* output) {
  DCHECK(output);
  output->resize(input.num_frames() * input.num_channels());
  FillExternalBuffer(input, output->data(), input.num_frames(),
                     input.num_channels());
}

void FillExternalBuffer(const AudioBuffer& input,
                        int16* const* planar_output_ptrs,
                        size_t num_output_frames, size_t num_output_channels) {
  ConvertPlanarToPlanarTemplated<const AudioBuffer&, int16* const*>(
      input, input.num_frames(), input.num_channels(),
      0 /*input_offset_frames*/, nullptr /* channel_map*/, planar_output_ptrs,
      num_output_frames, num_output_channels, 0 /* output_offset_frames*/,
      num_output_frames);
}

void FillExternalBuffer(const AudioBuffer& input,
                        float* const* planar_output_ptrs,
                        size_t num_output_frames, size_t num_output_channels) {
  ConvertPlanarToPlanarTemplated<const AudioBuffer&, float* const*>(
      input, input.num_frames(), input.num_channels(),
      0 /*input_offset_frames*/, nullptr /* channel_map*/, planar_output_ptrs,
      num_output_frames, num_output_channels, 0 /* output_offset_frames*/,
      num_output_frames);
}

void FillExternalBuffer(const AudioBuffer& input,
                        int16* interleaved_output_buffer,
                        size_t num_output_frames, size_t num_output_channels) {
  ConvertPlanarToInterleavedTemplated<const AudioBuffer&, int16*>(
      input, input.num_frames(), input.num_channels(),
      0 /*input_offset_frames*/, interleaved_output_buffer, num_output_frames,
      num_output_channels, 0 /* output_offset_frames*/, num_output_frames);
}

void FillExternalBuffer(const AudioBuffer& input,
                        float* interleaved_output_buffer,
                        size_t num_output_frames, size_t num_output_channels) {
  ConvertPlanarToInterleavedTemplated<const AudioBuffer&, float*>(
      input, input.num_frames(), input.num_channels(),
      0 /*input_offset_frames*/, interleaved_output_buffer, num_output_frames,
      num_output_channels, 0 /* output_offset_frames*/, num_output_frames);
}

void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  int16* const* planar_output_ptrs,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy) {
  ConvertPlanarToPlanarTemplated<const AudioBuffer&, int16* const*>(
      input, input.num_frames(), input.num_channels(), input_offset_frames,
      nullptr /* channel_map */, planar_output_ptrs, num_output_frames,
      num_output_channels, output_offset_frames, num_frames_convert_and_copy);
}

void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  float* const* planar_output_ptrs,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy) {
  ConvertPlanarToPlanarTemplated<const AudioBuffer&, float* const*>(
      input, input.num_frames(), input.num_channels(), input_offset_frames,
      nullptr /* channel_map */, planar_output_ptrs, num_output_frames,
      num_output_channels, output_offset_frames, num_frames_convert_and_copy);
}

void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  int16* interleaved_output_buffer,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy) {
  ConvertPlanarToInterleavedTemplated<const AudioBuffer&, int16*>(
      input, input.num_frames(), input.num_channels(), input_offset_frames,
      interleaved_output_buffer, num_output_frames, num_output_channels,
      output_offset_frames, num_frames_convert_and_copy);
}

void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  float* interleaved_output_buffer,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy) {
  ConvertPlanarToInterleavedTemplated<const AudioBuffer&, float*>(
      input, input.num_frames(), input.num_channels(), input_offset_frames,
      interleaved_output_buffer, num_output_frames, num_output_channels,
      output_offset_frames, num_frames_convert_and_copy);
}

void GetRawChannelDataPointersFromAudioBuffer(
    AudioBuffer* audio_buffer, std::vector<float*>* channel_ptr_vector) {
  DCHECK(audio_buffer);
  DCHECK(channel_ptr_vector);
  DCHECK_EQ(audio_buffer->num_channels(), channel_ptr_vector->size());
  for (size_t i = 0; i < audio_buffer->num_channels(); ++i) {
    (*channel_ptr_vector)[i] = &(*audio_buffer)[i][0];
  }
}

void GetRawChannelDataPointersFromAudioBuffer(
    const AudioBuffer& audio_buffer,
    std::vector<const float*>* channel_ptr_vector) {
  DCHECK(channel_ptr_vector);
  DCHECK_EQ(audio_buffer.num_channels(), channel_ptr_vector->size());
  for (size_t i = 0; i < audio_buffer.num_channels(); ++i) {
    (*channel_ptr_vector)[i] = &audio_buffer[i][0];
  }
}

}  // namespace vraudio
