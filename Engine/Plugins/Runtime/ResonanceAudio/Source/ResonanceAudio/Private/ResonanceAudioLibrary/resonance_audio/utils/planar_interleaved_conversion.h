/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_PLANAR_INTERLEAVED_CONVERSION_H_
#define RESONANCE_AUDIO_UTILS_PLANAR_INTERLEAVED_CONVERSION_H_

#include <vector>

#include "base/integral_types.h"
#include "base/audio_buffer.h"
#include "base/logging.h"

namespace vraudio {

// Copies interleaved audio data from a raw float pointer into separate channel
// buffers specified by a vector of raw float pointers.
//
// @param interleaved_buffer Raw float pointer to interleaved audio data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param planar_buffer_ptr Raw float pointers to each planar channel buffer.
// @param num_output_frames Number of frames per channel in output buffer.
void PlanarFromInterleaved(const float* interleaved_buffer,
                           size_t num_input_frames, size_t num_input_channels,
                           const std::vector<float*>& planar_buffer_ptr,
                           size_t num_output_frames);

// Copies interleaved audio data from a raw int16 pointer into separate channel
// buffers specified by a vector of raw float pointers.
//
// @param interleaved_buffer Raw int16 pointer to interleaved audio data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param planar_buffer_ptr Raw float pointers to each planar channel buffer.
// @param num_output_frames Number of frames per channel in output buffer.
void PlanarFromInterleaved(const int16* interleaved_buffer,
                           size_t num_input_frames, size_t num_input_channels,
                           const std::vector<float*>& planar_buffer_ptr,
                           size_t num_output_frames);

// Copies interleaved audio data from a raw float pointer into a planar
// |AudioBuffer|. Note that the number of output channels and frames is defined
// by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Raw float pointer to interleaved audio data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param output Target output buffer.
void FillAudioBuffer(const float* interleaved_buffer, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output);

// Copies interleaved audio data from a raw int16 pointer into a planar
// |AudioBuffer|. Note that the number of output channels and frames is defined
// by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Raw int16 pointer to interleaved audio data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param output Target output buffer.
void FillAudioBuffer(const int16* interleaved_buffer, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output);

// Copies interleaved audio data from a float vector into a planar
// |AudioBuffer|. Note that the number of output channels and frames is defined
// by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Interleaved audio data.
// @param num_input_channels Number of channels in interleaved audio data.
// @param output Target output buffer.
void FillAudioBuffer(const std::vector<float>& interleaved_buffer,
                     size_t num_input_channels, AudioBuffer* output);

// Copies interleaved audio data from a int16 vector into a planar
// |AudioBuffer|. Note that the number of output channels and frames is defined
// by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Interleaved audio data.
// @param num_input_channels Number of channels in interleaved audio data.
// @param output Target output buffer.
void FillAudioBuffer(const std::vector<int16>& interleaved_buffer,
                     size_t num_input_channels, AudioBuffer* output);

// Copies raw planar float audio data into a planar |AudioBuffer|. Note that the
// number of output channels and frames is defined by the target |AudioBuffer|
// instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw float channel data.
// @param num_input_frames Size of planar input in frames.
// @param num_input_channels Number of channels in planar input buffer.
// @param output Target output buffer.
void FillAudioBuffer(const float* const* planar_ptrs, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output);

// Copies raw planar int16 audio data into a planar |AudioBuffer|. Note that the
// number of output channels and frames is defined by the target |AudioBuffer|
// instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw int16 channel data.
// @param num_input_frames Size of planar input in frames.
// @param num_input_channels Number of channels in planar input buffer.
// @param output Target output buffer.
void FillAudioBuffer(const int16* const* planar_ptrs, size_t num_input_frames,
                     size_t num_input_channels, AudioBuffer* output);

// Copies interleaved audio data from a raw float pointer into a planar
// |AudioBuffer| with a specified output frame offset. Note that the
// number of output channels is defined by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Raw float pointer to interleaved audio data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param input_frame_offset First source frame position in input.
// @param output_frame_offset First frame destination in output.
// @param num_frames_to_copy Number of frames per copy.
// @param output Target output buffer.
void FillAudioBufferWithOffset(const float* interleaved_buffer,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output);

// Copies interleaved audio data from a raw int16 pointer into a planar
// |AudioBuffer| with a specified output frame offset. Note that the
// number of output channels is defined by the target |AudioBuffer| instance.
//
// @param interleaved_buffer Raw int16 pointer to interleaved audio data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param input_frame_offset First source frame position in input.
// @param output_frame_offset First frame destination in output.
// @param num_frames_to_copy Number of frames per copy.
// @param output Target output buffer.
void FillAudioBufferWithOffset(const int16* interleaved_buffer,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output);

// Copies raw planar float audio data into a planar |AudioBuffer| with a
// specified output frame offset. Note that the number of output channels is
// defined by the target |AudioBuffer| instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw float channel data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param input_frame_offset First source frame position in input.
// @param output_frame_offset First frame destination in output.
// @param num_frames_to_copy Number of frames per copy.
// @param output Target output buffer.
void FillAudioBufferWithOffset(const float* const* planar_ptrs,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output);

// Copies raw planar int16 audio data into a planar |AudioBuffer| with a
// specified output frame offset. Note that the number of output channels is
// defined by the target |AudioBuffer| instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw int16 channel data.
// @param num_input_frames Size of |interleaved_buffer| in frames.
// @param num_input_channels Number of channels in interleaved audio data.
// @param input_frame_offset First source frame position in input.
// @param output_frame_offset First frame destination in output.
// @param num_frames_to_copy Number of frames per copy.
// @param output Target output buffer.
void FillAudioBufferWithOffset(const int16* const* planar_ptrs,
                               size_t num_input_frames,
                               size_t num_input_channels,
                               size_t input_frame_offset,
                               size_t output_frame_offset,
                               size_t num_frames_to_copy, AudioBuffer* output);

// Copies interleaved audio data from a raw int16 pointer into a planar
// |AudioBuffer|. The |channel_map| argument allows to reorder the channel
// mapping between the interleaved input and output buffer. The i'th channel in
// the output buffer corresponds to the |channel_map[i]|'th input channel. Note
// that the number of output channels and frames is derived from the target
// |AudioBuffer| instance.
//
// @param interleaved_buffer Raw int16 pointer to interleaved audio data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of input channels.
// @param channel_map Mapping that maps output channels to input channels
// @param output Target output buffer.
void FillAudioBufferWithChannelRemapping(const int16* interleaved_buffer,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output);

// Copies interleaved audio data from a raw float pointer into a planar
// |AudioBuffer|. The |channel_map| argument allows to reorder the channel
// mapping between the interleaved input and output buffer. The i'th channel in
// the output buffer corresponds to the |channel_map[i]| input channel. Note
// that the number of output channels and frames is derived from the target
// |AudioBuffer| instance.
//
// @param interleaved_buffer Raw float pointer to interleaved audio data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of input channels.
// @param channel_map Mapping that maps output channels to input channels
// @param output Target output buffer.
void FillAudioBufferWithChannelRemapping(const float* interleaved_buffer,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output);

// Copies raw planar float audio data into a planar |AudioBuffer|. The
// |channel_map| argument allows to reorder the channel mapping between the
// planar input and output buffer. The i'th channel in the output buffer
// corresponds to the |channel_map[i]| input channel. Note that the number of
// output channels and frames is derived from the target |AudioBuffer| instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw float channel data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of input channels.
// @param channel_map Mapping that maps output channels to input channels
// @param output Target output buffer.
void FillAudioBufferWithChannelRemapping(const float* const* planar_ptr,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output);

// Copies raw planar int16 audio data into a planar |AudioBuffer|. The
// |channel_map| argument allows to reorder the channel mapping between the
// planar input and output buffer. The i'th channel in the output buffer
// corresponds to the |channel_map[i]| input channel. Note that the number of
// output channels and frames is derived from the target |AudioBuffer| instance.
//
// @param planar_ptrs Pointer to an array of pointers to raw int16 channel data.
// @param num_input_frames Size of interleaved_buffer in frames.
// @param num_input_channels Number of input channels.
// @param channel_map Mapping that maps output channels to input channels
// @param output Target output buffer.
void FillAudioBufferWithChannelRemapping(const int16* const* planar_ptr,
                                         size_t num_input_frames,
                                         size_t num_input_channels,
                                         const std::vector<size_t>& channel_map,
                                         AudioBuffer* output);

// Copies planar audio data from an |AudioBuffer| to an external interleaved
// float vector. Note this method resizes the target vector to match number of
// input samples.
//
// @param input Input audio buffer.
// @param output interleaved output vector.
void FillExternalBuffer(const AudioBuffer& input, std::vector<float>* output);

// Copies and converts planar audio data from an |AudioBuffer| to an external
// interleaved int16 vector. Note this method resizes the target vector to match
// number of input samples.
//
// @param input Input audio buffer.
// @param output interleaved output vector.
void FillExternalBuffer(const AudioBuffer& input, std::vector<int16>* output);

// Copies planar audio data from an |AudioBuffer| to an external planar raw
// float buffer. Note that the input and output buffer must match in terms of
// number of channels and frames.
//
// @param input Input audio buffer.
// @param planar_output_ptrs Planar output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
void FillExternalBuffer(const AudioBuffer& input,
                        float* const* planar_output_ptrs,
                        size_t num_output_frames, size_t num_output_channels);

// Copies and converts audio data from an |AudioBuffer| to an external planar
// int16 buffer. Note that the input and output buffer must match in terms of
// number of channels and frames.
//
// @param input Input audio buffer.
// @param planar_output_ptrs Planar output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
void FillExternalBuffer(const AudioBuffer& input,
                        int16* const* planar_output_ptrs,
                        size_t num_output_frames, size_t num_output_channels);

// Copies and converts planar audio data from an |AudioBuffer| to an external
// interleaved raw int16 buffer. Note that the input and output buffer must
// match in terms of number of channels and frames.
//
// @param input Input audio buffer.
// @param interleaved_output_buffer Interleaved output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
void FillExternalBuffer(const AudioBuffer& input,
                        int16* interleaved_output_buffer,
                        size_t num_output_frames, size_t num_output_channels);

// Copies planar audio data from an |AudioBuffer| to an external interleaved
// raw float buffer. Note that the input and output buffer must match in terms
// of number of channels and frames.
//
// @param input Input audio buffer.
// @param interleaved_output_buffer Interleaved output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
void FillExternalBuffer(const AudioBuffer& input,
                        float* interleaved_output_buffer,
                        size_t num_output_frames, size_t num_output_channels);

// Copies and converts audio data from an |AudioBuffer| to an external
// planar raw int16 buffer with the ability to specify an offset into the
// input and output buffer.
//
// @param input Input audio buffer.
// @param input_offset_frames Offset into input buffer in frames.
// @param planar_output_ptrs Planar output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
// @param output_offset_frames Offset into the output buffer in frames.
// @param num_frames_convert_and_copy Number of frames to be processed.
void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  int16* const* planar_output_ptrs,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy);

// Copies audio data from an |AudioBuffer| to an external planar raw float
// buffer with the ability to specify an offset into the input and output
// buffer.
//
// @param input Input audio buffer.
// @param input_offset_frames Offset into input buffer in frames.
// @param planar_output_ptrs Planar output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
// @param output_offset_frames Offset into the output buffer in frames.
// @param num_frames_convert_and_copy Number of frames to be processed.
void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  float* const* planar_output_ptrs,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy);

// Copies and converts audio data from an |AudioBuffer| to an external
// interleaved raw int16 buffer with the ability to specify an offset into the
// input and output buffer.
//
// @param input Input audio buffer.
// @param input_offset_frames Offset into input buffer in frames.
// @param interleaved_output_buffer Interleaved output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
// @param output_offset_frames Offset into the output buffer in frames.
// @param num_frames_convert_and_copy Number of frames to be processed.
void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  int16* interleaved_output_buffer,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy);

// Copies and audio data from an |AudioBuffer| to an external interleaved raw
// float buffer with the ability to specify an offset into the input and output
// buffer.
//
// @param input Input audio buffer.
// @param input_offset_frames Offset into input buffer in frames.
// @param interleaved_output_buffer Interleaved output vector.
// @param num_output_frames Number of frames in output buffer.
// @param num_output_channels Number of channels in output buffer.
// @param output_offset_frames Offset into the output buffer in frames.
// @param num_frames_convert_and_copy Number of frames to be processed.
void FillExternalBufferWithOffset(const AudioBuffer& input,
                                  size_t input_offset_frames,
                                  float* interleaved_output_buffer,
                                  size_t num_output_frames,
                                  size_t num_output_channels,
                                  size_t output_offset_frames,
                                  size_t num_frames_convert_and_copy);

// Generates a vector of mutable float pointers to the beginning of each channel
// buffer in an |AudioBuffer|. The size of the |channel_ptr_vector| output
// vector must match the number of channels in |audio_buffer|.
//
// @param audio_buffer Audio buffer.
// @param channel_ptr_vector Output std::vector<float*> vector.
void GetRawChannelDataPointersFromAudioBuffer(
    AudioBuffer* audio_buffer, std::vector<float*>* channel_ptr_vector);

// Generates a vector of immutable float pointers to the beginning of each
// channel buffer in an |AudioBuffer|. The size of the |channel_ptr_vector|
// output vector must match the number of channels in |audio_buffer|.
//
// @param audio_buffer Audio buffer.
// @param channel_ptr_vector Output std::vector<const float*> vector.
void GetRawChannelDataPointersFromAudioBuffer(
    const AudioBuffer& audio_buffer,
    std::vector<const float*>* channel_ptr_vector);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_PLANAR_INTERLEAVED_CONVERSION_H_
