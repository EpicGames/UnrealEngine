/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_PSEUDOINVERSE_H_
#define RESONANCE_AUDIO_UTILS_PSEUDOINVERSE_H_

#include "ResonanceEigenHelper.h"

namespace vraudio {

// Computes the Moore-Penrose pseudoinverse of |matrix|.
//
// @tparam MatrixType The type of the input matrix (an Eigen::Matrix).
// @param matrix The input matrix to compute the pseudoinverse of.
// @return The Moore-Penrose pseudoinverse of |matrix|.
template <typename MatrixType>
Eigen::Matrix<typename MatrixType::Scalar, MatrixType::ColsAtCompileTime,
              MatrixType::RowsAtCompileTime>
Pseudoinverse(const MatrixType& matrix) {
  Eigen::JacobiSVD<Eigen::Matrix<typename MatrixType::Scalar, Eigen::Dynamic,
                                 Eigen::Dynamic>> svd(matrix,
                                                      Eigen::ComputeThinU |
                                                          Eigen::ComputeThinV);
  return svd.solve(
      Eigen::Matrix<typename MatrixType::Scalar, MatrixType::RowsAtCompileTime,
                    MatrixType::RowsAtCompileTime>::Identity(matrix.rows(),
                                                             matrix.rows()));
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_PSEUDOINVERSE_H_
