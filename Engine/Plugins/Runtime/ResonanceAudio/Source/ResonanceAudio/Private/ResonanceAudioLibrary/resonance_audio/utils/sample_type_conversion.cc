/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "utils/sample_type_conversion.h"

#include "base/simd_utils.h"

namespace vraudio {

void ConvertPlanarSamples(size_t length, const int16* input, float* output) {
  FloatFromInt16(length, input, output);
}

void ConvertPlanarSamples(size_t length, const float* input, float* output) {
  std::copy_n(input, length, output);
}

void ConvertPlanarSamples(size_t length, const float* input, int16* output) {
  Int16FromFloat(length, input, output);
}

}  // namespace vraudio
