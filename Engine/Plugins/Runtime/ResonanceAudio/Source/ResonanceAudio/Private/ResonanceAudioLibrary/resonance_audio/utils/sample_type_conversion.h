/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_SAMPLE_TYPE_CONVERSION_H_
#define RESONANCE_AUDIO_UTILS_SAMPLE_TYPE_CONVERSION_H_

#include <algorithm>

#include "base/integral_types.h"
#include "base/logging.h"

namespace vraudio {

// Convert the given int16 to a float in the range [-1.0f, 1.0f].
inline void ConvertSampleToFloatFormat(int16 input, float* output) {
  DCHECK(output);
  static const float kInt16Max = static_cast<float>(0x7FFF);
  static const float kInt16ToFloat = 1.0f / kInt16Max;
  *output = input * kInt16ToFloat;
}

// Overloaded input argument to support sample type templated methods.
inline void ConvertSampleToFloatFormat(float input, float* output) {
  DCHECK(output);
  *output = input;
}

// Saturating if the float is not in [-1.0f, 1.0f].
inline void ConvertSampleFromFloatFormat(float input, int16* output) {
  DCHECK(output);
  // Convert the given float to an int16 in the range
  // [-32767 (0x7FFF), 32767 (0x7FFF)],
  static const float kInt16Min = static_cast<float>(-0x7FFF);
  static const float kInt16Max = static_cast<float>(0x7FFF);
  static const float kFloatToInt16 = kInt16Max;
  const float scaled_value = input * kFloatToInt16;
  const float clamped_value =
      std::min(kInt16Max, std::max(kInt16Min, scaled_value));
  *output = static_cast<int16>(clamped_value);
}

// Overloaded output argument to support sample type templated methods.
inline void ConvertSampleFromFloatFormat(float input, float* output) {
  DCHECK(output);
  *output = input;
}

// Convert a vector of int16 samples to float format in the range [-1.0f, 1.0f].
void ConvertPlanarSamples(size_t length, const int16* input,
                                 float* output);

// Overloaded input argument to support sample type templated methods.
void ConvertPlanarSamples(size_t length, const float* input,
                                 float* output);

// Overloaded method to support methods templated against the input sample type.
void ConvertPlanarSamples(size_t length, const float* input,
                                   int16* output);

// Overloaded output argument to support sample type templated methods.
void ConvertPlanarSamples(size_t length, const float* input,
                                   float* output);

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_SAMPLE_TYPE_CONVERSION_H_
