/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_SEMI_LOCKLESS_FIFO_H_
#define RESONANCE_AUDIO_UTILS_SEMI_LOCKLESS_FIFO_H_

#include <atomic>
#include <chrono>
#include <condition_variable>
#include <mutex>
#include <vector>

#include "base/logging.h"

namespace vraudio {

// Thread-safe multiple producer - single consumer FIFO queue to share data
// between threads. The FIFO takes over ownership of the queue elements. Note
// that |PushBack| calls are synchronized with a mutex and may block. Calls to
// |PopFront| are lockless and never block.
//
// @tparam DataType Object type that the FIFO handles.
template <typename DataType>
class SemiLocklessFifo {
 public:
  typedef std::chrono::steady_clock::duration ClockDuration;

  SemiLocklessFifo();

  ~SemiLocklessFifo();

  // Takes over ownership of |input| and pushes it to the FIFO queue back.
  //
  // @param input Input element to be added to the FIFO queue.
  void PushBack(DataType&& input);

  // Pops element from FIFO queue front.
  //
  // @return Element from FIFO queue front. Must not be called if the queue is
  //     empty.
  DataType PopFront();

  // Returns true if FIFO queue is empty, false otherwise. This method is *not*
  // thread-safe and should only be called from the consumer thread.
  bool Empty() const;

  // Clears the FIFO queue and deletes all its elements. This method is *not*
  // thread-safe and should only be called from the consumer thread.
  void Clear();

  // Sleeps until the number of elements in the FIFO queue drop below a target
  // threshold. This method can be used to synchronize the producer and the
  // consumer. Sleeping is enabled by default and can be disabled via
  // |EnableBlockingSleepUntilMethods|.
  //
  // @param target_size Target size of FIFO queue.
  // @param max_wait Maximum waiting period.
  // @return True if number of FIFO elements is below target size.
  bool SleepUntilBelowSizeTarget(size_t target_size,
                                 const ClockDuration& max_wait);

  // Sleeps until the number of elements in the FIFO queue is greater or equal a
  // target threshold. This method can be used to synchronize the producer and
  // the consumer. Sleeping is enabled by default and can be disabled via
  // |EnableBlockingSleepUntilMethods|.
  //
  // @param target_size Target size of FIFO queue.
  // @param max_wait Maximum waiting period.
  // @return True if number of FIFO elements is greater or equal the target
  //     size.
  bool SleepUntilNumElementsInQueue(size_t target_size,
                                    const ClockDuration& max_wait);

  // Allows for unblocking |SleepUntil[BelowSizeTarget|NumElementsInQueue]|
  // method.
  void EnableBlockingSleepUntilMethods(bool enable);

 private:
  // Node in single-linked list.
  struct Node {
    Node() : next(nullptr) {}
    std::atomic<Node*> next;
    DataType data;
  };

  // Head of linked list.
  Node* head_;

  // Tail of linked list.
  Node* tail_;

  // Number of elements.
  std::atomic<size_t> fifo_size_;

  // Mutex to synchronize |PushBack| calls from multiple threads.
  std::mutex push_mutex_;

  // Conditional to signal consumption.
  std::condition_variable pop_conditional_;

  // Mutex to block on until signal consumption occurs.
  std::mutex pop_conditional_mutex_;

  // Conditional to signal new elements on the FIFO.
  std::condition_variable push_conditional_;

  // Mutex to block on until new elements have been added to the FIFO.
  std::mutex push_conditional_mutex_;

  // Flag to enable and disable blocking sleeping calls.
  std::atomic<bool> enable_sleeping_;
};

template <typename DataType>
SemiLocklessFifo<DataType>::SemiLocklessFifo()
    : fifo_size_(0), enable_sleeping_(true) {
  head_ = tail_ = new Node();
}

template <typename DataType>
SemiLocklessFifo<DataType>::~SemiLocklessFifo() {
  Clear();
  DCHECK_EQ(head_, tail_);
  DCHECK(head_->next.load() == nullptr);
  delete head_;
}

template <typename DataType>
void SemiLocklessFifo<DataType>::PushBack(DataType&& input) {
  std::lock_guard<std::mutex> lock(push_mutex_);
  tail_->data = std::move(input);
  Node* const new_node = new Node();
  DCHECK(tail_->next.load() == nullptr);
  tail_->next = new_node;
  tail_ = new_node;
  ++fifo_size_;

  {
    // Taking the lock and dropping it immediately assure that the notify
    // cannot happen between the check of the predicate and wait of the
    // |push_conditional_|.
    std::lock_guard<std::mutex> lock(push_conditional_mutex_);
  }
  push_conditional_.notify_all();
}

template <typename DataType>
DataType SemiLocklessFifo<DataType>::PopFront() {
  DCHECK(!Empty());

  Node* const front_node = head_;
  head_ = front_node->next;

  DataType output = std::move(front_node->data);
  delete front_node;

  DCHECK_GT(fifo_size_.load(), 0u);
  --fifo_size_;

  {
    // Taking the lock and dropping it immediately assure that the notify
    // cannot happen between the check of the predicate and wait of the
    // |pop_conditional_|.
    std::lock_guard<std::mutex> lock(pop_conditional_mutex_);
  }
  pop_conditional_.notify_one();
  return output;
}

template <typename DataType>
bool SemiLocklessFifo<DataType>::Empty() const {
  return fifo_size_.load() == 0;
}

template <typename DataType>
void SemiLocklessFifo<DataType>::Clear() {
  while (!Empty()) {
    PopFront();
  }
  DCHECK_EQ(fifo_size_, 0u);
}

template <typename DataType>
bool SemiLocklessFifo<DataType>::SleepUntilBelowSizeTarget(
    size_t target_size, const ClockDuration& max_wait) {
  DCHECK_GT(target_size, 0);
  std::unique_lock<std::mutex> lock(pop_conditional_mutex_);
  pop_conditional_.wait_for(lock, max_wait, [this, target_size]() {
    return fifo_size_ < target_size || !enable_sleeping_.load();
  });
  return fifo_size_ < target_size;
}

template <typename DataType>
bool SemiLocklessFifo<DataType>::SleepUntilNumElementsInQueue(
    size_t target_size, const ClockDuration& max_wait) {
  DCHECK_GT(target_size, 0u);
  std::unique_lock<std::mutex> lock(push_conditional_mutex_);
  push_conditional_.wait_for(lock, max_wait, [this, target_size]() {
    return fifo_size_ >= target_size || !enable_sleeping_.load();
  });
  return fifo_size_ >= target_size;
}

template <typename DataType>
void SemiLocklessFifo<DataType>::EnableBlockingSleepUntilMethods(bool enable) {
  enable_sleeping_ = enable;
  // Taking the lock and dropping it immediately assure that the notify
  // cannot happen between the check of the predicate and wait of the
  // |pop_conditional_| and |push_conditional_|.
  { std::lock_guard<std::mutex> lock(pop_conditional_mutex_); }
  { std::lock_guard<std::mutex> lock(push_conditional_mutex_); }
  pop_conditional_.notify_one();
  push_conditional_.notify_one();
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_SEMI_LOCKLESS_FIFO_H_
