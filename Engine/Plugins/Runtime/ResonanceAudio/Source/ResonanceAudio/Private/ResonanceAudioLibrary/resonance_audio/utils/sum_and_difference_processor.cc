/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "utils/sum_and_difference_processor.h"

#include "base/constants_and_types.h"
#include "base/logging.h"


namespace vraudio {

SumAndDifferenceProcessor::SumAndDifferenceProcessor(size_t num_frames)
    : temp_buffer_(kNumMonoChannels, num_frames) {}

void SumAndDifferenceProcessor::Process(AudioBuffer* stereo_buffer) {

  DCHECK_EQ(stereo_buffer->num_channels(), kNumStereoChannels);
  AudioBuffer::Channel* temp_channel = &temp_buffer_[0];
  // channel_1' = channel_1 + channel_2;
  // channel_2' = channel_1 - channel_2;
  *temp_channel = (*stereo_buffer)[0];
  *temp_channel -= (*stereo_buffer)[1];
  (*stereo_buffer)[0] += (*stereo_buffer)[1];
  (*stereo_buffer)[1] = *temp_channel;
}

}  // namespace vraudio
