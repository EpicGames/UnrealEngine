/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_SUM_AND_DIFFERENCE_PROCESSOR_H_
#define RESONANCE_AUDIO_UTILS_SUM_AND_DIFFERENCE_PROCESSOR_H_

#include "base/audio_buffer.h"

namespace vraudio {

// Class which converts a 2-channel input audio buffer into its sum and
// difference signals and stores them in the left and right channel
// respectively.
class SumAndDifferenceProcessor {
 public:
  // Constructs a stereo sum and difference processor.
  //
  // @param num_frames Number of frames in the stereo input audio buffer.
  explicit SumAndDifferenceProcessor(size_t num_frames);

  // Converts a 2-channel buffer signals into their sum and difference.
  void Process(AudioBuffer* stereo_buffer);

 private:
  // Temporary audio buffer to store left channel input data during conversion.
  AudioBuffer temp_buffer_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_SUM_AND_DIFFERENCE_PROCESSOR_H_
