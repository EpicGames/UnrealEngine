/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4324) /* structure was padded due to alignment specifier */
#pragma warning(disable:6011) /* Dereferencing NULL pointer '<ptr>' */
#pragma warning(disable:6385) /* Reading invalid data from '<array>': the readable size is '<array size>' bytes, but '<read size>' bytes may be read */
#pragma warning(disable:6386) /* Buffer overrun while writing to '<array>': the writable size is '<array size>' bytes, but '<write size>' bytes might be written */
#pragma warning(disable:4244) /* Conversion from <type> to <type>, possible loss of data. */
#pragma warning(disable:4702) /* unreachable code */
#pragma warning(disable:4100) /* unreferenced formal parameter */
#endif

#include "utils/task_thread_pool.h"

#include <thread>

#include "base/integral_types.h"
#include "base/logging.h"

namespace vraudio {

// A simple worker thread wrapper, to be used by TaskThreadPool.
class TaskThreadPool::WorkerThread {
 public:
  // Constructor.
  //
  // @param parent_pool The |TaskThreadPool| which owns and manages this
  //     |WorkerThread| instance.
  WorkerThread()
      : parent_pool_(), task_closure_(), task_loop_triggered_(false) {}

  // Copy constructor. Necessary for allowing this class to be storied in
  // std::vector<WorkerThread> container class.
  //
  // @param other |WorkerThread| instance this instance will be copied from.
  WorkerThread(const TaskThreadPool::WorkerThread& /*other*/)
      : parent_pool_(), task_closure_(), task_loop_triggered_(false) {}

  // Destructor.
  ~WorkerThread();

  // Sets the parent pool for this |WorkerThread|.
  //
  // @param parent_pool The |TaskThreadPool| which owns this |WorkerThread|.
  // @return True indicates that the thread could be started.
  bool SetParentAndStart(TaskThreadPool* parent_pool);

  // Indicates that the worker thread should run the |TaskClosure| task.
  //
  // @param task_closure The task function which |TaskThreadPool| assigns to
  //      this |WorkerThread|.
  void Run(TaskClosure task_closure);

  // Waits for the |WorkerThread| task loop to exit, for shutdown.
  void Join();

  // Checks whether this |WorkerThread| is available for a task assignment.
  bool IsAvailable() const;

 private:
  // The task loop for this |WorkerThread|. This function will wait for an
  // assigned task, execute the task, and then reset itself to wait for the next
  // task.
  void TaskLoop();

  // This |WorkerThread|'s parent |TaskThreadPool|.
  TaskThreadPool* parent_pool_;

  // Condition allowing the |TaskThreadPool| to trigger this worker thread to
  // continue once it has been given a work assignment.
  std::condition_variable execute_task_condition_;

  // Mutex for receiving |execute_task_condition_| notification.
  std::mutex execute_task_mutex_;

  // The current task binding which the Worker Thread has been asked to
  // execute.
  TaskClosure task_closure_;

  // An atomic boolean to indicate that a task loop trigger has occurred. This
  // may happen even when a task has not been assigned during shutdown.
  std::atomic<bool> task_loop_triggered_;

  // The worker thread.
  std::thread task_thread_;
};

TaskThreadPool::TaskThreadPool()
    : num_worker_threads_available_(0),
      is_pool_running_(false) {}

TaskThreadPool::~TaskThreadPool() { StopThreadPool(); }

bool TaskThreadPool::StartThreadPool(size_t num_worker_threads) {
  if (is_pool_running_) {
    return true;
  }
  is_pool_running_ = true;
  worker_threads_.resize(num_worker_threads);

  // Start all worker threads.
  for (auto& worker_thread : worker_threads_) {
    bool thread_started = worker_thread.SetParentAndStart(this);
    if (!thread_started) {
      StopThreadPool();
      return false;
    }
  }

  // Wait for all worker threads to be launched.
  std::unique_lock<std::mutex> worker_lock(worker_available_mutex_);
  worker_available_condition_.wait(worker_lock, [this, num_worker_threads]() {
    return !is_pool_running_.load() || num_worker_threads_available_.load() ==
                                           static_cast<int>(num_worker_threads);
  });

  return true;
}

void TaskThreadPool::StopThreadPool() {
  if (!is_pool_running_) {
    return;
  }
  // Shut down all active worker threads.
  {
    std::lock_guard<std::mutex> worker_lock(worker_available_mutex_);
    is_pool_running_ = false;
  }
  worker_available_condition_.notify_one();

  // Join and destruct workers.
  worker_threads_.resize(0);
}

bool TaskThreadPool::WaitUntilWorkerBecomesAvailable() {
  if (!is_pool_running_.load()) {
    return false;
  }
  if (num_worker_threads_available_.load() > 0) {
    return true;
  }
  std::unique_lock<std::mutex> worker_lock(worker_available_mutex_);
  worker_available_condition_.wait(worker_lock, [this]() {
    return (num_worker_threads_available_.load() > 0) ||
           !is_pool_running_.load();
  });
  return num_worker_threads_available_.load() > 0 && is_pool_running_.load();
}

bool TaskThreadPool::RunOnWorkerThread(TaskThreadPool::TaskClosure closure) {
  if (!is_pool_running_.load() || num_worker_threads_available_.load() == 0) {
    return false;
  }
  // Find the first available worker thread.
  WorkerThread* available_worker_thread = nullptr;
  for (auto& worker_thread : worker_threads_) {
    if (worker_thread.IsAvailable()) {
      available_worker_thread = &worker_thread;
      break;
    }
  }
  DCHECK(available_worker_thread);
  {
    std::lock_guard<std::mutex> lock(worker_available_mutex_);
    --num_worker_threads_available_;
  }

  if(available_worker_thread)
    available_worker_thread->Run(std::move(closure));

  return true;
}

size_t TaskThreadPool::GetAvailableTaskThreadCount() const {
  return num_worker_threads_available_.load();
}

bool TaskThreadPool::IsPoolRunning() { return is_pool_running_.load(); }

void TaskThreadPool::SignalWorkerAvailable() {
  {
    std::lock_guard<std::mutex> lock(worker_available_mutex_);
    ++num_worker_threads_available_;
  }
  worker_available_condition_.notify_one();
}

TaskThreadPool::WorkerThread::~WorkerThread() { Join(); }

bool TaskThreadPool::WorkerThread::SetParentAndStart(
    TaskThreadPool* parent_pool) {
  parent_pool_ = parent_pool;

  // Start the worker thread.
  task_thread_ = std::thread(std::bind(&WorkerThread::TaskLoop, this));
  return true;
}

void TaskThreadPool::WorkerThread::Run(TaskThreadPool::TaskClosure closure) {
  if (closure) {
    task_closure_ = std::move(closure);
    {
      std::lock_guard<std::mutex> lock(execute_task_mutex_);
      task_loop_triggered_ = true;
    }
    execute_task_condition_.notify_one();
  }
}

void TaskThreadPool::WorkerThread::Join() {
  DCHECK(!parent_pool_->IsPoolRunning());
  // Aquire and release lock to assure that the notify cannot happen between the
  // check of the predicate and wait of the |push_conditional_|.
  { std::lock_guard<std::mutex> lock(execute_task_mutex_); }
  execute_task_condition_.notify_one();
  if (task_thread_.joinable()) {
    task_thread_.join();
  }
}

bool TaskThreadPool::WorkerThread::IsAvailable() const {
  return !task_loop_triggered_.load();
}

void TaskThreadPool::WorkerThread::TaskLoop() {
  // Signal back to the parent thread pool that this thread has started and is
  // ready for use.
  task_loop_triggered_ = false;

  while (parent_pool_->IsPoolRunning() || task_closure_ != nullptr) {
    parent_pool_->SignalWorkerAvailable();
    std::unique_lock<std::mutex> task_lock(execute_task_mutex_);
    execute_task_condition_.wait(task_lock, [this]() {
      return task_loop_triggered_.load() || !parent_pool_->IsPoolRunning();
    });

    // Execute the assigned task.
    if (task_closure_ != nullptr) {
      task_closure_();

      // Clear the assigned task and return to ready state.
      task_closure_ = nullptr;
    }
    task_loop_triggered_ = false;
  }
}

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif
