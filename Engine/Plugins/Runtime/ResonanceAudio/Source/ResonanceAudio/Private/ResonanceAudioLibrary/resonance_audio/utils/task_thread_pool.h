/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_TASK_THREAD_POOL_H_
#define RESONANCE_AUDIO_UTILS_TASK_THREAD_POOL_H_

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4324) /* structure was padded due to alignment specifier */
#pragma warning(disable:6011) /* Dereferencing NULL pointer '<ptr>' */
#pragma warning(disable:6385) /* Reading invalid data from '<array>': the readable size is '<array size>' bytes, but '<read size>' bytes may be read */
#pragma warning(disable:6386) /* Buffer overrun while writing to '<array>': the writable size is '<array size>' bytes, but '<write size>' bytes might be written */
#pragma warning(disable:4244) /* Conversion from <type> to <type>, possible loss of data. */
#pragma warning(disable:4702) /* unreachable code */
#endif

#include <atomic>
#include <chrono>
#include <condition_variable>
#include <functional>
#include <mutex>
#include <vector>

namespace vraudio {

class TaskThreadPool;

// A very basic thread pool for launching discrete encapsulated task functions
// on a configurable number of task threads. Note that this pool expects tasks
// to complete their work with no management by the pool itself. Any means of
// managing or terminating tasks must be designed into the task contexts and
// managed externally in a thread-safe way.
class TaskThreadPool {
  friend class WorkerThread;

 public:
  // Type definition for task function which may be assigned to a |WorkerThread|
  // in this pool. Note that tasks should be self-contained and not require
  // communication with other tasks.
  typedef std::function<void()> TaskClosure;

  // Constructor.
  //
  TaskThreadPool();

  ~TaskThreadPool();

  // Creates and initializes thread pool. This method blocks until all threads
  // are loaded and initialized.
  //
  // @param num_worker_threads The number of worker threads to make available in
  //     the pool.
  // @return true on success or if thread pool has been already started.
  bool StartThreadPool(size_t num_worker_threads);

  // Signals all |WorkerThread|s to stop and waits for completion.
  void StopThreadPool();

  // Waits until a |WorkerThread| becomes available. It is assumed that only a
  // single thread will dispatch threads using this function, and therefore this
  // function should not itself be considered thread safe.
  //
  // @return True if a |WorkerThread| is available.
  bool WaitUntilWorkerBecomesAvailable();

  // Executes a |TaskClosure| on a |WorkerThread|. It is assumed that only a
  // signal thread will dispatch threads using this function, and therefore this
  // function should not itself be considered thread safe.
  //
  // @param closure The client task which will begin execution if and when this
  //     function returns True.
  // @return True if a |WorkerThread| is allocated to execute the closure
  //     function, false if no |WorkerThread| is available.
  bool RunOnWorkerThread(TaskClosure closure);

  // Query the number of |WorkerThread|s current available to do work.
  size_t GetAvailableTaskThreadCount() const;

 private:
  // Forward declaration of |WorkerThread| class. See implementation file for
  // class details.
  class WorkerThread;

  // Query whether the |TaskThreadPool| is active.
  //
  // @return True if the pool is still running.
  bool IsPoolRunning();

  // Signals to thread pool that a worker thread has become available for
  // task assignment.
  void SignalWorkerAvailable();

  // Closure reusable task loop to be executed by each |WorkerThread|.
  void WorkerLoopFunction();

  // Task Loop executed by each worker thread.
  void WorkerThreadLoop();

  // Number of worker threads currently available to execute tasks.
  std::atomic<int> num_worker_threads_available_;

  // Control of all worker thread loops.
  std::atomic<bool> is_pool_running_;

  // Container of available worker threads, waiting to be used.

  std::vector<WorkerThread> worker_threads_;

  // Condition to indicate that a worker thread has become available.
  std::condition_variable worker_available_condition_;

  // Mutex for the worker thread available condition notification receiver.
  std::mutex worker_available_mutex_;
};

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif

#endif  // RESONANCE_AUDIO_UTILS_TASK_THREAD_POOL_H_
