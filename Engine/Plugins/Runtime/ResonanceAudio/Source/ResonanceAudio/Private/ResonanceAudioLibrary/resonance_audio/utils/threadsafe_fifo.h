/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_THREADSAFE_FIFO_H_
#define RESONANCE_AUDIO_UTILS_THREADSAFE_FIFO_H_

#include <atomic>
#include <chrono>
#include <condition_variable>
#include <memory>
#include <mutex>
#include <thread>
#include <vector>

#include "base/logging.h"

namespace vraudio {

// Container to share preallocated data between threads. It is thread-safe for
// single producer - single consumer FIFO usage.
//
// @tparam T Object type that the FIFO handles.
template <typename T>
class ThreadsafeFifo {
 public:
  // Constructor preallocates the maximum number of objects in the FIFO queue
  // and defines the maximum waiting period before triggering a buffer underflow
  // or overflow event. Sleeping is enabled by default and can be disabled via
  // |EnableBlockingSleepUntilMethods|.
  //
  // @param max_objects Maximum number of objects in FIFO queue.
  explicit ThreadsafeFifo(size_t max_objects);

  // Constructor preallocates the maximum number of objects in the FIFO queue.
  // Sleeping is enabled by default and can be disabled via
  // |EnableBlockingSleepUntilMethods|.
  //
  // @param max_objects Maximum number of objects in FIFO queue.
  // @param init Initializer to be assigned to allocated objects.
  ThreadsafeFifo(size_t max_objects, const T& init);

  // Returns a pointer to an available input object T. If the queue is full, a
  // nullptr is returned.
  //
  // @return Pointer to an available input object. Nullptr if no input object is
  //     available.
  T* AcquireInputObject();

  // Releases a previously acquired input object to be pushed onto the FIFO
  // front.
  void ReleaseInputObject(const T* object);

  // Returns a pointer to an output object T. If the queue is empty, a nullptr
  // is returned.
  //
  // @return Pointer to the output object. Nullptr on empty queue.
  T* AcquireOutputObject();

  // Releases a previously acquired output object back to the FIFO.
  void ReleaseOutputObject(const T* object);

  // Blocks until the FIFO queue has an input object available or
  // |EnableBlockingSleepUntilMethods(false)| is called.
  //
  // Returns true if free slot is available.
  bool SleepUntilInputObjectIsAvailable() const;

  // Blocks until the FIFO queue has an output object available or
  // |EnableBlockingSleepUntilMethods(false)| is called.
  //
  // Returns true if an object is available.
  bool SleepUntilOutputObjectIsAvailable() const;

  // Allows for unblocking |SleepUntil[Input|Output]ObjectIsAvailable|
  // method.
  void EnableBlockingSleepUntilMethods(bool enable);

  // Returns the number of objects in the FIFO queue.
  size_t Size() const;

  // Returns true if FIFO queue is empty, false otherwise.
  bool Empty() const;

  // Returns true if FIFO queue is full, false otherwise.
  bool Full() const;

  // Clears the FIFO queue. This call is only thread-safe if called by the
  // consumer.
  void Clear();

 private:
  // Conditional to signal empty/full queue events.
  mutable std::mutex fifo_empty_mutex_;
  mutable std::condition_variable fifo_empty_conditional_;

  mutable std::mutex fifo_full_mutex_;
  mutable std::condition_variable fifo_full_conditional_;

  // Vector that stores all objects.
  std::vector<T> fifo_;
  size_t read_pos_;
  size_t write_pos_;

  // Atomic counter that reflects the size of |fifo_|.
  std::atomic<size_t> fifo_size_;

  std::atomic<bool> enable_sleeping_;
};

template <typename T>
ThreadsafeFifo<T>::ThreadsafeFifo(size_t max_objects)
    : fifo_(max_objects),
      read_pos_(0),
      write_pos_(0),
      fifo_size_(0),
      enable_sleeping_(true) {
  CHECK_GT(max_objects, 0) << "FIFO size must be greater than zero";
}

template <typename T>
ThreadsafeFifo<T>::ThreadsafeFifo(size_t max_objects, const T& init)
    : ThreadsafeFifo(max_objects) {
  for (auto& object : fifo_) {
    object = init;
  }
}

template <typename T>
T* ThreadsafeFifo<T>::AcquireInputObject() {
  if (Full()) {
    return nullptr;
  }
  CHECK_LT(fifo_size_, fifo_.size());

  // Add object to FIFO queue.
  return &fifo_[write_pos_];
}

template <typename T>
void ThreadsafeFifo<T>::ReleaseInputObject(const T* object) {
  DCHECK_EQ(object, &fifo_[write_pos_]);

  ++write_pos_;
  write_pos_ = write_pos_ % fifo_.size();
  if (fifo_size_.fetch_add(1) == 0) {
    {
      // Taking the lock and dropping it immediately assure that the notify
      // cannot happen between the check of the predicate and wait of the
      // |fifo_empty_conditional_|.
      std::lock_guard<std::mutex> lock(fifo_empty_mutex_);
    }
    // In case of an empty queue, notify reader.
    fifo_empty_conditional_.notify_one();
  }
}

template <typename T>
T* ThreadsafeFifo<T>::AcquireOutputObject() {
  if (Empty()) {
    return nullptr;
  }
  CHECK_GT(fifo_size_, 0);
  return &fifo_[read_pos_];
}

template <typename T>
void ThreadsafeFifo<T>::ReleaseOutputObject(const T* object) {
  DCHECK_EQ(object, &fifo_[read_pos_]);

  ++read_pos_;
  read_pos_ = read_pos_ % fifo_.size();

  if (fifo_size_.fetch_sub(1) == fifo_.size()) {
    {
      // Taking the lock and dropping it immediately assure that the notify
      // cannot happen between the check of the predicate and wait of the
      // |fifo_full_conditional_|.
      std::lock_guard<std::mutex> lock(fifo_full_mutex_);
    }
    // In case of a previously full queue, notify writer.
    fifo_full_conditional_.notify_one();
  }
}

template <typename T>
bool ThreadsafeFifo<T>::SleepUntilInputObjectIsAvailable() const {
  // In case of a full queue, wait to allow objects to be popped from the
  // FIFO queue.
  std::unique_lock<std::mutex> lock(fifo_full_mutex_);
  fifo_full_conditional_.wait(lock, [this]() {
    return fifo_size_.load() < fifo_.size() || !enable_sleeping_.load();
  });
  return fifo_size_.load() < fifo_.size();
}

template <typename T>
bool ThreadsafeFifo<T>::SleepUntilOutputObjectIsAvailable() const {
  // In case of an empty queue, wait for new objects to be added.
  std::unique_lock<std::mutex> lock(fifo_empty_mutex_);
  fifo_empty_conditional_.wait(lock, [this]() {
    return fifo_size_.load() > 0 || !enable_sleeping_.load();
  });
  return fifo_size_.load() > 0;
}

template <typename T>
void ThreadsafeFifo<T>::EnableBlockingSleepUntilMethods(bool enable) {
  enable_sleeping_ = enable;
  // Taking the lock and dropping it immediately assure that the notify
  // cannot happen between the check of the predicate and wait of the
  // |fifo_empty_conditional_| and |fifo_full_conditional_|.
  { std::lock_guard<std::mutex> lock(fifo_empty_mutex_); }
  { std::lock_guard<std::mutex> lock(fifo_full_mutex_); }
  fifo_empty_conditional_.notify_one();
  fifo_full_conditional_.notify_one();
}

template <typename T>
size_t ThreadsafeFifo<T>::Size() const {
  return fifo_size_.load();
}

template <typename T>
bool ThreadsafeFifo<T>::Empty() const {
  return fifo_size_.load() == 0;
}

template <typename T>
bool ThreadsafeFifo<T>::Full() const {
  return fifo_size_.load() == fifo_.size();
}

template <typename T>
void ThreadsafeFifo<T>::Clear() {
  while (!Empty()) {
    T* output = AcquireOutputObject();
    if (output != nullptr) {
      ReleaseOutputObject(output);
    }
  }
}

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_THREADSAFE_FIFO_H_
