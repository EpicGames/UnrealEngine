/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

// Prevent Visual Studio from complaining about std::copy_n.
#if defined(_WIN32)
#define _SCL_SECURE_NO_WARNINGS
#endif

#include "utils/vorbis_stream_encoder.h"

#include "vorbis/vorbisenc.h"
#include "base/audio_buffer.h"
#include "base/logging.h"

namespace vraudio {

VorbisStreamEncoder::VorbisStreamEncoder() : init_(false) {}

bool VorbisStreamEncoder::InitializeForFile(const std::string& filename,
                                            size_t num_channels,
                                            int sample_rate,
                                            EncodingMode encoding_mode,
                                            int bitrate, float quality) {
  output_file_stream_.open(filename,
                           std::ios::out | std::ios::trunc | std::ios::binary);
  if (!output_file_stream_.good()) {
    LOG(ERROR) << "Could not open output file: " << filename;
    return false;
  }

  int return_value = 1;
  vorbis_info_init(&vorbis_info_);

  switch (encoding_mode) {
    case EncodingMode::kVariableBitRate:
      return_value = vorbis_encode_init_vbr(
          &vorbis_info_, static_cast<long>(num_channels),
          sample_rate, quality);
      break;
    case EncodingMode::kAverageBitRate:
      return_value = vorbis_encode_init(
          &vorbis_info_, static_cast<long>(num_channels),
          sample_rate, -1 /* max_bitrate */, bitrate, -1 /* quality */);
      break;
    case EncodingMode::kUndefined:
    default:
      break;
  }

  if (return_value != 0) {
    return false;
  }

  vorbis_comment_init(&vorbis_comment_);
  vorbis_comment_add_tag(&vorbis_comment_, "ENCODER", "VrAudio");
  vorbis_analysis_init(&vorbis_state_, &vorbis_info_);
  vorbis_block_init(&vorbis_state_, &vorbis_block_);
  ogg_stream_init(&stream_state_, 1 /* serial_number */);

  // Generate Ogg header
  ogg_packet header;
  ogg_packet header_comments;
  ogg_packet header_code;

  vorbis_analysis_headerout(&vorbis_state_, &vorbis_comment_, &header,
                            &header_comments, &header_code);
  ogg_stream_packetin(&stream_state_, &header);
  ogg_stream_packetin(&stream_state_, &header_comments);
  ogg_stream_packetin(&stream_state_, &header_code);

  while (true) {
    return_value = ogg_stream_flush(&stream_state_, &ogg_page_);
    if (return_value == 0) {
      break;
    }
    if (!WriteOggPage()) {
      return false;
    }
  }
  init_ = true;
  return true;
}

bool VorbisStreamEncoder::AddPlanarBuffer(const float* const* input_ptrs,
                                          size_t num_channels,
                                          size_t num_frames) {
  CHECK(init_);
  PrepareVorbisBuffer(input_ptrs, num_channels, num_frames);
  return PerformEncoding();
}

bool VorbisStreamEncoder::FlushAndClose() {
  // Signal end of stream.
  vorbis_analysis_wrote(&vorbis_state_, 0);
  if (!PerformEncoding()) {
    return false;
  }

  output_file_stream_.close();

  vorbis_comment_clear(&vorbis_comment_);
  vorbis_dsp_clear(&vorbis_state_);
  vorbis_block_clear(&vorbis_block_);
  ogg_stream_clear(&stream_state_);
  vorbis_info_clear(&vorbis_info_);

  init_ = false;
  return true;
}

void VorbisStreamEncoder::PrepareVorbisBuffer(const float* const* input_ptrs,
                                              size_t num_channels,
                                              size_t num_frames) {
  float** buffer = vorbis_analysis_buffer(
      &vorbis_state_, static_cast<int>(num_channels * num_frames));
  for (size_t channel = 0; channel < num_channels; ++channel) {
    std::copy_n(input_ptrs[channel], num_frames, buffer[channel]);
  }

  vorbis_analysis_wrote(&vorbis_state_, static_cast<int>(num_frames));
}

bool VorbisStreamEncoder::PerformEncoding() {
  CHECK(init_);
  while (vorbis_analysis_blockout(&vorbis_state_, &vorbis_block_) == 1) {
    vorbis_analysis(&vorbis_block_, nullptr);
    vorbis_bitrate_addblock(&vorbis_block_);

    while (vorbis_bitrate_flushpacket(&vorbis_state_, &ogg_packet_)) {
      ogg_stream_packetin(&stream_state_, &ogg_packet_);

      bool end_of_stream = false;
      while (!end_of_stream) {
        int result = ogg_stream_pageout(&stream_state_, &ogg_page_);
        if (result == 0) {
          break;
        }
        if (!WriteOggPage()) {
          return false;
        }
        if (ogg_page_eos(&ogg_page_)) {
          end_of_stream = true;
        }
      }
    }
  }
  return true;
}

bool VorbisStreamEncoder::WriteOggPage() {
  output_file_stream_.write(reinterpret_cast<char*>(ogg_page_.header),
                            ogg_page_.header_len);
  output_file_stream_.write(reinterpret_cast<char*>(ogg_page_.body),
                            ogg_page_.body_len);
  if (!output_file_stream_.good()) {
    return false;
  }
  return true;
}

}  // namespace vraudio
