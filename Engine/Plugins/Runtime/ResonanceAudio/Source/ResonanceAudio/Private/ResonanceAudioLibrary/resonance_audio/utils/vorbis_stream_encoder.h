/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_VORBIS_STREAM_ENCODER_H_
#define RESONANCE_AUDIO_UTILS_VORBIS_STREAM_ENCODER_H_

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4324) /* structure was padded due to alignment specifier */
#pragma warning(disable:6011) /* Dereferencing NULL pointer '<ptr>' */
#pragma warning(disable:6385) /* Reading invalid data from '<array>': the readable size is '<array size>' bytes, but '<read size>' bytes may be read */
#pragma warning(disable:6386) /* Buffer overrun while writing to '<array>': the writable size is '<array size>' bytes, but '<write size>' bytes might be written */
#pragma warning(disable:4244) /* 'initializing': conversion from '<type>' to '<type>', possible loss of data */
#pragma warning(disable:4121) /* '<type>' alignment of a member was sensitive to packing */
#pragma warning(disable:4668) /* '<macro>' is not defined as a preprocessor macro, replacing with '0' for '#if/#elif' */
#endif

#include <fstream>
#include <iostream>

#include "base/integral_types.h"

#include "ogg/ogg.h"
#include "vorbis/codec.h"

namespace vraudio {

class VorbisStreamEncoder {
 public:
  // Supported encoding modes.
  enum class EncodingMode {
    kUndefined,
    // Variable bit rate mode (VBR).
    kVariableBitRate,
    // Average bit rate mode (ABR).
    kAverageBitRate,
  };

  VorbisStreamEncoder();

  // Initializes Vorbis encoding.
  //
  // @param filename Ogg vorbis output file. If it doesn't exist, it will be
  //    created. Existing files will be overwritten.
  // @param num_channels Number of input channels.
  // @param sample_rate Sample rate of input audio buffers.
  // @param encoding_mode Selects variable (VBR) or average encoding (ABR).
  // @param bitrate Target bitrate (only used when selecting ABR encoding).
  // @param quality Target quality (only used when selecting VBR encoding). The
  //     usable range is -.1 (lowest quality, smallest file) to 1. (highest
  //     quality, largest file).
  // @return False in case of file I/O errors or libvorbis initialization
  //     failures like non-supported channel/sample rate configuration.
  bool InitializeForFile(const std::string& filename, size_t num_channels,
                         int sample_rate, EncodingMode encoding_mode,
                         int bitrate, float quality);

  // Feeds input audio data into libvorbis encoder and triggers encoding.
  //
  // @param Array of pointers to planar channel data.
  // @param num_channels Number of input channels.
  // @param num_frames Number of input frames.
  // @return False in case of file I/O errors or missing encoder initialization.
  bool AddPlanarBuffer(const float* const* input_ptrs, size_t num_channels,
                       size_t num_frames);

  // Flushes the remaining audio buffers and closes the output file.
  //
  // @return False in case of file I/O errors or missing encoder initialization.
  bool FlushAndClose();

 private:
  // Copies input audio data into libvorbis encoder buffer.
  //
  // @param Array of pointers to planar channel data.
  // @param num_channels Number of input channels.
  // @param num_frames Number of input frames.
  void PrepareVorbisBuffer(const float* const* input_ptrs, size_t num_channels,
                           size_t num_frames);

  // Performs encoding of audio data prepared via |PrepareVorbisBuffer| or when
  // the end of stream has been signaled.
  //
  // @return False in case of file I/O errors or missing encoder initialization.
  bool PerformEncoding();

  // Dumps data from |ogg_page_| struct to |output_file_stream_|.
  //
  // @return False in case of file I/O errors or missing encoder initialization.
  bool WriteOggPage();

  // Flag indicating if encoder has been successfully initialized.
  bool init_;

  // Output file stream.
  std::ofstream output_file_stream_;

  // libogg structs.
  ogg_stream_state stream_state_;
  ogg_page ogg_page_;
  ogg_packet ogg_packet_;

  // libvorbis structs.
  vorbis_info vorbis_info_;
  vorbis_comment vorbis_comment_;
  vorbis_dsp_state vorbis_state_;
  vorbis_block vorbis_block_;
};

}  // namespace vraudio

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif

#endif  // RESONANCE_AUDIO_UTILS_VORBIS_STREAM_ENCODER_H_
