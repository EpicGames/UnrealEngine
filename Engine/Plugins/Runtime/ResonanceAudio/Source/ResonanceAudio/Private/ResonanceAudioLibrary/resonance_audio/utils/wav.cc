/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "utils/wav.h"

#include <cerrno>
#include <fstream>
#include <string>

#include "base/integral_types.h"
#include "base/logging.h"
#include "utils/wav_reader.h"

namespace vraudio {

Wav::Wav(size_t num_channels, int sample_rate,
         std::vector<int16_t>&& interleaved_samples)
    : num_channels_(num_channels),
      sample_rate_(sample_rate),
      interleaved_samples_(interleaved_samples) {}

Wav::~Wav() {}

std::unique_ptr<const Wav> Wav::CreateOrNull(const unsigned char* byte_array, size_t array_size) {
  WavReader wav_reader(byte_array, array_size);
  const size_t num_total_samples = wav_reader.GetNumTotalSamples();
  if (!wav_reader.IsHeaderValid() || num_total_samples == 0) {
    return nullptr;
  }
  std::vector<int16> interleaved_samples(num_total_samples);
  if (wav_reader.ReadSamples(num_total_samples, &interleaved_samples[0]) !=
      num_total_samples) {
    return nullptr;
  }
  return std::unique_ptr<Wav>(new Wav(wav_reader.GetNumChannels(),
                                      wav_reader.GetSampleRateHz(),
                                      std::move(interleaved_samples)));
}

}  // namespace vraudio
