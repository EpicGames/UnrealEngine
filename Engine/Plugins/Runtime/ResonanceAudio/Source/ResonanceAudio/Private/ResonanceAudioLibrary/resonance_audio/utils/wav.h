/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_WAV_H_
#define RESONANCE_AUDIO_UTILS_WAV_H_

#include <cstdint>
#include <memory>
#include <vector>

namespace vraudio {

// Wraps WavReader class to decode a wav file into memory.
class Wav {
 public:
  ~Wav();

  // Reads a RIFF WAVE from a byte array.
  static std::unique_ptr<const Wav> CreateOrNull(const unsigned char* byte_array, size_t array_size);

  // Returns reference to interleaved samples.
  const std::vector<int16_t>& interleaved_samples() const {
    return interleaved_samples_;
  }

  // Returns number of channels.
  size_t GetNumChannels() const { return num_channels_; }

  // Returns sample rate.
  int GetSampleRateHz() const { return sample_rate_; }

 private:
  // Private constructor used by static factory methods.
  //
  // @param num_channels Number of audio channels.
  // @param sample_rate Sample rate.
  // @param interleaved_samples Decoded interleaved samples.
  Wav(size_t num_channels, int sample_rate,
      std::vector<int16_t>&& interleaved_samples);

  // Number of channels.
  size_t num_channels_;

  // Sample rate.
  int sample_rate_;

  // Interleaved samples.
  std::vector<int16_t> interleaved_samples_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_WAV_H_
