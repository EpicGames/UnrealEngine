/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#include "utils/wav_reader.h"

#include <algorithm>
#include <string>
#include <memory>

#include "base/integral_types.h"
#include "base/logging.h"

namespace vraudio {

namespace {

struct ChunkHeader {
  char id[4];
  uint32 size;
};

struct WavFormat {
  ChunkHeader header;
  uint16 format_tag;                // Integer identifier of the format
  uint16 num_channels;              // Number of audio channels
  uint32 samples_rate;              // Audio sample rate
  uint32 average_bytes_per_second;  // Bytes per second (possibly approximate)
  uint16 block_align;      // Size in bytes of a sample block (all channels)
  uint16 bits_per_sample;  // Size in bits of a single per-channel sample
};
static_assert(sizeof(WavFormat) == 24, "Padding in WavFormat struct detected");

struct WavHeader {
  struct {
    ChunkHeader header;
    char format[4];
  } riff;
  WavFormat format;
  struct {
    ChunkHeader header;
  } data;
};

// Size of WAV header.
const size_t kWavHeaderSize = 44;

static_assert(sizeof(WavHeader) == kWavHeaderSize,
              "Padding in WavHeader struct detected");

// Supported WAV encoding formats.
static const uint16 kExtensibleWavFormat = 0xfffe;
static const uint16 kPcmFormat = 0x1;
}  // namespace

WavReader::WavReader(const unsigned char* byte_array, const size_t num_bytes)
    : source(byte_array, num_bytes),
      num_channels_(0),
      sample_rate_hz_(-1),
      num_total_samples_(0),
      num_remaining_samples_(0),
      pcm_offset_bytes_(0) {
  init_ = ParseHeader();
}

size_t WavReader::ReadBinaryDataFromStream(void* target_ptr, size_t size) {
  // if we are out of bounds, return 0.
  if (source.current_byte_index + size > source.num_bytes) {
    return 0;
  }

  //binary_stream_->read(static_cast<char*>(target_ptr), size);
  memcpy(target_ptr, &source.byte_array[source.current_byte_index], size);
  source.current_byte_index += size;
  return size;
}

bool WavReader::ParseHeader() {
  WavHeader header;
  // Exclude data field to be able to optionally parse the two-byte extension
  // field.
  if (ReadBinaryDataFromStream(&header, kWavHeaderSize - sizeof(header.data)) !=
      kWavHeaderSize - sizeof(header.data))
    return false;
  const uint32 format_size = header.format.header.size;
  // Size of |WavFormat| without |ChunkHeader|.
  static const uint32 kFormatSubChunkHeader =
      sizeof(WavFormat) - sizeof(ChunkHeader);
  if (format_size < kFormatSubChunkHeader) {
    return false;
  }
  if (format_size != kFormatSubChunkHeader) {
    // Parse optional extension fields.
    int16 extension_size;
    if (ReadBinaryDataFromStream(&extension_size, sizeof(extension_size)) !=
        sizeof(extension_size))
      return false;
    int8 extension_data;
    for (size_t i = 0; i < static_cast<size_t>(extension_size); ++i) {
      if (ReadBinaryDataFromStream(&extension_data, sizeof(extension_data)) !=
          sizeof(extension_data))
        return false;
    }
  }
  if (header.format.format_tag == kExtensibleWavFormat) {
    // Skip extensible wav format "fact" header.
    ChunkHeader fact_header;
    if (ReadBinaryDataFromStream(&fact_header, sizeof(fact_header)) !=
        sizeof(fact_header)) {
      return false;
    }
    if (std::string(fact_header.id, 4) != "fact") {
      return false;
    }
    int8 fact_data;
    for (size_t i = 0; i < static_cast<size_t>(fact_header.size); ++i) {
      if (ReadBinaryDataFromStream(&fact_data, sizeof(fact_data)) !=
          sizeof(fact_data))
        return false;
    }
  }

  // Read the "data" header.
  if (ReadBinaryDataFromStream(&header.data, sizeof(header.data)) !=
      sizeof(header.data)) {
    return false;
  }

  num_channels_ = header.format.num_channels;
  sample_rate_hz_ = header.format.samples_rate;

  bytes_per_sample_ = header.format.bits_per_sample / 8;
  if (bytes_per_sample_ != sizeof(int16)) {
    return false;
  }
  const size_t bytes_in_payload = header.data.header.size;
  num_total_samples_ = bytes_in_payload / bytes_per_sample_;
  num_remaining_samples_ = num_total_samples_;

  if (header.format.num_channels == 0 || num_total_samples_ == 0 ||
      bytes_in_payload % bytes_per_sample_ != 0 ||
      (header.format.format_tag != kPcmFormat &&
       header.format.format_tag != kExtensibleWavFormat) ||
      (std::string(header.riff.header.id, 4) != "RIFF") ||
      (std::string(header.riff.format, 4) != "WAVE") ||
      (std::string(header.format.header.id, 4) != "fmt ") ||
      (std::string(header.data.header.id, 4) != "data")) {
    return false;
  }

  pcm_offset_bytes_ = static_cast<uint64>(source.current_byte_index);
  return true;
}

size_t WavReader::ReadSamples(size_t num_samples, int16* target_buffer) {
  const size_t num_samples_to_read =
      std::min(num_remaining_samples_, num_samples);
  if (num_samples_to_read == 0) {
    return 0;
  }
  const size_t num_bytes_read =
      ReadBinaryDataFromStream(target_buffer, num_samples * sizeof(int16));
  const size_t num_samples_read = num_bytes_read / bytes_per_sample_;

  num_remaining_samples_ -= num_samples_read;
  return num_samples_read;
}

int64 WavReader::SeekToFrame(const uint64 frame_position) {
  DCHECK_GT(num_channels_, 0U);
  if (frame_position <= (num_total_samples_ / num_channels_)) {
    const uint64 seek_position_byte =
        pcm_offset_bytes_ + frame_position * num_channels_ * bytes_per_sample_;
    
	//ensure the offset position isn't beyond the end of our stream:
	DCHECK_LT(seek_position_byte, source.num_bytes);

	// for 32 bit devices, we need to truncate seek_position_byte to a 32 bit value.
	// If seek_position_byte is less than num_bytes, we should not need to worry about
	// overflow.
	source.current_byte_index = static_cast<size_t>(seek_position_byte);
  }

  int64 binary_stream_position_byte =
      static_cast<int64>(source.current_byte_index);
  if (binary_stream_position_byte < 0) {
    // Return an error status if the actual bitstream position could not be
    // queried.
    return binary_stream_position_byte;
  }

  if (static_cast<uint64>(binary_stream_position_byte) > pcm_offset_bytes_) {
    DCHECK_GT(bytes_per_sample_, 0U);
    return (static_cast<uint64>(binary_stream_position_byte) -
            pcm_offset_bytes_) /
           (bytes_per_sample_ * num_channels_);
  }

  return 0;
}

size_t WavReader::GetNumTotalSamples() const { return num_total_samples_; }

size_t WavReader::GetNumChannels() const { return num_channels_; }

int WavReader::GetSampleRateHz() const { return sample_rate_hz_; }

bool WavReader::IsHeaderValid() const { return init_; }

}  // namespace vraudio
