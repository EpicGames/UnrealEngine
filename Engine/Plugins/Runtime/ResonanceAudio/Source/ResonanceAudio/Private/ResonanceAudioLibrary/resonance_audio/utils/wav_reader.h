/*
Copyright 2018 Google Inc. All Rights Reserved.

Licensed under the Apache License, Version 2.0 (the "License");
you may not use this file except in compliance with the License.
You may obtain a copy of the License at

    http://www.apache.org/licenses/LICENSE-2.0

Unless required by applicable law or agreed to in writing, software
distributed under the License is distributed on an "AS-IS" BASIS,
WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
See the License for the specific language governing permissions and
limitations under the License.
*/

#ifndef RESONANCE_AUDIO_UTILS_WAV_READER_H_
#define RESONANCE_AUDIO_UTILS_WAV_READER_H_

#include "HAL/PlatformCrt.h"

#include <cstdint>
#include <istream>

#include "base/integral_types.h"

namespace vraudio {

// Basic RIFF WAVE decoder that supports multichannel 16-bit PCM.
class WavReader {
 public:

  // Constructor that takes a pointer to a raw byte array and the size of that byte stream.
  // 
  // @param byte_array Pointer to raw array of bytes.
  // @param num_bytes Size of byte_array.
  explicit WavReader(const unsigned char* byte_array, const size_t num_bytes);

  // True if WAV header was successfully parsed.
  bool IsHeaderValid() const;

  // Returns the total number of samples defined in the WAV header. Note that
  // the actual number of samples in the file can differ.
  size_t GetNumTotalSamples() const;

  // Returns number of channels.
  size_t GetNumChannels() const;

  // Returns sample rate in Hertz.
  int GetSampleRateHz() const;

  // Seek to a specific frame position within the wave file. If frame_position
  // is not a valid address, then the internal read position remains unchanged.
  //
  // @param frame_position Destination frame position for play cursor.
  // @return Actual frame position of cursor after this seek operation. A
  //     negative return value indicates a stream failure.
  int64 SeekToFrame(const uint64 frame_position);

  // Reads samples from WAV file into target buffer.
  //
  // @param num_samples Number of samples to read.
  // @param target_buffer Target buffer to write to.
  // @return Number of decoded samples.
  size_t ReadSamples(size_t num_samples, int16_t* target_buffer);

 private:
  // Parses WAV header.
  //
  // @return True on success.
  bool ParseHeader();

  // Helper method to read binary data from input stream.
  //
  // @param size Number of bytes to read.
  // @param target_ptr Target buffer to write to.
  // @return Number of bytes read.
  size_t ReadBinaryDataFromStream(void* target_ptr, size_t size);

  struct SourceByteArray
  {
	  const unsigned char* byte_array;
	  const size_t num_bytes;
	  size_t current_byte_index;

	  SourceByteArray(const unsigned char* in_byte_array, const size_t in_num_bytes)
		  : byte_array(in_byte_array)
		  , num_bytes(in_num_bytes)
		  , current_byte_index(0)
	  {
	  }
  };

  // source we are deserializing.
  SourceByteArray source;

  // Flag indicating if the WAV header was parsed successfully.
  bool init_;

  // Number of audio channels.
  size_t num_channels_;

  // Sample rate in Hertz.
  int sample_rate_hz_;

  // Total number of samples.
  size_t num_total_samples_;

  // Number of remaining samples in WAV file.
  size_t num_remaining_samples_;

  // Bytes per sample as defined in the WAV header.
  size_t bytes_per_sample_;

  // Offset into data stream where PCM data begins.
  uint64 pcm_offset_bytes_;
};

}  // namespace vraudio

#endif  // RESONANCE_AUDIO_UTILS_WAV_READER_H_
