//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioModule.h"

#include "AudioDevice.h"
#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"
#include "ResonanceAudioPluginListener.h"
#include "ResonanceAudioReverb.h"
#include "ResonanceAudioSettings.h"
#include "ResonanceAudioSpatialization.h"

IMPLEMENT_MODULE(ResonanceAudio::FResonanceAudioModule, ResonanceAudio)

namespace ResonanceAudio
{
	void* FResonanceAudioModule::ResonanceAudioDynamicLibraryHandle = nullptr;
	UResonanceAudioSpatializationSourceSettings* FResonanceAudioModule::GlobalSpatializationSourceSettings = nullptr;

	static bool bModuleInitialized = false;

	FResonanceAudioModule::FResonanceAudioModule()
	{
	}

	FResonanceAudioModule::~FResonanceAudioModule()
	{
	}

	void FResonanceAudioModule::StartupModule()
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		check(bModuleInitialized == false);
		bModuleInitialized = true;

		// Register the Resonance Audio plugin factories.
		IModularFeatures::Get().RegisterModularFeature(FSpatializationPluginFactory::GetModularFeatureName(), &SpatializationPluginFactory);
		IModularFeatures::Get().RegisterModularFeature(FReverbPluginFactory::GetModularFeatureName(), &ReverbPluginFactory);

		if (!IsRunningDedicatedServer() && !GlobalSpatializationSourceSettings)
		{
			// Load the global source preset settings:
			const FSoftObjectPath GlobalPluginPresetName = GetDefault<UResonanceAudioSettings>()->GlobalSourcePreset;
			if (GlobalPluginPresetName.IsValid())
			{
				GlobalSpatializationSourceSettings = LoadObject<UResonanceAudioSpatializationSourceSettings>(nullptr, *GlobalPluginPresetName.ToString());
			}
			
			if (!GlobalSpatializationSourceSettings)
			{
				GlobalSpatializationSourceSettings = NewObject<UResonanceAudioSpatializationSourceSettings>(UResonanceAudioSpatializationSourceSettings::StaticClass(), TEXT("Default Global Resonance Spatialization Preset"));
			}

			if (GlobalSpatializationSourceSettings)
			{ 
				GlobalSpatializationSourceSettings->AddToRoot();
			}
		}
	}

	void FResonanceAudioModule::ShutdownModule()
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		check(bModuleInitialized == true);
		bModuleInitialized = false;
		UE_LOGF(LogResonanceAudio, Log, "Resonance Audio Module is Shutdown");
	}

	IAudioPluginFactory* FResonanceAudioModule::GetPluginFactory(EAudioPlugin PluginType)
	{
		switch (PluginType)
		{
		case EAudioPlugin::SPATIALIZATION:
			return &SpatializationPluginFactory;
		case EAudioPlugin::REVERB:
			return &ReverbPluginFactory;
		default:
			return nullptr;
		}
	}

	void FResonanceAudioModule::RegisterAudioDevice(FAudioDevice* AudioDeviceHandle)
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		if (!RegisteredAudioDevices.Contains(AudioDeviceHandle))
		{
			TAudioPluginListenerPtr NewResonanceAudioPluginListener = TAudioPluginListenerPtr(new FResonanceAudioPluginListener());
			AudioDeviceHandle->RegisterPluginListener(NewResonanceAudioPluginListener);
			RegisteredAudioDevices.Add(AudioDeviceHandle);
		}
	}

	void FResonanceAudioModule::UnregisterAudioDevice(FAudioDevice* AudioDeviceHandle)
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		RegisteredAudioDevices.Remove(AudioDeviceHandle);
		UE_LOGF(LogResonanceAudio, Log, "Audio Device unregistered from Resonance");
	}

	UResonanceAudioSpatializationSourceSettings* FResonanceAudioModule::GetGlobalSpatializationSourceSettings()
	{
		return GlobalSpatializationSourceSettings;
	}

	TAudioSpatializationPtr FSpatializationPluginFactory::CreateNewSpatializationPlugin(FAudioDevice* OwningDevice)
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		// Register the audio device to the Resonance Audio module.
		FResonanceAudioModule* Module = &FModuleManager::GetModuleChecked<FResonanceAudioModule>("ResonanceAudio");
		if (Module != nullptr)
		{
			Module->RegisterAudioDevice(OwningDevice);
		}
		return TAudioSpatializationPtr(new FResonanceAudioSpatialization());
	}

	TAudioReverbPtr FReverbPluginFactory::CreateNewReverbPlugin(FAudioDevice* OwningDevice)
	{
		LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);
		// Register the audio device to the Resonance Audio module.
		FResonanceAudioModule* Module = &FModuleManager::GetModuleChecked<FResonanceAudioModule>("ResonanceAudio");
		if (Module != nullptr)
		{
			Module->RegisterAudioDevice(OwningDevice);
		}
		return TAudioReverbPtr(new FResonanceAudioReverb());
	}

} // namespace ResonanceAudio
