//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "IAudioExtensionPlugin.h"
#include "IResonanceAudioModule.h"
#include "ResonanceAudioSpatializationSourceSettings.h"
#include "AudioPluginUtilities.h"
#include "AudioDevice.h"
#include "ResonanceAudioAmbisonics.h"

namespace ResonanceAudio
{
	/**********************************************************************/
	/* Spatialization Plugin Factory                                      */
	/**********************************************************************/
	class FSpatializationPluginFactory : public IAudioSpatializationFactory
	{
	public:

		virtual FString GetDisplayName() override
		{
			static FString DisplayName = FString(TEXT("Resonance Audio"));
			return DisplayName;
		}

		virtual bool SupportsPlatform(const FString& PlatformName) override
		{
			return true;
		}

		virtual TAudioSpatializationPtr CreateNewSpatializationPlugin(FAudioDevice* OwningDevice) override;
		virtual UClass* GetCustomSpatializationSettingsClass() const override { return UResonanceAudioSpatializationSourceSettings::StaticClass(); };
		
		virtual bool IsExternalSend() override { return true; }

		virtual int32 GetMaxSupportedChannels() override
		{
			return 2;
		}

		virtual bool ReturnsToSubmixGraph() const override { return true; }

	};

	/******************************************************/
	/* Reverb Plugin Factory                              */
	/******************************************************/
	class FReverbPluginFactory : public IAudioReverbFactory
	{
	public:

		virtual FString GetDisplayName() override
		{
			static FString DisplayName = FString(TEXT("Resonance Audio"));
			return DisplayName;
		}

		virtual bool SupportsPlatform(const FString& PlatformName) override
		{
			return true;
		}

		virtual TAudioReverbPtr CreateNewReverbPlugin(FAudioDevice* OwningDevice) override;
		virtual bool IsExternalSend() override { return true; }
		virtual bool ReturnsToSubmixGraph() const override { return true; }
	};

	/*********************************************************/
	/* Resonance Audio Module                                */
	/*********************************************************/
	class FResonanceAudioModule : public IResonanceAudioModule
	{
	public:
		FResonanceAudioModule();
		~FResonanceAudioModule();

		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		// Returns a pointer to a given PluginType or nullptr if PluginType is invalid.
		IAudioPluginFactory* GetPluginFactory(EAudioPlugin PluginType);

		// Registers given audio device with the Resonance Audio module.
		void RegisterAudioDevice(FAudioDevice* AudioDeviceHandle);

		// Unregisters given audio device from the Resonance Audio module.
		void UnregisterAudioDevice(FAudioDevice* AudioDeviceHandle);

		// Get all registered Audio Devices that are using Resonance Audio as their reverb plugin.
		TArray<FAudioDevice*>& GetRegisteredAudioDevices() { return RegisteredAudioDevices; };

		// Returns Resonance Audio API dynamic library handle.
		static void* GetResonanceAudioDynamicLibraryHandle() { return ResonanceAudioDynamicLibraryHandle; };

		// Returns the global spatialization source settings set in the project settings
		static UResonanceAudioSpatializationSourceSettings* GetGlobalSpatializationSourceSettings();

	private:
		// Resonance Audio API dynamic library handle.
		static void* ResonanceAudioDynamicLibraryHandle;

		// The global resonance audio spatialization source settings
		static UResonanceAudioSpatializationSourceSettings* GlobalSpatializationSourceSettings;

		// List of registered audio devices.
		TArray<FAudioDevice*> RegisteredAudioDevices;

		// Plugin factories.
		FSpatializationPluginFactory SpatializationPluginFactory;
		FReverbPluginFactory ReverbPluginFactory;
		FAmbisonicsFactory AmbisonicsFactory;
	};

}  // namespace ResonanceAudio
