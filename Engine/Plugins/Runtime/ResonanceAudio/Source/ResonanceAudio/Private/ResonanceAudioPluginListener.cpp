//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioPluginListener.h"
#include "ResonanceAudioAmbisonics.h"
#include "AudioDevice.h"
#include "Async/Async.h"
#include "Components/BrushComponent.h"
#include "Engine/World.h"
#include "Model.h"

namespace ResonanceAudioConsoleVariables
{
	int32 bUsingReverb = 1;
	FAutoConsoleVariableRef CVarUsingReverb(
		TEXT("au.Resonance.UsingReverb"),
		bUsingReverb,
		TEXT("Allows Resonance to Query AudioVolumes for reverb effects.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

} // namespace ResonanceAudioConsoleVariables

namespace ResonanceAudio
{
	FResonanceAudioPluginListener::FResonanceAudioPluginListener()
		: ResonanceAudioApi(nullptr)
		, OwningAudioDevice(nullptr)
		, ResonanceAudioModule(nullptr)
		, ReverbPtr(nullptr)
		, SpatializationPtr(nullptr)
	{
	}

	FResonanceAudioPluginListener::~FResonanceAudioPluginListener()
	{
		if (ResonanceAudioApi.IsValid())
		{
			check(OwningAudioDevice);

			FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);
			ResonanceApiMap.Remove(OwningAudioDevice);
		}
	}

	void FResonanceAudioPluginListener::OnListenerInitialize(FAudioDevice* AudioDevice, UWorld* ListenerWorld)
	{
		if (!ResonanceAudioModule)
		{
			ResonanceAudioModule = &FModuleManager::GetModuleChecked<FResonanceAudioModule>("ResonanceAudio");
		}

		// Initialize Resonance Audio API.
		const size_t FramesPerBuffer = static_cast<size_t>(AudioDevice->GetBufferLength());
		const int SampleRate = static_cast<int>(AudioDevice->GetSampleRate());

		vraudio::ResonanceAudioApi* Api = CreateResonanceAudioApi(ResonanceAudioModule->GetResonanceAudioDynamicLibraryHandle(), 2 /* num channels */, FramesPerBuffer, SampleRate);

		ResonanceAudioApi = TSharedPtr<vraudio::ResonanceAudioApi, ESPMode::ThreadSafe>(Api);

		if (!ResonanceAudioApi.IsValid())
		{
			UE_LOGF(LogResonanceAudio, Error, "Failed to initialize Resonance Audio API");
			return;
		}
		else
		{
			FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);
			check(AudioDevice);
			ResonanceApiMap.FindOrAdd(AudioDevice, ResonanceAudioApi);
			OwningAudioDevice = AudioDevice;
		}

		// Get the names of the currently configured spatializer / reverb
		const FString CurrentSpatializationPluginName = AudioPluginUtilities::GetDesiredPluginName(EAudioPlugin::SPATIALIZATION);
		const FString CurrentReverbPluginName = AudioPluginUtilities::GetDesiredPluginName(EAudioPlugin::REVERB);

		IAudioReverb* CurrentReverbPtr = AudioDevice->ReverbPluginInterface.Get();
		IAudioSpatialization* CurrentSpatializationPtr = AudioDevice->GetSpatializationPluginInterface().Get();

		// Check if reverb and spatializer are Resonance
		bool ReverbIsResonance = CurrentReverbPtr != nullptr && CurrentReverbPluginName.Equals("Resonance Audio");
		bool SpatializationIsResonance = CurrentSpatializationPtr != nullptr && CurrentSpatializationPluginName.Equals("Resonance Audio");

		if (ReverbIsResonance && SpatializationIsResonance)
		{
			// Resonance is configured correctly, setup the pointers.
			ReverbPtr = (FResonanceAudioReverb*)CurrentReverbPtr;
			SpatializationPtr = (FResonanceAudioSpatialization*)CurrentSpatializationPtr;
			ReverbPtr->SetResonanceAudioApi(ResonanceAudioApi.Get());
			SpatializationPtr->SetResonanceAudioApi(ResonanceAudioApi.Get());
			UE_LOGF(LogResonanceAudio, Display, "Resonance Audio Listener is initialized");
		}
		else if (CurrentReverbPluginName.Equals("Resonance Audio") || CurrentSpatializationPluginName.Equals("Resonance Audio"))
		{
			// Only one of reverb or spatializer was set to Resonance. Inform user that both need to be enabled
			UE_LOGF(LogResonanceAudio, Error, "Resonance Audio requires both Reverb and Spatialization plugins. Please enable them in the Project Settings.");
			return;
		}
	}

	void FResonanceAudioPluginListener::OnListenerUpdated(FAudioDevice* AudioDevice, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds)
	{
		if (!ResonanceAudioApi.IsValid())
		{
			UE_LOGF(LogResonanceAudio, Error, "Resonance Audio API not loaded");
			return;
		}
		else
		{
			const FVector ConvertedPosition = ConvertToResonanceAudioCoordinates(ListenerTransform.GetLocation());
			ResonanceAudioApi->SetHeadPosition(ConvertedPosition.X, ConvertedPosition.Y, ConvertedPosition.Z);
			const FQuat ConvertedRotation = ConvertToResonanceAudioRotation(ListenerTransform.GetRotation());
			ResonanceAudioApi->SetHeadRotation(ConvertedRotation.X, ConvertedRotation.Y, ConvertedRotation.Z, ConvertedRotation.W);
		}
	}

	void FResonanceAudioPluginListener::OnListenerShutdown(FAudioDevice* AudioDevice)
	{
		if (ResonanceAudioModule)
		{
			ResonanceAudioModule->UnregisterAudioDevice(AudioDevice);
		}

		UE_LOGF(LogResonanceAudio, Display, "Resonance Audio Listener is shutdown");
	}

	void FResonanceAudioPluginListener::OnTick(UWorld* InWorld, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds)
	{
		if (ResonanceAudioConsoleVariables::bUsingReverb != 0)
		{
			if (ReverbPtr != nullptr && InWorld->AudioVolumes.Num() > 0)
			{
				AAudioVolume* CurrentVolume = InWorld->GetAudioSettings(ListenerTransform.GetLocation(), nullptr, nullptr);
				if (CurrentVolume != nullptr)
				{
					UResonanceAudioReverbPluginPreset* Preset = Cast<UResonanceAudioReverbPluginPreset>(CurrentVolume->GetReverbSettings().ReverbPluginEffect);
					if (Preset != nullptr && Preset->UseAudioVolumeTransform())
					{
						// Obtain Resonance Audio room transform from the Unreal Audio Volume transform.
						const FVector CurrentVolumePosition = CurrentVolume->GetActorLocation();
						Preset->SetRoomPosition(CurrentVolumePosition);
						const FQuat CurrentVolumeRotation = CurrentVolume->GetActorQuat();
						Preset->SetRoomRotation(CurrentVolumeRotation);
						const FVector CurrentVolumeDimensions = CurrentVolume->GetActorScale3D();
						const FVector CurrentBrushShapeExtents = 2.0f * CurrentVolume->GetBrushComponent()->Brush->Bounds.BoxExtent;
						const FVector RoomDimensions = CurrentVolumeDimensions * CurrentBrushShapeExtents;
						// The default Audio Volume cube size is 200cm, please see UCubeBuilder constructor for initialization details.
						Preset->SetRoomDimensions(RoomDimensions);
					}
					// Activate this preset or no room effects if nullptr.
					ReverbPtr->SetPreset(Preset);
				}
				else
				{
					ReverbPtr->SetPreset(nullptr);
					UE_LOGF(LogResonanceAudio, Verbose, "Set reverb preset to nullptr");
				}
			}
		}
	}

	FResonanceAudioApiSharedPtr FResonanceAudioPluginListener::GetResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice)
	{
		FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);

		check(InAudioDevice);
		if (FResonanceAudioApiSharedPtr* ResonanceApiPtr = ResonanceApiMap.Find(InAudioDevice))
		{
			return *ResonanceApiPtr;
		}
		else
		{
			return FResonanceAudioApiSharedPtr(nullptr);
		}
	}

	void FResonanceAudioPluginListener::SetResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice, FResonanceAudioApiSharedPtr InResonanceSystem)
	{
		FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);
		ResonanceApiMap.Add(InAudioDevice, InResonanceSystem);
	}

	void FResonanceAudioPluginListener::RemoveResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice)
	{
		FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);
		ResonanceApiMap.Remove(InAudioDevice);
	}

	void FResonanceAudioPluginListener::RemoveResonanceAPIForAudioDevice(const vraudio::ResonanceAudioApi* InResonanceSystem)
	{
		FScopeLock ScopeLock(&ResonanceApiMapCriticalSection);

		const FAudioDevice* AudioDeviceKey = nullptr;
		for (auto& Pair : ResonanceApiMap)
		{
			if (Pair.Value.Get() == InResonanceSystem)
			{
				AudioDeviceKey = Pair.Key;
				break;
			}
		}

		checkf(AudioDeviceKey, TEXT("RemoveResonanceAPIForAudioDevice was called for a resonance system that wasn't registered using SetResonanceAPIForAudioDevice!"));
		ResonanceApiMap.Remove(AudioDeviceKey);
	}

	TMap<const FAudioDevice*, FResonanceAudioApiSharedPtr> FResonanceAudioPluginListener::ResonanceApiMap;

	FCriticalSection FResonanceAudioPluginListener::ResonanceApiMapCriticalSection;

}  // namespace ResonanceAudio
