//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "api/resonance_audio_api.h"

#include "AudioPluginUtilities.h"
#include "ResonanceAudioCommon.h"
#include "ResonanceAudioModule.h"
#include "ResonanceAudioReverb.h"
#include "ResonanceAudioSpatialization.h"

namespace ResonanceAudio
{

	// Dispatches listener information to the Resonance Audio plugins and owns the vraudio::ResonanceAudioApi.
	class FResonanceAudioPluginListener : public IAudioPluginListener
	{
	public:

		FResonanceAudioPluginListener();
		~FResonanceAudioPluginListener();

		virtual void OnListenerInitialize(FAudioDevice* AudioDevice, UWorld* ListenerWorld) override;
		virtual void OnListenerUpdated(FAudioDevice* AudioDevice, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds) override;
		virtual void OnListenerShutdown(FAudioDevice* AudioDevice) override;
		virtual void OnTick(UWorld* InWorld, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds) override;

		static FResonanceAudioApiSharedPtr GetResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice);
		static void SetResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice, FResonanceAudioApiSharedPtr InResonanceSystem);
		static void RemoveResonanceAPIForAudioDevice(const FAudioDevice* InAudioDevice);
		static void RemoveResonanceAPIForAudioDevice(const vraudio::ResonanceAudioApi* InResonanceSystem);

	private:
		// Resonance Audio API instance.
		FResonanceAudioApiSharedPtr ResonanceAudioApi;

		// Map of Resonance API systems to Audio Device IDs.
		static TMap<const FAudioDevice*, FResonanceAudioApiSharedPtr> ResonanceApiMap;
		static FCriticalSection ResonanceApiMapCriticalSection;

		// This audio device ptr should only be used to remove the ResonanceAudioApi* from the ResonanceApiMap on destruction.
		const FAudioDevice* OwningAudioDevice;

		class FResonanceAudioModule* ResonanceAudioModule;
		class FResonanceAudioReverb* ReverbPtr;
		class FResonanceAudioSpatialization* SpatializationPtr;
	};

} // namespace ResonanceAudio
