// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ResonanceEigenHelper.h"
#include "base/misc_math.h"
#include "base/audio_buffer.h"
#include "node/processing_node.h"