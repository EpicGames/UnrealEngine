//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioReverb.h"

#include "AudioDeviceManager.h"
#include "AudioMixerDevice.h"
#include "ResonanceAudioCommon.h"
#include "ResonanceAudioModule.h"
#include "ResonanceAudioSettings.h"
#include "Sound/SoundSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioReverb)

static int32 IgnoreUserResonanceSubmixCVar = 0;
FAutoConsoleVariableRef CVarIgnoreUserResonanceSubmix(
	TEXT("au.IgnoreUserResonanceSubmix"),
	IgnoreUserResonanceSubmixCVar,
	TEXT("When set to 1, the resonance project setting will be bypassed.\n")
	TEXT("1: Submix Effects are disabled."),
	ECVF_Default);


namespace ResonanceAudio
{
	/*********************************************/
	/* Resonance Audio Reverb                    */
	/*********************************************/
	FResonanceAudioReverb::FResonanceAudioReverb()
		: ResonanceAudioApi(nullptr)
		, ResonanceAudioModule(nullptr)
		, ReverbPluginPreset(nullptr)
		, TemporaryStereoBuffer()
		, ReverbSubmixPtr(nullptr)
	{
	}

	FResonanceAudioReverb::~FResonanceAudioReverb()
	{
		ReverbPluginPreset = nullptr;
		if (!Audio::GRemoveSubmixesFromRoot && ReverbSubmixPtr)
		{
			ReverbSubmixPtr->SetParentSubmix(nullptr, false /* bModifyAssets */);
			ReverbSubmixPtr->RemoveFromRoot();
		}

	}

	void FResonanceAudioReverb::Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
		ResonanceAudioModule = &FModuleManager::GetModuleChecked<FResonanceAudioModule>("ResonanceAudio");
	}

	void FResonanceAudioReverb::OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, UReverbPluginSourceSettingsBase* InSettings)
	{
	}

	void FResonanceAudioReverb::OnReleaseSource(const uint32 SourceId)
	{
	}

	void FResonanceAudioReverb::ProcessSourceAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
		// This is left empty as we don't implement per-source rendering in the Resonance Audio Reverb plugin.
	}

	void FResonanceAudioReverb::ProcessMixedAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
	{
		if (ResonanceAudioModule == nullptr || ResonanceAudioApi == nullptr)
		{
			return;
		}

		UpdateRoomEffects();

		// This is where we trigger Resonance Audio processing.
		if (OutData.NumChannels == 2)
		{
			ResonanceAudioApi->FillInterleavedOutputBuffer(2 /*num. output channels */, InData.NumFrames, OutData.AudioBuffer->GetData());
		}
		else if (OutData.NumChannels > 2)
		{
			// FillInterleavedOutputBuffer (below) can fail if the graph does not have any sources to process, so the temp buffer must be zeroed
			TemporaryStereoBuffer.Reset();
			TemporaryStereoBuffer.AddZeroed(2 * InData.NumFrames);

			ResonanceAudioApi->FillInterleavedOutputBuffer(2 /*num. output channels */, InData.NumFrames, TemporaryStereoBuffer.GetData());
			float* OutputBufferPtr = OutData.AudioBuffer->GetData();
			for (int32 i = 0; i < InData.NumFrames; ++i)
			{
				const int32 output_offset = i * OutData.NumChannels;
				const int32 stereo_offset = i * 2;
				OutputBufferPtr[output_offset] = TemporaryStereoBuffer[stereo_offset];
				OutputBufferPtr[output_offset + 1] = TemporaryStereoBuffer[stereo_offset + 1];
			}
		}
		else if (OutData.NumChannels == 1)
		{
			UE_LOGF(LogResonanceAudio, Warning, "Resonance Audio Reverb connected to 1-channel output, down-mixing spatialized audio");

			// FillInterleavedOutputBuffer (below) can fail if the graph does not have any sources to process, so the temp buffer must be zeroed
			TemporaryStereoBuffer.Reset();
			TemporaryStereoBuffer.AddZeroed(2 * InData.NumFrames);

			ResonanceAudioApi->FillInterleavedOutputBuffer(2 /*num. output channels */, InData.NumFrames, TemporaryStereoBuffer.GetData());
			float* OutputBufferPtr = OutData.AudioBuffer->GetData();
			for (int32 i = 0; i < InData.NumFrames; ++i)
			{
				const int32 stereo_offset = i * 2;
				OutputBufferPtr[i] = 0.5f * (TemporaryStereoBuffer[stereo_offset] + TemporaryStereoBuffer[stereo_offset + 1]);
			}
		}
	}

	static UResonanceAudioReverbPluginPreset* GlobalReverbPluginPreset = nullptr;
	void FResonanceAudioReverb::SetGlobalReverbPluginPreset(UResonanceAudioReverbPluginPreset* InPreset)
	{
		//If we're using the global reverb preset right now, ensure that we update ReverbPluginPreset as well.
		if (GlobalReverbPluginPreset == ReverbPluginPreset)
		{
			GlobalReverbPluginPreset = InPreset;
			SetPreset(InPreset);
		}
		else
		{
			GlobalReverbPluginPreset = InPreset;
		}
	}

	UResonanceAudioReverbPluginPreset* FResonanceAudioReverb::GetGlobalReverbPluginPreset()
	{
		return GlobalReverbPluginPreset;
	}

	void FResonanceAudioReverb::InitEffectSubmix()
	{
		check(!SubmixEffect.IsValid());

		// initialize the GlobalReverbPluginPreset
		if (GlobalReverbPluginPreset == nullptr)
		{
			// Load the global reverb preset settings:
			const FSoftObjectPath ReverbPluginPresetName = GetDefault<UResonanceAudioSettings>()->GlobalReverbPreset;
			if (ReverbPluginPresetName.IsValid())
			{
				GlobalReverbPluginPreset = LoadObject<UResonanceAudioReverbPluginPreset>(nullptr, *ReverbPluginPresetName.ToString());
			}

			// If loading of the Reverb Plugin Preset asset fails, create a temporary preset.
			if (GlobalReverbPluginPreset == nullptr)
			{
				GlobalReverbPluginPreset = NewObject<UResonanceAudioReverbPluginPreset>();
			}
			
			if (GlobalReverbPluginPreset)
			{
				GlobalReverbPluginPreset->AddToRoot();
			}
		}

		if (GlobalReverbPluginPreset)
		{
			ReverbPluginPreset = GlobalReverbPluginPreset;

			SubmixEffect = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(FSoundEffectSubmixInitData(), *ReverbPluginPreset);
			StaticCastSharedPtr<FResonanceAudioReverbPlugin, FSoundEffectSubmix, ESPMode::ThreadSafe>(SubmixEffect)->SetResonanceAudioReverbPlugin(this);
			SubmixEffect->SetEnabled(true);
		}
		else
		{
			UE_LOGF(LogResonanceAudio, Error, "Failed to create Resonance Reverb global Preset");
			ReverbPluginPreset = nullptr;
		}
	}

	FSoundEffectSubmixPtr FResonanceAudioReverb::GetEffectSubmix()
	{
		if (!SubmixEffect.IsValid())
		{
			InitEffectSubmix();
		}

		return SubmixEffect;
	}

	USoundSubmix* FResonanceAudioReverb::GetSubmix()
	{
		return ReverbSubmixPtr;
	}

	USoundSubmix* FResonanceAudioReverb::LoadSubmix()
	{
		if (ReverbSubmixPtr)
		{
			return ReverbSubmixPtr;
		}

		const UResonanceAudioSettings* Settings = GetDefault<UResonanceAudioSettings>();
		check(Settings);

		if (!IgnoreUserResonanceSubmixCVar)
		{
			ReverbSubmixPtr = Cast<USoundSubmix>(Settings->OutputSubmix.TryLoad());
		}

		if (!ReverbSubmixPtr)
		{
			if (!IgnoreUserResonanceSubmixCVar)
			{
				static const FString DefaultSubmixName = TEXT("Resonance Reverb Submix");
				UE_LOGF(LogResonanceAudio, Error, "Couldn't load Resonance Reverb Submix from object path '%ls' in ResonanceSettings. Creating stub submix '%ls'.",
					*Settings->OutputSubmix.GetAssetPathString(),
					*DefaultSubmixName);
			}

			ReverbSubmixPtr = NewObject<USoundSubmix>();
			// This AddToRoot is redundant since we AddToRoot in the LoadSoundSubmix stack in the AudioMixerDevice.
			// LoadSoundSubmix gets called immediately after this function in FMixerDevice::LoadPluginSoundSubmixes.
			if (!Audio::GRemoveSubmixesFromRoot)
			{
				ReverbSubmixPtr->AddToRoot();
			}
			ReverbSubmixPtr->bMuteWhenBackgrounded = true;

			if (const FAudioDeviceManager* DeviceManager = FAudioDeviceManager::Get())
			{
				if (const FAudioDevice* MainDevice = DeviceManager->GetMainAudioDeviceRaw())
				{
					USoundSubmix* MainSubmixObjectPtr = &MainDevice->GetMainSubmixObject();
					UE_LOGF(LogResonanceAudio, Log, "Setting Resonance's submix parent to Main Submix %ls"
						, *MainSubmixObjectPtr->GetName());

					ReverbSubmixPtr->SetParentSubmix(MainSubmixObjectPtr, false /* bModifyAssets */);
				}
			}

		}

		ReverbSubmixPtr->bAutoDisable = false;

		// SubmixEffect is required to be initialized for ReverbPluginPreset to be set
		if (!SubmixEffect.IsValid())
		{
			InitEffectSubmix();
		}

		if (ReverbPluginPreset)
		{
			bool bFoundPreset = false;
			for (USoundEffectSubmixPreset* Preset : ReverbSubmixPtr->SubmixEffectChain)
			{
				if (UResonanceAudioReverbPluginPreset* PluginPreset = Cast<UResonanceAudioReverbPluginPreset>(Preset))
				{
					bFoundPreset = true;
					break;
				}
			}

			if (!bFoundPreset)
			{
				if (!IgnoreUserResonanceSubmixCVar)
				{
					static const FString DefaultPresetName = TEXT("ResonanceReverbDefault_0");
					UE_LOGF(LogResonanceAudio, Error, "Failed to find Resonance UResonanceAudioReverbPluginPreset on default reverb submix. Creating stub '%ls'.",
						*DefaultPresetName);
				}
				ReverbSubmixPtr->SubmixEffectChain.Add(ReverbPluginPreset);
			}
		}

		return ReverbSubmixPtr;
	}

	void FResonanceAudioReverb::SetPreset(UResonanceAudioReverbPluginPreset* InPreset)
	{
		if (InPreset)
		{
			ReverbPluginPreset = InPreset;
		}
		else
		{
			ReverbPluginPreset = GlobalReverbPluginPreset;
		}
	};

	void FResonanceAudioReverb::UpdateRoomEffects()
	{
		if (ResonanceAudioApi != nullptr)
		{
			if (ReverbPluginPreset != nullptr)
			{
				// Update Room Properties.
				ResonanceAudioApi->EnableRoomEffects(ReverbPluginPreset->Settings.bEnableRoomEffects);
				if (ReverbPluginPreset->Settings.bEnableRoomEffects)
				{
					SetRoomPosition();
					SetRoomRotation();
					SetRoomDimensions();
					SetRoomMaterials();
					SetReflectionScalar();
					SetReverbGain();
					SetReverbTimeModifier();
					SetReverbBrightness();

					ReflectionProperties = vraudio::ComputeReflectionProperties(RoomProperties);
					ReverbProperties = vraudio::ComputeReverbProperties(RoomProperties);

					ResonanceAudioApi->SetReflectionProperties(ReflectionProperties);
					ResonanceAudioApi->SetReverbProperties(ReverbProperties);
				}
			}
			else
			{
				ResonanceAudioApi->EnableRoomEffects(false);
			}
		}
	}

	void FResonanceAudioReverb::SetRoomPosition()
	{
		if (ReverbSettings.RoomPosition != ReverbPluginPreset->Settings.RoomPosition)
		{
			ReverbSettings.RoomPosition = ReverbPluginPreset->Settings.RoomPosition;
			const FVector ConvertedRoomPosition = ConvertToResonanceAudioCoordinates(ReverbSettings.RoomPosition);
			RoomProperties.position[0] = ConvertedRoomPosition.X;
			RoomProperties.position[1] = ConvertedRoomPosition.Y;
			RoomProperties.position[2] = ConvertedRoomPosition.Z;
		}
	}

	void FResonanceAudioReverb::SetRoomRotation()
	{
		if (ReverbSettings.RoomRotation != ReverbPluginPreset->Settings.RoomRotation)
		{
			ReverbSettings.RoomRotation = ReverbPluginPreset->Settings.RoomRotation;
			const FQuat ConvertedRoomRotation = ConvertToResonanceAudioRotation(ReverbSettings.RoomRotation);
			RoomProperties.rotation[0] = ConvertedRoomRotation.X;
			RoomProperties.rotation[1] = ConvertedRoomRotation.Y;
			RoomProperties.rotation[2] = ConvertedRoomRotation.Z;
			RoomProperties.rotation[3] = ConvertedRoomRotation.W;
		}
	}

	void FResonanceAudioReverb::SetRoomDimensions()
	{
		if (ReverbSettings.RoomDimensions != ReverbPluginPreset->Settings.RoomDimensions)
		{
			ReverbSettings.RoomDimensions = ReverbPluginPreset->Settings.RoomDimensions;
			const FVector ConvertedRoomDimensions = ConvertToResonanceAudioCoordinates(ReverbSettings.RoomDimensions);
			RoomProperties.dimensions[0] = FMath::Abs(ConvertedRoomDimensions.X);
			RoomProperties.dimensions[1] = FMath::Abs(ConvertedRoomDimensions.Y);
			RoomProperties.dimensions[2] = FMath::Abs(ConvertedRoomDimensions.Z);
		}
	}

	void FResonanceAudioReverb::SetRoomMaterials()
	{
		if (ReverbSettings.LeftWallMaterial != ReverbPluginPreset->Settings.LeftWallMaterial)
		{
			ReverbSettings.LeftWallMaterial = ReverbPluginPreset->Settings.LeftWallMaterial;
			RoomProperties.material_names[0] = ConvertToResonanceMaterialName(ReverbSettings.LeftWallMaterial);
		}
		if (ReverbSettings.RightWallMaterial != ReverbPluginPreset->Settings.RightWallMaterial)
		{
			ReverbSettings.RightWallMaterial = ReverbPluginPreset->Settings.RightWallMaterial;
			RoomProperties.material_names[1] = ConvertToResonanceMaterialName(ReverbSettings.RightWallMaterial);
		}
		if (ReverbSettings.FloorMaterial != ReverbPluginPreset->Settings.FloorMaterial)
		{
			ReverbSettings.FloorMaterial = ReverbPluginPreset->Settings.FloorMaterial;
			RoomProperties.material_names[2] = ConvertToResonanceMaterialName(ReverbSettings.FloorMaterial);
		}
		if (ReverbSettings.CeilingMaterial != ReverbPluginPreset->Settings.CeilingMaterial)
		{
			ReverbSettings.CeilingMaterial = ReverbPluginPreset->Settings.CeilingMaterial;
			RoomProperties.material_names[3] = ConvertToResonanceMaterialName(ReverbSettings.CeilingMaterial);
		}
		if (ReverbSettings.FrontWallMaterial != ReverbPluginPreset->Settings.FrontWallMaterial)
		{
			ReverbSettings.FrontWallMaterial = ReverbPluginPreset->Settings.FrontWallMaterial;
			RoomProperties.material_names[4] = ConvertToResonanceMaterialName(ReverbSettings.FrontWallMaterial);
		}
		if (ReverbSettings.BackWallMaterial != ReverbPluginPreset->Settings.BackWallMaterial)
		{
			ReverbSettings.BackWallMaterial = ReverbPluginPreset->Settings.BackWallMaterial;
			RoomProperties.material_names[5] = ConvertToResonanceMaterialName(ReverbSettings.BackWallMaterial);
		}
	}

	void FResonanceAudioReverb::SetReflectionScalar()
	{
		RoomProperties.reflection_scalar = ReverbPluginPreset->Settings.ReflectionScalar;
	}

	void FResonanceAudioReverb::SetReverbGain()
	{
		RoomProperties.reverb_gain = ReverbPluginPreset->Settings.ReverbGain;
	}

	void FResonanceAudioReverb::SetReverbTimeModifier()
	{
		RoomProperties.reverb_time = ReverbPluginPreset->Settings.ReverbTimeModifier;
	}

	void FResonanceAudioReverb::SetReverbBrightness()
	{
		RoomProperties.reverb_brightness = ReverbPluginPreset->Settings.ReverbBrightness;
	}
} // namespace ResonanceAudio

  /******************************************************/
  /* Resonance Audio Reverb Plugin (FSoundEffectSubmix) */
  /******************************************************/
FResonanceAudioReverbPlugin::FResonanceAudioReverbPlugin()
	: ResonanceAudioReverb(nullptr)
{
}

void FResonanceAudioReverbPlugin::Init(const FSoundEffectSubmixInitData& InData)
{
}

void FResonanceAudioReverbPlugin::OnPresetChanged()
{
	GET_EFFECT_SETTINGS(ResonanceAudioReverbPlugin);
	if (ResonanceAudioReverb)
	{
		ResonanceAudioReverb->UpdateRoomEffects();
	}
}

uint32 FResonanceAudioReverbPlugin::GetDesiredInputChannelCountOverride() const
{
	return 2;
}

void FResonanceAudioReverbPlugin::OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData)
{
	if (ResonanceAudioReverb)
	{
		ResonanceAudioReverb->ProcessMixedAudio(InData, OutData);
	}
}

void FResonanceAudioReverbPlugin::SetResonanceAudioReverbPlugin(ResonanceAudio::FResonanceAudioReverb* InResonanceAudioReverbPlugin)
{
	ResonanceAudioReverb = InResonanceAudioReverbPlugin;
}

/**************************************************************************************/
/* ResonanceAudioReverbPluginPreset                                                   */
// These methods allow control via blueprints                                         */
/**************************************************************************************/
void UResonanceAudioReverbPluginPreset::SetEnableRoomEffects(bool bInEnableRoomeEffects)
{
	Settings.bEnableRoomEffects = bInEnableRoomeEffects;
	UpdateSettings(Settings);
}

void UResonanceAudioReverbPluginPreset::SetRoomPosition(const FVector& InPosition)
{
	if (Settings.RoomPosition != InPosition)
	{
		Settings.RoomPosition = InPosition;
		UpdateSettings(Settings);
	}
}

void UResonanceAudioReverbPluginPreset::SetRoomRotation(const FQuat& InRotation)
{
	if (Settings.RoomRotation != InRotation)
	{
		Settings.RoomRotation = InRotation;
		UpdateSettings(Settings);
	}
}

void UResonanceAudioReverbPluginPreset::SetRoomDimensions(const FVector& InDimensions)
{
	if (Settings.RoomDimensions != InDimensions)
	{
		Settings.RoomDimensions = InDimensions;
		UpdateSettings(Settings);
	}
}

void UResonanceAudioReverbPluginPreset::SetRoomMaterials(const TArray<ERaMaterialName>& InMaterials)
{
	Settings.LeftWallMaterial = InMaterials[0];
	Settings.RightWallMaterial = InMaterials[1];
	Settings.FloorMaterial = InMaterials[2];
	Settings.CeilingMaterial = InMaterials[3];
	Settings.FrontWallMaterial = InMaterials[4];
	Settings.BackWallMaterial = InMaterials[5];
	UpdateSettings(Settings);
}

void UResonanceAudioReverbPluginPreset::SetReflectionScalar(float InReflectionScalar)
{
	Settings.ReflectionScalar = InReflectionScalar;
	UpdateSettings(Settings);
}

void UResonanceAudioReverbPluginPreset::SetReverbGain(float InReverbGain)
{
	Settings.ReverbGain = InReverbGain;
	UpdateSettings(Settings);
}

void UResonanceAudioReverbPluginPreset::SetReverbTimeModifier(float InReverbTimeModifier)
{
	Settings.ReverbTimeModifier = InReverbTimeModifier;
	UpdateSettings(Settings);
}

void UResonanceAudioReverbPluginPreset::SetReverbBrightness(float InReverbBrightness)
{
	Settings.ReverbBrightness = InReverbBrightness;
	UpdateSettings(Settings);
}

