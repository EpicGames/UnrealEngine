//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "DSP/BufferVectorOperations.h"
#include "IAudioExtensionPlugin.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundEffectPreset.h"
#include "ResonanceAudioEnums.h"
#include "ResonanceAudioCommon.h"
#include "ResonanceAudioReverb.generated.h"

#define UE_API RESONANCEAUDIO_API

class UResonanceAudioReverbPluginPreset;

USTRUCT(BlueprintType)
struct FResonanceAudioReverbPluginSettings
{
	GENERATED_USTRUCT_BODY()

	// Whether Resonance Audio room effects are enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "General")
    bool bEnableRoomEffects;

	// Whether the room transform should be taken from the Audio Volume this preset is attached to.
	// If not used with the Audio Volume, last saved values will be used.
	UPROPERTY(EditAnywhere, Category = "Room Transform")
	bool bGetTransformFromAudioVolume = false;

	// Room position in 3D space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Room Transform", meta = (EditCondition = "!bGetTransformFromAudioVolume"))
	FVector RoomPosition;

	// Room rotation in 3D space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Room Transform", meta = (EditCondition = "!bGetTransformFromAudioVolume"))
	FQuat RoomRotation;

	// Room dimensions in 3D space
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Room Transform", meta = (EditCondition = "!bGetTransformFromAudioVolume"))
	FVector RoomDimensions;

	// Left wall acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName LeftWallMaterial;

	// Right wall acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName RightWallMaterial;

	// Floor acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName FloorMaterial;

	// Ceiling acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName CeilingMaterial;

	// Front wall acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName FrontWallMaterial;

	// Back wall acoustic material
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Acoustic Materials")
	ERaMaterialName BackWallMaterial;

	// Early reflections gain multiplier. Default: 1.0
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Early Reflections", meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float ReflectionScalar;

	// Reverb gain multiplier. Default: 1.0
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reverb", meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float ReverbGain;

	// Reverb time modifier. Default: 1.0
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reverb", meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float ReverbTimeModifier;

	// Reverb brightness modifier. Default: 0.0
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Reverb", meta = (ClampMin = "-1.0", ClampMax = "1.0", UIMin = "-1.0", UIMax = "1.0"))
	float ReverbBrightness;

	FResonanceAudioReverbPluginSettings()
		: bEnableRoomEffects(false)
		, RoomPosition(FVector::ZeroVector)
		, RoomRotation(FQuat::Identity)
		, RoomDimensions(FVector::ZeroVector)
		, LeftWallMaterial(ERaMaterialName::TRANSPARENT)
		, RightWallMaterial(ERaMaterialName::TRANSPARENT)
		, FloorMaterial(ERaMaterialName::TRANSPARENT)
		, CeilingMaterial(ERaMaterialName::TRANSPARENT)
		, FrontWallMaterial(ERaMaterialName::TRANSPARENT)
		, BackWallMaterial(ERaMaterialName::TRANSPARENT)
		, ReflectionScalar(1.0f)
		, ReverbGain(1.0f)
		, ReverbTimeModifier(1.0f)
		, ReverbBrightness(0.0f)
	{
	}
};

namespace ResonanceAudio
{
	class FResonanceAudioModule;
	class FResonanceAudioReverb : public IAudioReverb
	{
	public:

		FResonanceAudioReverb();
		~FResonanceAudioReverb();

		virtual void Initialize(const FAudioPluginInitializationParams InitializationParams) override;
		virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, UReverbPluginSourceSettingsBase* InSettings) override;
		virtual void OnReleaseSource(const uint32 SourceId) override;
		virtual FSoundEffectSubmixPtr GetEffectSubmix() override;
		virtual USoundSubmix* GetSubmix() override;
		virtual USoundSubmix* LoadSubmix() override;
		virtual void ProcessSourceAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData) override;

		void SetResonanceAudioApi(vraudio::ResonanceAudioApi* InResonanceAudioApi) { ResonanceAudioApi = InResonanceAudioApi; };
		void SetPreset(UResonanceAudioReverbPluginPreset* InPreset);
		void ProcessMixedAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData);
		
		void SetGlobalReverbPluginPreset(UResonanceAudioReverbPluginPreset* InPreset);
		UResonanceAudioReverbPluginPreset* GetGlobalReverbPluginPreset();

		// Sets & updates Room Effect params from current |ReverbPluginPreset|.
		void UpdateRoomEffects();
		void SetRoomPosition();
		void SetRoomRotation();
		void SetRoomDimensions();
		void SetRoomMaterials();
		void SetReflectionScalar();
		void SetReverbGain();
		void SetReverbTimeModifier();
		void SetReverbBrightness();

	private:

		void InitEffectSubmix();

		FResonanceAudioReverbPluginSettings ReverbSettings;
		RaRoomProperties RoomProperties;
		RaReverbProperties ReverbProperties;
		RaReflectionProperties ReflectionProperties;


		vraudio::ResonanceAudioApi* ResonanceAudioApi;
		FResonanceAudioModule* ResonanceAudioModule;
		UResonanceAudioReverbPluginPreset* ReverbPluginPreset;
		Audio::AlignedFloatBuffer TemporaryStereoBuffer;

		FSoundEffectSubmixPtr SubmixEffect;

		// Reverb submix to be used only when Resonance fails to load the submix asset.
		USoundSubmix* ReverbSubmixPtr;
	};

} // namespace ResonanceAudio

class FResonanceAudioReverbPlugin : public FSoundEffectSubmix
{
public:
	FResonanceAudioReverbPlugin();

	virtual void Init(const FSoundEffectSubmixInitData& InData) override;
	virtual uint32 GetDesiredInputChannelCountOverride() const override;
	virtual void OnProcessAudio(const FSoundEffectSubmixInputData& InData, FSoundEffectSubmixOutputData& OutData) override;
	virtual void OnPresetChanged() override;

	void SetResonanceAudioReverbPlugin(ResonanceAudio::FResonanceAudioReverb* InResonanceAudioReverbPlugin);

private:
	ResonanceAudio::FResonanceAudioReverb* ResonanceAudioReverb;
};

UCLASS(MinimalAPI, BlueprintType)
class UResonanceAudioReverbPluginPreset : public USoundEffectSubmixPreset
{
	GENERATED_BODY()

public:

	EFFECT_PRESET_METHODS(ResonanceAudioReverbPlugin)

	UPROPERTY(EditAnywhere, Category = "Resonance Audio Room Effect Settings", Meta = (ShowOnlyInnerProperties))
	FResonanceAudioReverbPluginSettings Settings;

	// Returns true, if the transform should be obtained from the Audio Volume, instead of set manually.
	bool UseAudioVolumeTransform() { return Settings.bGetTransformFromAudioVolume; };

	// Enables/disables Resonance Audio room effects
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetEnableRoomEffects(bool bInEnableRoomEffects);

	// Sets room position in 3D space
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetRoomPosition(const FVector& InPosition);

	// Sets room rotation in 3D space
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetRoomRotation(const FQuat& InRotation);

	// Sets room dimensions in 3D space
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetRoomDimensions(const FVector& InDimensions);

	// Sets Resonance Audio room's acoustic materials
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetRoomMaterials(const TArray<ERaMaterialName>& InMaterials);

	// Sets early reflections gain multiplier
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetReflectionScalar(float InReflectionScalar);

	// Sets reverb gain multiplier
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetReverbGain(float InReverbGain);

	// Adjusts the reverberation time across all frequency bands by multiplying the computed values by this factor.
	// Has no effect when set to 1.0
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetReverbTimeModifier(float InReverbTimeModifier);

	// Increases high frequency reverberation times when positive, decreases when negative.
	// Has no effect when set to 0.0
	UFUNCTION(BlueprintCallable, Category = "Resonance Audio Room Effect Settings")
	UE_API void SetReverbBrightness(float InReverbBrightness);
};

#undef UE_API
