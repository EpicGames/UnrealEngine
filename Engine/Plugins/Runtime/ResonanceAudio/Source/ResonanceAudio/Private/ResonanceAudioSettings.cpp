//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioSettings)

UResonanceAudioSettings::UResonanceAudioSettings()
	: QualityMode(ERaQualityMode::BINAURAL_HIGH)
	, GlobalReverbPreset(nullptr)
{
	OutputSubmix = FString(TEXT("/ResonanceAudio/ResonanceSubmixDefault.ResonanceSubmixDefault"));
}

