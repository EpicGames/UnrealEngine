//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once
#include "ResonanceAudioEnums.h"
#include "ResonanceAudioSettings.generated.h"

#define UE_API RESONANCEAUDIO_API


UCLASS(MinimalAPI, config = Engine, defaultconfig)
class UResonanceAudioSettings : public UObject
{
	GENERATED_BODY()

public:

	UE_API UResonanceAudioSettings();

	// Reference to submix where reverb plugin audio is routed.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "Reverb", meta = (AllowedClasses = "/Script/Engine.SoundSubmix"))
	FSoftObjectPath OutputSubmix;

	// Global Quality mode to use for directional sound sources.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = General)
	ERaQualityMode QualityMode;

	// Default settings for global reverb: This is overridden when a player enters Audio Volumes.
	UPROPERTY(GlobalConfig, EditAnywhere, Category = General, meta = (AllowedClasses = "/Script/ResonanceAudio.ResonanceAudioReverbPluginPreset"))
	FSoftObjectPath GlobalReverbPreset;

	// Default settings for global source settings
	UPROPERTY(GlobalConfig, EditAnywhere, Category = General, meta = (AllowedClasses = "/Script/ResonanceAudio.ResonanceAudioSpatializationSourceSettings"))
	FSoftObjectPath GlobalSourcePreset;
};

#undef UE_API
