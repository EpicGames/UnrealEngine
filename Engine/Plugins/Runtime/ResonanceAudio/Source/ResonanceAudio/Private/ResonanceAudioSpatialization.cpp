#include "ResonanceAudioSpatialization.h"

#include "ResonanceAudioConstants.h"
#include "ResonanceAudioModule.h"
#include "ResonanceAudioSettings.h"


static int32 ResonanceQualityOverrideCVar = 0;
FAutoConsoleVariableRef CVarResonanceQualityOverride(
	TEXT("au.resonance.quality"),
	ResonanceQualityOverrideCVar,
	TEXT("Override the quality of resonance sound sources. Will not increase quality levels. The quality used will be min of the quality in the resonance source settings and this override.\n")
	TEXT("0: Quality is not overridden, 1: Stereo Panning, 2: Low Quality, 3: Medium Quality, 4: High Quality"),
	ECVF_Default);


static int32 ExtraResonanceLoggingCVar = 0;
FAutoConsoleVariableRef CVarExtraResonanceLogging(
	TEXT("au.ExtraResonanceLogging"),
	ExtraResonanceLoggingCVar,
	TEXT("If non-zero, will log extra information about the state of Resonance HRTF processing.\n")
	TEXT("0: Disable, >0: Enable"),
	ECVF_Default);

namespace ResonanceAudio {

	/************************************************************/
	/* Resonance Audio Spatialization                           */
	/************************************************************/
	FResonanceAudioSpatialization::FResonanceAudioSpatialization()
		: ResonanceAudioApi(nullptr)
		, ResonanceAudioModule(nullptr)
	{
	}

	FResonanceAudioSpatialization::~FResonanceAudioSpatialization()
	{
	}

	void FResonanceAudioSpatialization::Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
		ResonanceAudioModule = &FModuleManager::GetModuleChecked<FResonanceAudioModule>("ResonanceAudio");

		// Pre-create all Resonance Audio Sources as there is a guaranteed finite number.
		BinauralSources.AddDefaulted(InitializationParams.NumSources);
		for (auto& BinauralSource : BinauralSources)
		{
			BinauralSource.Id = RA_INVALID_SOURCE_ID;
		}

		// Initialize spatialization settings array for each sound source.
		SpatializationSettings.Init(nullptr, InitializationParams.NumSources);

		bIsInitialized = true;

		UE_LOGF(LogResonanceAudio, Log, "Google Resonance Audio Spatialization is initialized.");
	}

	bool FResonanceAudioSpatialization::IsSpatializationEffectInitialized() const
	{
		return bIsInitialized;
	}

	void FResonanceAudioSpatialization::OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings)
	{
		if (ResonanceAudioApi == nullptr)
		{
			return;
		}

		// Create a sound source and register it with Resonance Audio API.
		auto& BinauralSource = BinauralSources[SourceId];
		if (BinauralSource.Id != RA_INVALID_SOURCE_ID)
		{
			ResonanceAudioApi->DestroySource(BinauralSource.Id);
			BinauralSource.Id = RA_INVALID_SOURCE_ID;
		}

		// If we weren't passed in any settings, we want to get the settings from the global project settings
		if (!InSettings)
		{
			SpatializationSettings[SourceId] = FResonanceAudioModule::GetGlobalSpatializationSourceSettings();
		}
		else
		{
			SpatializationSettings[SourceId] = static_cast<UResonanceAudioSpatializationSourceSettings*>(InSettings);
		}


		if (SpatializationSettings[SourceId] == nullptr)
		{
			UE_LOGF(LogResonanceAudio, Error, "No Spatialization Settings Preset added to the sound source! Please create a new preset or add an existing one.");
			return;
		}

		vraudio::RenderingMode ResonanceRenderingMode = vraudio::RenderingMode::kStereoPanning;

		if (SpatializationSettings[SourceId]->SpatializationMethod != ERaSpatializationMethod::STEREO_PANNING)
		{
			switch (GetDefault<UResonanceAudioSettings>()->QualityMode)
			{
			case ERaQualityMode::STEREO_PANNING:
				ResonanceRenderingMode = vraudio::RenderingMode::kStereoPanning;
				UE_LOGF(LogResonanceAudio, Verbose, "ResonanceAudioSpatializer::OnInitSource: STEREO_PANNING mode chosen.");
				break;

			case ERaQualityMode::BINAURAL_LOW:
				ResonanceRenderingMode = vraudio::RenderingMode::kBinauralLowQuality;
				UE_LOGF(LogResonanceAudio, Verbose, "ResonanceAudioSpatializer::OnInitSource: BINAURAL_LOW mode chosen.");
				break;

			case ERaQualityMode::BINAURAL_MEDIUM:
				ResonanceRenderingMode = vraudio::RenderingMode::kBinauralMediumQuality;
				UE_LOGF(LogResonanceAudio, Verbose, "ResonanceAudioSpatializer::OnInitSource: BINAURAL_MEDIUM mode chosen.");
				break;

			case ERaQualityMode::BINAURAL_HIGH:
				ResonanceRenderingMode = vraudio::RenderingMode::kBinauralHighQuality;
				UE_LOGF(LogResonanceAudio, Verbose, "ResonanceAudioSpatializer::OnInitSource: BINAURAL_HIGH mode chosen.");
				break;

			default:
				UE_LOGF(LogResonanceAudio, Error, "ResonanceAudioSpatializer::OnInitSource: Unknown quality mode!");
				break;
			}
		}

		// Only let cvar values between 0 and vraudio::RenderingMode::kRoomEffectsOnly + 1 affect the rendering mode
		// Note that 0 is treated as non-overridden, but we want to map to the enumeration values which are 
		if (ResonanceQualityOverrideCVar > 0 && ResonanceQualityOverrideCVar < ((int32)(vraudio::RenderingMode::kRoomEffectsOnly) + 1))
		{
			ResonanceRenderingMode = (vraudio::RenderingMode)FMath::Min((int32)ResonanceRenderingMode, ResonanceQualityOverrideCVar - 1);
		}

		BinauralSource.Id = ResonanceAudioApi->CreateSoundObjectSource(ResonanceRenderingMode);

		// Set initial sound source directivity.
		BinauralSource.Pattern = SpatializationSettings[SourceId]->Pattern;
		BinauralSource.Sharpness = SpatializationSettings[SourceId]->Sharpness;
		ResonanceAudioApi->SetSoundObjectDirectivity(BinauralSource.Id, BinauralSource.Pattern, BinauralSource.Sharpness);

		// Set initial sound source spread (width).
		BinauralSource.Spread = SpatializationSettings[SourceId]->Spread;
		ResonanceAudioApi->SetSoundObjectSpread(BinauralSource.Id, BinauralSource.Spread);

		// Set distance attenuation model.
		switch (SpatializationSettings[SourceId]->Rolloff)
		{
		case ERaDistanceRolloffModel::LOGARITHMIC:
			ResonanceAudioApi->SetSourceDistanceModel(BinauralSource.Id, vraudio::DistanceRolloffModel::kLogarithmic, SCALE_FACTOR * SpatializationSettings[SourceId]->MinDistance, SCALE_FACTOR * SpatializationSettings[SourceId]->MaxDistance);
			break;
		case ERaDistanceRolloffModel::LINEAR:
			ResonanceAudioApi->SetSourceDistanceModel(BinauralSource.Id, vraudio::DistanceRolloffModel::kLinear, SCALE_FACTOR * SpatializationSettings[SourceId]->MinDistance, SCALE_FACTOR * SpatializationSettings[SourceId]->MaxDistance);
			break;
		case ERaDistanceRolloffModel::NONE:
			ResonanceAudioApi->SetSourceDistanceModel(BinauralSource.Id, vraudio::DistanceRolloffModel::kNone, SCALE_FACTOR * SpatializationSettings[SourceId]->MinDistance, SCALE_FACTOR * SpatializationSettings[SourceId]->MaxDistance);
			break;
		default:
			UE_LOGF(LogResonanceAudio, Error, "ResonanceAudioSpatializer::OnInitSource: Undefined distance roll-off model!");
			break;
		}


		if (ExtraResonanceLoggingCVar)
		{
			UE_LOGF(LogResonanceAudio, Warning, "Source initialized (Our source ID: %i) (Binaural SourceId: %i).", SourceId, BinauralSource.Id);
		}
	}

	void FResonanceAudioSpatialization::OnReleaseSource(const uint32 SourceId)
	{
		if (ResonanceAudioApi == nullptr)
		{
			return;
		}

		auto& BinauralSource = BinauralSources[SourceId];
		if (BinauralSource.Id != RA_INVALID_SOURCE_ID)
		{
			ResonanceAudioApi->DestroySource(BinauralSource.Id);
			BinauralSource.Id = RA_INVALID_SOURCE_ID;
		}

		if (ExtraResonanceLoggingCVar)
		{
			UE_LOGF(LogResonanceAudio, Warning, "Destroying Source (Our Source ID: %i)(Binaural Source ID: %i)", SourceId, BinauralSource.Id);
		}
	}

	void FResonanceAudioSpatialization::ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
		if (ResonanceAudioApi == nullptr)
		{
			return;
		}
		else
		{
			auto& BinauralSource = BinauralSources[InputData.SourceId];
			auto& Position = InputData.SpatializationParams->EmitterWorldPosition;
			auto& Rotation = InputData.SpatializationParams->EmitterWorldRotation;

			const FVector ConvertedPosition = ConvertToResonanceAudioCoordinates(Position);
			const FQuat ConvertedRotation = ConvertToResonanceAudioRotation(Rotation);
			ResonanceAudioApi->SetSourcePosition(BinauralSource.Id, ConvertedPosition.X, ConvertedPosition.Y, ConvertedPosition.Z);
			ResonanceAudioApi->SetSourceRotation(BinauralSource.Id, ConvertedRotation.X, ConvertedRotation.Y, ConvertedRotation.Z, ConvertedRotation.W);

			// Check whether spatialization settings have been updated.
			if (SpatializationSettings[InputData.SourceId] != nullptr)
			{
				// Set sound source directivity.
				if (DirectivityChanged(InputData.SourceId))
				{
					ResonanceAudioApi->SetSoundObjectDirectivity(BinauralSource.Id, SpatializationSettings[InputData.SourceId]->Pattern, SpatializationSettings[InputData.SourceId]->Sharpness);
				}

				// Set sound source spread (width).
				if (SpreadChanged(InputData.SourceId))
				{
					ResonanceAudioApi->SetSoundObjectSpread(BinauralSource.Id, SpatializationSettings[InputData.SourceId]->Spread);
				}
			}

			// Add source buffer to process.
			ResonanceAudioApi->SetInterleavedBuffer(BinauralSource.Id, InputData.AudioBuffer->GetData(), InputData.NumChannels, InputData.AudioBuffer->Num() / InputData.NumChannels);

			// optional heartbeat to make sure Resonance is processing audio
			static const int32 NumBuffersBetweenLogs = 500;
			static int32 ResonanceAudioBuffersProcessecdCounter = 0;

			if (ExtraResonanceLoggingCVar && (ResonanceAudioBuffersProcessecdCounter++ % NumBuffersBetweenLogs) == 0)
			{
				UE_LOGF(LogResonanceAudio, Warning, "FResonanceAudioSpatialization::ProcessAudio() has been called %i times", NumBuffersBetweenLogs);
			}
		}
	}

	bool FResonanceAudioSpatialization::DirectivityChanged(const uint32 SourceId)
	{
		auto& BinauralSource = BinauralSources[SourceId];
		if (FMath::IsNearlyEqual(BinauralSource.Pattern, SpatializationSettings[SourceId]->Pattern) && FMath::IsNearlyEqual(BinauralSource.Sharpness, SpatializationSettings[SourceId]->Sharpness))
		{
			return false;
		}
		BinauralSource.Pattern = SpatializationSettings[SourceId]->Pattern;
		BinauralSource.Sharpness = SpatializationSettings[SourceId]->Sharpness;
		return true;
	}

	bool FResonanceAudioSpatialization::SpreadChanged(const uint32 SourceId)
	{
		auto& BinauralSource = BinauralSources[SourceId];
		if (FMath::IsNearlyEqual(BinauralSource.Spread, SpatializationSettings[SourceId]->Spread))
		{
			return false;
		}
		BinauralSource.Spread = SpatializationSettings[SourceId]->Spread;
		return true;
	}

}  // namespace ResonanceAudio
