//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "IAudioExtensionPlugin.h"
#include "ResonanceAudioCommon.h"
#include "ResonanceAudioSpatializationSourceSettings.h"

namespace ResonanceAudio
{
	struct FBinauralSource
	{
		RaSourceId Id;
		float Pattern;
		float Sharpness;
		float Spread;

		FBinauralSource()
			: Id(RA_INVALID_SOURCE_ID)
			, Pattern(0.0f)
			, Sharpness(1.0f)
			, Spread(0.0f)
		{};
	};

	class FResonanceAudioSpatialization : public IAudioSpatialization
	{
	public:
		FResonanceAudioSpatialization();
		~FResonanceAudioSpatialization();

		virtual void Initialize(const FAudioPluginInitializationParams InitializationParams) override;
		virtual bool IsSpatializationEffectInitialized() const override;
		virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings) override;
		virtual void OnReleaseSource(const uint32 SourceId) override;
		virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData) override;

		void SetResonanceAudioApi(vraudio::ResonanceAudioApi* InResonanceAudioApi) { ResonanceAudioApi = InResonanceAudioApi; };

	private:
		bool DirectivityChanged(const uint32 SourceId);
		bool SpreadChanged(const uint32 SourceId);

		bool bIsInitialized;
		TArray<FBinauralSource> BinauralSources;
		vraudio::ResonanceAudioApi* ResonanceAudioApi;
		class FResonanceAudioModule* ResonanceAudioModule;
		TArray<UResonanceAudioSpatializationSourceSettings*> SpatializationSettings;
	};

}  // namespace ResonanceAudio
