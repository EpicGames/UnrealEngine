//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioSpatializationSourceSettings.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioSpatializationSourceSettings)

#if WITH_EDITOR
#include "Components/AudioComponent.h"
#include "Editor.h"
#include "Engine/LevelStreaming.h"
#include "LevelEditorViewport.h"
#include "ResonanceAudioDirectivityVisualizer.h"
#include "ResonanceAudioSettings.h"
#endif // WITH_EDITOR



UResonanceAudioSpatializationSourceSettings::UResonanceAudioSpatializationSourceSettings()
	: SpatializationMethod(ERaSpatializationMethod::HRTF)
	, Pattern(0.0f)
	, Sharpness(1.0f)
	, bToggleVisualization(false)
	, Scale(1.0f)
	, Spread(0.0f)
	, Rolloff(ERaDistanceRolloffModel::NONE)
	, MinDistance(0.0f)
	, MaxDistance(50000.0f)
{
}



#if WITH_EDITOR
bool UResonanceAudioSpatializationSourceSettings::DoesAudioComponentReferenceThis(UAudioComponent* InAudioComponent)
{
	const FSoundAttenuationSettings* ComponentSettings = InAudioComponent->GetAttenuationSettingsToApply();
	if (ComponentSettings != nullptr)
	{
		return ComponentSettings->PluginSettings.SpatializationPluginSettingsArray.Contains(this);
	}
	else
	{
		return false;
	}
}

bool UResonanceAudioSpatializationSourceSettings::CanEditChange(const FProperty* InProperty) const
{
	const bool ParentVal = Super::CanEditChange(InProperty);

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, SpatializationMethod)))
	{
		return ParentVal && GetDefault<UResonanceAudioSettings>()->QualityMode != ERaQualityMode::STEREO_PANNING;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, Scale)))
	{
		return ParentVal && bToggleVisualization == true;
	}

	// TODO: Remove once spread (width) control is implemented for stereo sound sources.
	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, Spread)))
	{
		return ParentVal && GetDefault<UResonanceAudioSettings>()->QualityMode != ERaQualityMode::STEREO_PANNING && SpatializationMethod == ERaSpatializationMethod::HRTF;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, MinDistance)))
	{
		return ParentVal && Rolloff != ERaDistanceRolloffModel::NONE;
	}

	if ((InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, MaxDistance)))
	{
		return ParentVal && Rolloff != ERaDistanceRolloffModel::NONE;
	}

	return ParentVal;
}

void UResonanceAudioSpatializationSourceSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, bToggleVisualization)))
	{
		if (bToggleVisualization)
		{
			UWorld* const World = GEditor->GetEditorWorldContext().World();
			for (TObjectIterator<UAudioComponent> Itr; Itr; ++Itr)
			{
				UAudioComponent* AudioComponent = *Itr;
				if (IsValid(AudioComponent) && DoesAudioComponentReferenceThis(AudioComponent))
				{
					AActor* CurrentActor = AudioComponent->GetOwner();
					if (CurrentActor != nullptr)
					{
						AResonanceAudioDirectivityVisualizer* DirectivityVisualizer = Cast<AResonanceAudioDirectivityVisualizer>(GEditor->AddActor(World->GetCurrentLevel(), AResonanceAudioDirectivityVisualizer::StaticClass(), CurrentActor->GetTransform()));
						// Attach the visualizer to the owning object so that it can be moved with it in the editor.
						DirectivityVisualizer->AttachToActor(CurrentActor, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
						DirectivityVisualizer->SetSettings(this);
						DirectivityVisualizer->DrawPattern();
					}
				}
			}
		}
		else
		{
			for (TObjectIterator<AResonanceAudioDirectivityVisualizer> Itr; Itr; ++Itr)
			{
				if ((*Itr)->GetSettings() == this)
				{
					(*Itr)->Destroy();
				}
			}
		}
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, Pattern) || PropertyName == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, Sharpness) || PropertyName == GET_MEMBER_NAME_CHECKED(UResonanceAudioSpatializationSourceSettings, Scale))
	{

		for (TObjectIterator<AResonanceAudioDirectivityVisualizer> Itr; Itr; ++Itr)
		{
			if ((*Itr)->GetSettings() == this)
			{
				(*Itr)->SetSettings(this);
				(*Itr)->DrawPattern();
			}
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // WITH_EDITOR

void UResonanceAudioSpatializationSourceSettings::SetSoundSourceDirectivity(float InPattern, float InSharpness)
{
	Pattern = InPattern;
	Sharpness = InSharpness;
}

void UResonanceAudioSpatializationSourceSettings::SetSoundSourceSpread(float InSpread)
{
	Spread = InSpread;
}

