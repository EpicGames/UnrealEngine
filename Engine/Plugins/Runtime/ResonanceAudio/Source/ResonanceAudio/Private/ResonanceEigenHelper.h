// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// This file provides a way to include Eigen/Core without producing static analysis warnings.

#include "HAL/Platform.h"

#if defined(__clang__)
_Pragma("clang diagnostic push") \
_Pragma("clang diagnostic ignored \"-Wshadow\"")
_Pragma("clang diagnostic ignored \"-Wunreachable-code-aggressive\"")
#elif defined(_MSC_VER)
#pragma warning(push)
#pragma warning(disable:6294) /* Ill-defined for-loop:  initial condition does not satisfy test.  Loop body not executed. */
#pragma warning(disable:6326) /* Potential comparison of a constant with another constant. */
#pragma warning(disable:4456) /* declaration of 'LocalVariable' hides previous local declaration */ 
#pragma warning(disable:4457) /* declaration of 'LocalVariable' hides function parameter */ 
#pragma warning(disable:4458) /* declaration of 'LocalVariable' hides class member */ 
#pragma warning(disable:4459) /* declaration of 'LocalVariable' hides global declaration */ 
#pragma warning(disable:6244) /* local declaration of <variable> hides previous declaration at <line> of <file> */
#pragma warning(disable:4702) /* unreachable code */
#endif

PRAGMA_DEFAULT_VISIBILITY_START
PRAGMA_DISABLE_DEPRECATION_WARNINGS

#include "Eigen/Core"
#include "Eigen/Dense"

PRAGMA_ENABLE_DEPRECATION_WARNINGS
PRAGMA_DEFAULT_VISIBILITY_END

#if defined(__clang__)
_Pragma("clang diagnostic pop")
#elif defined(_MSC_VER)
#pragma warning(pop)
#endif
