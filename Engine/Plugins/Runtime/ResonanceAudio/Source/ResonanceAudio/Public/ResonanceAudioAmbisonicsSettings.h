//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "ISoundfieldFormat.h"
#include "Templates/UniquePtr.h"

#include "ResonanceAudioAmbisonicsSettings.generated.h"

#define UE_API RESONANCEAUDIO_API

UENUM()
enum class EResonanceRenderMode : uint8
{
	// Stereo panning, i.e., this disables HRTF-based rendering.
	StereoPanning,
	// HRTF-based rendering using First Order Ambisonics, over a virtual array of
	// 8 loudspeakers arranged in a cube configuration around the listener's head.
	BinauralLowQuality,
	// HRTF-based rendering using Second Order Ambisonics, over a virtual array of
	// 12 loudspeakers arranged in a dodecahedral configuration (using faces of
	// the dodecahedron).
	BinauralMediumQuality,
	// HRTF-based rendering using Third Order Ambisonics, over a virtual array of
	// 26 loudspeakers arranged in a Lebedev grid: https://goo.gl/DX1wh3.
	BinauralHighQuality,
	// Room effects only rendering. This disables HRTF-based rendering and direct
	// (dry) output of a sound object. Note that this rendering mode should *not*
	// be used for general-purpose sound object spatialization, as it will only
	// render the corresponding room effects of given sound objects without the
	// direct spatialization.
	RoomEffectsOnly,
};

UCLASS(MinimalAPI)
class UResonanceAudioSoundfieldSettings : public USoundfieldEncodingSettingsBase
{
	GENERATED_BODY()

public:
	//Which order of ambisonics to use for this submix.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Ambisonics)
	EResonanceRenderMode RenderMode = EResonanceRenderMode::BinauralHighQuality;

	UE_API virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> GetProxy() const override;
};

#undef UE_API
