//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "ResonanceAudioBlueprintFunctionLibrary.generated.h"

#define UE_API RESONANCEAUDIO_API

class UResonanceAudioReverbPluginPreset;

UCLASS(MinimalAPI)
class UResonanceAudioBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// This function overrides the Global Reverb Preset for Resonance Audio
	UFUNCTION(BlueprintCallable, Category = "ResonanceAudio|GlobalReverbPreset")
	static UE_API void SetGlobalReverbPreset(UResonanceAudioReverbPluginPreset* InPreset);

	// This function retrieves the Global Reverb Preset for Resonance Audio
	UFUNCTION(BlueprintCallable, Category = "ResonanceAudio|GlobalReverbPreset")
	static UE_API UResonanceAudioReverbPluginPreset* GetGlobalReverbPreset();
};

#undef UE_API
