//
// Copyright Google Inc. 2017. All rights reserved.
//

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "ResonanceAudioEnums.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogResonanceAudio, Display, All);

UENUM(BlueprintType)
enum class ERaQualityMode : uint8
{
	// Stereo panning.
	STEREO_PANNING						UMETA(DisplayName = "Stereo Panning"),
	// Binaural Low (First Order Ambisonics).
	BINAURAL_LOW						UMETA(DisplayName = "Binaural Low Quality"),
	// Binaural Medium (Second Order Ambisonics).
	BINAURAL_MEDIUM						UMETA(DisplayName = "Binaural Medium Quality"),
	// Binaural High (Third Order Ambisonics = Default).
	BINAURAL_HIGH						UMETA(DisplayName = "Binaural High Quality")
};

UENUM(BlueprintType)
enum class ERaSpatializationMethod : uint8
{
	// Stereo panning.
	STEREO_PANNING						UMETA(DisplayName = "Stereo Panning"),
	// Binaural rendering via HRTF.
	HRTF								UMETA(DisplayName = "HRTF")
};

UENUM(BlueprintType)
enum class ERaDistanceRolloffModel : uint8
{
	// Logarithmic distance attenuation model (default).
	LOGARITHMIC							UMETA(DisplayName = "Logarithmic"),
	// Linear distance attenuation model.
	LINEAR								UMETA(DisplayName = "Linear"),
	// Use Unreal Engine attenuation settings.
	NONE								UMETA(DisplayName = "None")
};

UENUM(BlueprintType)
enum class ERaMaterialName : uint8
{
	// Full acoustic energy absorption.
	TRANSPARENT							UMETA(DisplayName = "Transparent"),
	ACOUSTIC_CEILING_TILES				UMETA(DisplayName = "Acoustic Ceiling Tiles"),
	BRICK_BARE							UMETA(DisplayName = "Brick Bare"),
	BRICK_PAINTED						UMETA(DisplayName = "Brick Painted"),
	CONCRETE_BLOCK_COARSE				UMETA(DisplayName = "Concrete Block Coarse"),
	CONCRETE_BLOCK_PAINTED				UMETA(DisplayName = "Concrete Block Painted"),
	CURTAIN_HEAVY						UMETA(DisplayName = "Curtain Heavy"),
	FIBER_GLASS_INSULATION				UMETA(DisplayName = "Fiber Glass Insulation"),
	GLASS_THIN							UMETA(DisplayName = "Glass Thin"),
	GLASS_THICK							UMETA(DisplayName = "Glass Thick"),
	GRASS								UMETA(DisplayName = "Grass"),
	LINOLEUM_ON_CONCRETE				UMETA(DisplayName = "Linoleum On Concrete"),
	MARBLE								UMETA(DisplayName = "Marble"),
	METAL								UMETA(DisplayName = "Metal"),
	PARQUET_ONCONCRETE					UMETA(DisplayName = "Parquet On Concrete"),
	PLASTER_ROUGH						UMETA(DisplayName = "Plaster Rough"),
	PLASTER_SMOOTH						UMETA(DisplayName = "Plaster Smooth"),
	PLYWOOD_PANEL						UMETA(DisplayName = "Plywood Panel"),
	POLISHED_CONCRETE_OR_TILE			UMETA(DisplayName = "Polished Concrete Or Tile"),
	SHEETROCK							UMETA(DisplayName = "Sheetrock"),
	WATER_OR_ICE_SURFACE				UMETA(DisplayName = "Water Or Ice Surface"),
	WOOD_CEILING						UMETA(DisplayName = "Wood Ceiling"),
	WOOD_PANEL							UMETA(DisplayName = "Wood Panel"),
	// Uniform acoustic energy absorption across all frequency bands.
	UNIFORM								UMETA(DisplayName = "Uniform")
};

