//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "IAudioExtensionPlugin.h"
#include "ResonanceAudioEnums.h"
#include "ResonanceAudioSpatializationSourceSettings.generated.h"

#define UE_API RESONANCEAUDIO_API

UCLASS(MinimalAPI)
class UResonanceAudioSpatializationSourceSettings : public USpatializationPluginSourceSettingsBase
{
	GENERATED_BODY()

public:
	UE_API UResonanceAudioSpatializationSourceSettings();

	

#if WITH_EDITOR
	// See if Audio Component references this settings instance:
	UE_API bool DoesAudioComponentReferenceThis(class UAudioComponent* InAudioComponent);

	UE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	// Spatialization method to use for sound object(s). NOTE: Has no effect if 'Stereo Panning' global quality mode is selected for the project
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings")
	ERaSpatializationMethod SpatializationMethod;

	// Directivity pattern: 0.0 (omnidirectional), 0.5 (cardioid), 1.0 (figure-of-8)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Directivity", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Pattern;

	// Sharpness of the directivity pattern. Higher values result in a narrower sound emission beam
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Directivity", meta = (ClampMin = "1.0", ClampMax = "100.0", UIMin = "1.0", UIMax = "100.0"))
	float Sharpness;

	// Whether to visualize directivity pattern in the viewport.
	UPROPERTY(EditAnywhere, Category = "SpatializationSettings|Directivity")
	bool bToggleVisualization;

	// Scale (for directivity pattern visualization only).
	UPROPERTY(EditAnywhere, Category = "SpatializationSettings|Directivity", meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float Scale;

	// Spread (width) of the sound source (in degrees). Note: spread control is not available if 'Stereo Panning' global quality mode is enabled for the project and / or 'Stereo Panning' spatialization mode is enabled for the sound source
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Source spread (width)", meta = (ClampMin = "0.0", ClampMax = "180.0", UIMin = "0.0", UIMax = "180.0"))
	float Spread;

	// Roll-off model to use for sound source distance attenuation. Select 'None' (default) to use Unreal attenuation settings
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Distance attenuation")
	ERaDistanceRolloffModel Rolloff;

	// Minimum distance to apply the chosen attenuation method (default = 100.0 cm)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Distance attenuation", meta = (ClampMin = "0.0", ClampMax = "1000000.0", UIMin = "0.0", UIMax = "1000000.0"))
	float MinDistance;

	// Maximum distance to apply the chosen attenuation method ((default = 50000.0 cm)
	UPROPERTY(GlobalConfig, EditAnywhere, Category = "SpatializationSettings|Distance attenuation", meta = (ClampMin = "0.0", ClampMax = "1000000.0", UIMin = "0.0", UIMax = "1000000.0"))
	float MaxDistance;

	// Sets the sound source directivity, applies, and updates
	UFUNCTION(BlueprintCallable, Category = "ResonanceAudio|SoundSourceSpatializationSettings")
	UE_API void SetSoundSourceDirectivity(float InPattern, float InSharpness);

	// Sets the sound source spread (width), applies, and updates
	UFUNCTION(BlueprintCallable, Category = "ResonanceAudio|SoundSourceSpatializationSettings")
	UE_API void SetSoundSourceSpread(float InSpread);
};

#undef UE_API
