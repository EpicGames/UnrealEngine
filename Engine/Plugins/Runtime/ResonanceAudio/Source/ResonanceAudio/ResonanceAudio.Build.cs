//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

using UnrealBuildTool;
using System.IO;

public class ResonanceAudio : ModuleRules
{
	protected virtual bool bSupportsProceduralMesh { get { return true; } }

	public ResonanceAudio(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.KeepAsIsForNow;

		PrivatePCHHeaderFile = "Private/ResonanceAudioPrivatePCH.h";

		string ResonanceAudioPath = ModuleDirectory + "/Private/ResonanceAudioLibrary";
        string ResonanceAudioLibraryPath = ModuleDirectory + "/Private/ResonanceAudioLibrary/resonance_audio";
        string PFFTPath = ModuleDirectory + "/Private/ResonanceAudioLibrary/third_party/pfft";

        PublicIncludePaths.AddRange(
            new string[] {
            }
        );

        PrivateIncludePaths.AddRange(
            new string[] {
                ResonanceAudioPath,
                ResonanceAudioLibraryPath,
                PFFTPath,
				System.IO.Path.Combine(GetModuleDirectory("AudioMixer"), "Private"),
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
                "AudioMixer",
				"SoundFieldRendering",
                "ProceduralMeshComponent",
            }
        );

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "TargetPlatform"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "InputCore",
                "Projects",
                "AudioExtensions"
            }
        );

        if (Target.bBuildEditor == true)
        {
			PrivateDependencyModuleNames.Add("EditorFramework");
            PrivateDependencyModuleNames.Add("UnrealEd");
            PrivateDependencyModuleNames.Add("Landscape");
        }

		CppCompileWarningSettings.ShadowVariableWarningLevel = WarningLevel.Off;

        AddEngineThirdPartyPrivateStaticDependencies(Target,
                "UEOgg",
                "Vorbis",
                "VorbisFile",
                "Eigen"
                );

		if (bSupportsProceduralMesh)
		{
			PrivateDependencyModuleNames.Add("ProceduralMeshComponent");
			PrivateDefinitions.Add("SUPPORTS_PROCEDURAL_MESH=1");
		}
		else
		{
			PrivateDefinitions.Add("SUPPORTS_PROCEDURAL_MESH=0");
		}

		if (Target.Platform == UnrealTargetPlatform.Android)
        {
            PrivateDefinitions.Add("PFFFT_SIMD_DISABLE=1");
        }
    }
}
