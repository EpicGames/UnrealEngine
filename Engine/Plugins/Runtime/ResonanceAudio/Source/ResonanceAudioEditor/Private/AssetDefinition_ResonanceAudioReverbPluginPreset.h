//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once
#include "AssetDefinitionDefault.h"
#include "ResonanceAudioConstants.h"
#include "ResonanceAudioReverb.h"
#include "AssetDefinition_ResonanceAudioReverbPluginPreset.generated.h"

UCLASS()
class UAssetDefinition_ResonanceAudioReverbPluginPreset : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioReverbPluginPreset", "Resonance Audio Reverb Settings"); }
	virtual FLinearColor GetAssetColor() const override { return ResonanceAudio::ASSET_COLOR; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UResonanceAudioReverbPluginPreset::StaticClass(); }
	virtual bool CanRegisterStatically() const override { return true; }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioReverbPluginPresetSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioReverbPluginPresetSubMenuSection", "Binaural Spatialization"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
