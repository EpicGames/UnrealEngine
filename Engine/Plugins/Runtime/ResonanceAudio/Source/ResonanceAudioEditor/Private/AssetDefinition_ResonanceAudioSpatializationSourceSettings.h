//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once
#include "AssetDefinitionDefault.h"
#include "ResonanceAudioConstants.h"
#include "ResonanceAudioSpatializationSourceSettings.h"
#include "AssetDefinition_ResonanceAudioSpatializationSourceSettings.generated.h"

UCLASS()
class UAssetDefinition_ResonanceAudioSpatializationSourceSettings : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioSpatializationSourceSettings", "Resonance Audio Spatialization Settings"); }
	virtual FLinearColor GetAssetColor() const override { return ResonanceAudio::ASSET_COLOR; }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UResonanceAudioSpatializationSourceSettings::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = 
		{
			FAssetCategoryPath(EAssetCategoryPaths::Audio,
				NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioSpatializationSourceSettingsSubMenu", "Advanced"),
				FCategoryPath(NSLOCTEXT("AssetDefinition", "AssetDefinition_ResonanceAudioSpatializationSourceSettingsSubMenuSection", "Binaural Spatialization"), ECategoryMenuType::Section))
		};

		return Categories;
	}
	// UAssetDefinition End
};
