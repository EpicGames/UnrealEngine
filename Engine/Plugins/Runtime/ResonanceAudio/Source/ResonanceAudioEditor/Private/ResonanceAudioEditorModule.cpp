//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioEditorModule.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "ISettingsModule.h"
#include "Misc/Paths.h"
#include "ResonanceAudioReverbPluginPresetFactory.h"
#include "ResonanceAudioSettings.h"
#include "ResonanceAudioSpatializationSourceSettingsFactory.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

IMPLEMENT_MODULE(ResonanceAudio::FResonanceAudioEditorModule, ResonanceAudioEditor)

DEFINE_LOG_CATEGORY(LogResonanceAudioEditor);

namespace ResonanceAudio
{
	/***********************************************/
	/* Resonance Audio Editor Module               */
	/***********************************************/
	void FResonanceAudioEditorModule::StartupModule()
	{
		// Register plugin settings.
		ISettingsModule* SettingsModule = FModuleManager::Get().GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "Resonance Audio", NSLOCTEXT("ResonanceAudio", "Resonance Audio", "Resonance Audio"),
				NSLOCTEXT("ResonanceAudio", "Configure Resonance Audio settings", "Configure Resonance Audio settings"), GetMutableDefault<UResonanceAudioSettings>());
		}

		// Create and register custom slate style.
		FString ResonanceAudioContent = IPluginManager::Get().FindPlugin(TEXT("ResonanceAudio"))->GetBaseDir() + "/Content";
		FVector2D Vec16 = FVector2D(16.0f, 16.0f);
		FVector2D Vec64 = FVector2D(64.0f, 64.0f);

		ResonanceAudioStyleSet = MakeShareable(new FSlateStyleSet("ResonanceAudio"));
		ResonanceAudioStyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		ResonanceAudioStyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		ResonanceAudioStyleSet->Set("ClassIcon.ResonanceAudioSpatializationSourceSettings", new FSlateImageBrush(ResonanceAudioContent + "/ResonanceAudioSpatializationSourceSettings_16.png", Vec16));
		ResonanceAudioStyleSet->Set("ClassThumbnail.ResonanceAudioSpatializationSourceSettings", new FSlateImageBrush(ResonanceAudioContent + "/ResonanceAudioSpatializationSourceSettings_64.png", Vec64));
		ResonanceAudioStyleSet->Set("ClassIcon.ResonanceAudioReverbPluginPreset", new FSlateImageBrush(ResonanceAudioContent + "/ResonanceAudioReverbPluginPreset_16.png", Vec16));
		ResonanceAudioStyleSet->Set("ClassThumbnail.ResonanceAudioReverbPluginPreset", new FSlateImageBrush(ResonanceAudioContent + "/ResonanceAudioReverbPluginPreset_64.png", Vec64));

		FSlateStyleRegistry::RegisterSlateStyle(*ResonanceAudioStyleSet.Get());
	}

	void FResonanceAudioEditorModule::ShutdownModule()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*ResonanceAudioStyleSet.Get());
	}

} // namespace ResonanceAudio
