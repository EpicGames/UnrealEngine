//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "IResonanceAudioEditorModule.h"

DECLARE_LOG_CATEGORY_EXTERN(LogResonanceAudioEditor, Warning, All);

class FSlateStyleSet;

namespace ResonanceAudio
{
	class FResonanceAudioEditorModule : public IResonanceAudioEditorModule
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

	private:
		TSharedPtr<FSlateStyleSet> ResonanceAudioStyleSet;
	};

} // namespace ResonanceAudio
