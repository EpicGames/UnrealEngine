//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioReverbPluginPresetFactory.h"
#include "AssetTypeCategories.h"
#include "ResonanceAudioReverb.h"

#include "AudioAnalytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioReverbPluginPresetFactory)

UResonanceAudioReverbPluginPresetFactory::UResonanceAudioReverbPluginPresetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UResonanceAudioReverbPluginPreset::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UResonanceAudioReverbPluginPresetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage(TEXT("ResonanceAudio.ReverbPresetCreated"));
	return NewObject<UResonanceAudioReverbPluginPreset>(InParent, InName, Flags);
}

uint32 UResonanceAudioReverbPluginPresetFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Sounds;
}

