//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#include "ResonanceAudioSpatializationSourceSettingsFactory.h"
#include "AssetTypeCategories.h"
#include "ResonanceAudioSpatializationSourceSettings.h"

#include "AudioAnalytics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ResonanceAudioSpatializationSourceSettingsFactory)

UResonanceAudioSpatializationSourceSettingsFactory::UResonanceAudioSpatializationSourceSettingsFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UResonanceAudioSpatializationSourceSettings::StaticClass();

	bCreateNew = true;
	bEditorImport = false;
	bEditAfterNew = true;
}

UObject* UResonanceAudioSpatializationSourceSettingsFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	Audio::Analytics::RecordEvent_Usage(TEXT("ResonanceAudio.SpatializationSourceSettingsCreated"));
	return Cast<UObject>(NewObject<UResonanceAudioSpatializationSourceSettings>(InParent, InName, Flags));
}

uint32 UResonanceAudioSpatializationSourceSettingsFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Sounds;
}

