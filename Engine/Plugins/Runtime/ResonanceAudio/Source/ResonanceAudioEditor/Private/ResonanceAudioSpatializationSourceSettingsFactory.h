//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

#pragma once

#include "Factories/Factory.h"
#include "ResonanceAudioSpatializationSourceSettingsFactory.generated.h"

UCLASS(MinimalAPI, hidecategories = Object)
class UResonanceAudioSpatializationSourceSettingsFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual uint32 GetMenuCategories() const override;
};
