//
// Copyright (C) Google Inc. 2017. All rights reserved.
//

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class ResonanceAudioEditor : ModuleRules
    {
        public ResonanceAudioEditor(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateIncludePaths.AddRange(
                new string[] {
					Path.Combine(GetModuleDirectory("ResonanceAudio"), "Private"),
					Path.Combine(GetModuleDirectory("ResonanceAudio"), "Private", "ResonanceAudioLibrary"),
					Path.Combine(GetModuleDirectory("ResonanceAudio"), "Private", "ResonanceAudioLibrary", "resonance_audio"),
                }
            );

            PublicDependencyModuleNames.AddRange(
                new string[] {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "InputCore",
                    "UnrealEd",
                    "LevelEditor",
                    
                    "RenderCore",
                    "RHI",
                    "AudioEditor",
                    "AudioMixer",
                    "ResonanceAudio"
                }
            );

            PrivateDependencyModuleNames.AddRange(
                new string[] {
	                "AssetDefinition",
                    "Slate",
                    "SlateCore",
					"EditorFramework",
                    "Landscape",
                    "PropertyEditor",
                    "Projects",
                    
					"Eigen"
                 }
            );
        }
    }
}
